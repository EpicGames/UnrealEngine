// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class MovieSceneTracks : ModuleRules
{
	public MovieSceneTracks(ReadOnlyTargetRules Target) : base(Target)
	{
		PublicDependencyModuleNames.AddRange(new string[]
			{
				"Core",
				"CoreUObject",
				"Engine",
				"MovieScene",
				"TimeManagement",
				"AnimationCore",
				"Constraints",
				"AudioMixer",
				"SlateCore" 
			});

		PrivateDependencyModuleNames.AddRange(new string[]
			{
				"AnimGraphRuntime",
				"PropertyPath"
			});

		if (Target.bBuildWithEditorOnlyData && Target.bBuildEditor)
		{
			PublicDependencyModuleNames.AddRange(new string[]
				{
					"BlueprintGraph"
				});
			PrivateDependencyModuleNames.AddRange(new string[]
				{
					"AnimationBlueprintLibrary",
					"DataLayerEditor",
					"EditorFramework",
					"UnrealEd"
				});
		}
	}
}
