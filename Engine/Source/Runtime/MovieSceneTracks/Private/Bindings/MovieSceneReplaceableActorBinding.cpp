// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MovieSceneReplaceableActorBinding.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "MovieScenePossessable.h"
#include "MovieScene.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneBindingReferences.h"
#include "MovieSceneCommonHelpers.h"
#include "Bindings/MovieSceneSpawnableActorBinding.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneReplaceableActorBinding)

#define LOCTEXT_NAMESPACE "MovieScene"


TSubclassOf<UMovieSceneSpawnableBindingBase> UMovieSceneReplaceableActorBinding::GetInnerSpawnableClass() const
{
	return UMovieSceneSpawnableActorBinding::StaticClass();
}


#if WITH_EDITOR

FText UMovieSceneReplaceableActorBinding::GetBindingTypePrettyName() const
{
	return LOCTEXT("MovieSceneReplaceableActorBinding", "Replaceable Actor");
}

FText UMovieSceneReplaceableActorBinding_BPBase::GetBindingTypePrettyName() const
{
	return BindingTypePrettyName.IsEmpty() ? GetClass()->GetDisplayNameText() : BindingTypePrettyName;
}

FText UMovieSceneReplaceableActorBinding_BPBase::GetBindingTrackIconTooltip() const
{
	return BindingTypeTooltip.IsEmpty() ? Super::GetBindingTrackIconTooltip() : BindingTypeTooltip;
}


void UMovieSceneReplaceableActorBinding_BPBase::OnBindingAddedOrChanged(UMovieScene& OwnerMovieScene)
{
	if ((PreviewSpawnableType == nullptr && PreviewSpawnable)
		|| (PreviewSpawnableType != nullptr && (!PreviewSpawnable || PreviewSpawnable->GetClass() != PreviewSpawnableType)))
	{
		if (PreviewSpawnableType == nullptr)
		{
			PreviewSpawnable = nullptr;
		}
		else
		{
			PreviewSpawnable = NewObject<UMovieSceneSpawnableBindingBase>(&OwnerMovieScene, PreviewSpawnableType, NAME_None, RF_Transactional);
		}
	}
}


UMovieSceneCustomBinding* UMovieSceneReplaceableActorBinding_BPBase::CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	// We override this specifically to initialize PreviewSpawnableType on conversions so that we end up with a custom preview from whatever object was passed in in that case.
	UMovieSceneReplaceableActorBinding_BPBase* NewCustomBinding = nullptr;

	const FName TemplateName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), SourceObject ? SourceObject->GetFName() : GetClass()->GetFName());
	const FName InstancedBindingName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), *FString(TemplateName.ToString() + TEXT("_CustomBinding")));

	NewCustomBinding = NewObject<UMovieSceneReplaceableActorBinding_BPBase>(&OwnerMovieScene, GetClass(), InstancedBindingName, RF_Transactional);

	NewCustomBinding->PreviewSpawnable = NewCustomBinding->CreateInnerSpawnable(SourceObject, OwnerMovieScene);
	NewCustomBinding->InitReplaceableBinding(SourceObject, OwnerMovieScene);
	return NewCustomBinding;
}
#endif

FMovieSceneBindingResolveResult UMovieSceneReplaceableActorBinding_BPBase::ResolveRuntimeBindingInternal(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	FMovieSceneBindingResolveContext ResolveContext { SharedPlaybackState->GetPlaybackContext(), FMovieSceneBindingProxy(ResolveParams.ObjectBindingID, ResolveParams.Sequence)};
	return BP_ResolveRuntimeBinding(ResolveContext);
}

void UMovieSceneReplaceableActorBinding_BPBase::InitReplaceableBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	BP_InitReplaceableBinding(SourceObject, &OwnerMovieScene);
}

bool UMovieSceneReplaceableActorBinding_BPBase::SupportsBindingCreationFromObject(const UObject* SourceObject) const
{
	return BP_SupportsBindingCreationFromObject(SourceObject);
}

void UMovieSceneReplaceableActorBinding_BPBase::BP_InitReplaceableBinding_Implementation(UObject* SourceObject, UMovieScene* OwnerMovieScene)
{
	// Default to nothing
}

bool UMovieSceneReplaceableActorBinding_BPBase::BP_SupportsBindingCreationFromObject_Implementation(const UObject* SourceObject) const
{
	// Defaultly we just ensure the object is an actor
	return SourceObject && SourceObject->IsA<AActor>();
}


#undef LOCTEXT_NAMESPACE
