// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MovieSceneReplaceableDirectorBlueprintBinding.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "MovieScenePossessable.h"
#include "MovieScene.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneBindingReferences.h"
#include "MovieSceneCommonHelpers.h"
#include "Bindings/MovieSceneSpawnableActorBinding.h"
#include "MovieSceneDynamicBindingInvoker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneReplaceableDirectorBlueprintBinding)

#define LOCTEXT_NAMESPACE "MovieScene"

#if WITH_EDITOR

FText UMovieSceneReplaceableDirectorBlueprintBinding::GetBindingTypePrettyName() const
{
	return LOCTEXT("MovieSceneReplaceableDirectorBlueprintBinding", "Replaceable from Director Blueprint");
}


void UMovieSceneReplaceableDirectorBlueprintBinding::OnBindingAddedOrChanged(UMovieScene& OwnerMovieScene)
{
	if ((PreviewSpawnableType == nullptr && PreviewSpawnable)
		|| (PreviewSpawnableType != nullptr && (!PreviewSpawnable || PreviewSpawnable->GetClass() != PreviewSpawnableType)))
	{
		if (PreviewSpawnableType == nullptr)
		{
			PreviewSpawnable = nullptr;
		}
		else
		{
			PreviewSpawnable = NewObject<UMovieSceneSpawnableBindingBase>(&OwnerMovieScene, PreviewSpawnableType, NAME_None, RF_Transactional);
		}
	}
}

UMovieSceneCustomBinding* UMovieSceneReplaceableDirectorBlueprintBinding::CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	// We override this specifically to initialize PreviewSpawnableType on conversions so that we end up with a custom preview from whatever object was passed in in that case.
	UMovieSceneReplaceableDirectorBlueprintBinding* NewCustomBinding = nullptr;

	const FName TemplateName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), SourceObject ? SourceObject->GetFName() : GetClass()->GetFName());
	const FName InstancedBindingName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), *FString(TemplateName.ToString() + TEXT("_CustomBinding")));

	NewCustomBinding = NewObject<UMovieSceneReplaceableDirectorBlueprintBinding>(&OwnerMovieScene, GetClass(), InstancedBindingName, RF_Transactional);

	// If no inner spawnable class has been set, and it's available, set it to Spawnable Actor so we at least get some preview when converting an existing binding to this type
	if (!NewCustomBinding->PreviewSpawnableType && UMovieScene::IsCustomBindingClassAllowed(UMovieSceneSpawnableActorBinding::StaticClass()))
	{
		NewCustomBinding->PreviewSpawnableType = UMovieSceneSpawnableActorBinding::StaticClass();
	}

	NewCustomBinding->PreviewSpawnable = NewCustomBinding->CreateInnerSpawnable(SourceObject, OwnerMovieScene);
	NewCustomBinding->InitReplaceableBinding(SourceObject, OwnerMovieScene);
	return NewCustomBinding;
}
#endif

void UMovieSceneReplaceableDirectorBlueprintBinding::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	// If we were duplicated into a different package we can't reference the old function any more
	//    For now we null it out, but it would be good to copy the endpoint as well (we currently can't do that
	//    because there is no way to generically access the correct director BP class from UMovieSceneSequence
	if (DynamicBinding.Function && GetTypedOuter<UMovieScene>() != nullptr)
	{
		UPackage* FunctionPackage = DynamicBinding.Function->GetPackage();
		UPackage* Package = GetPackage();

		if (Package != FunctionPackage)
		{
			DynamicBinding.Function = nullptr;
			DynamicBinding.ResolveParamsProperty = nullptr;
#if WITH_EDITORONLY_DATA
			DynamicBinding.WeakEndpoint = nullptr;
#endif
		}
	}
}

FMovieSceneBindingResolveResult UMovieSceneReplaceableDirectorBlueprintBinding::ResolveRuntimeBindingInternal(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	FMovieSceneBindingResolveResult ResolveResult;
	FMovieSceneDynamicBindingResolveResult DynamicResolveResult = FMovieSceneDynamicBindingInvoker::ResolveDynamicBinding(SharedPlaybackState, ResolveParams.Sequence, ResolveParams.SequenceID, ResolveParams.ObjectBindingID, DynamicBinding);
	ResolveResult.Objects = DynamicResolveResult.Objects;
	ResolveResult.Object = DynamicResolveResult.Object; // Object deprecated 5.7
	return ResolveResult;
}

#undef LOCTEXT_NAMESPACE
