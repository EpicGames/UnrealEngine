// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MovieSceneSpawnableActorBinding.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "Engine/NetDriver.h"
#include "IMovieSceneObjectSpawner.h"
#include "Particles/ParticleSystemComponent.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Misc/PackageName.h"
#include "Engine/Engine.h"
#include "Engine/LevelStreaming.h"
#include "MovieScenePossessable.h"
#include "MovieScene.h"
#include "IMovieScenePlayer.h"
#include "Engine/Blueprint.h"
#include "Engine/BlueprintGeneratedClass.h"
#include "Components/StaticMeshComponent.h"
#include "TransformData.h"
#include "MovieSceneBindingReferences.h"
#include "MovieSceneCommonHelpers.h"
#include "Serialization/ArchiveReplaceOrClearExternalReferences.h"
#include "Systems/MovieSceneDeferredComponentMovementSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSpawnableActorBinding)

static const FName SequencerActorTag("SequencerActor");
static const FName SequencerPreviewActorTag("SequencerPreviewActor");
#define LOCTEXT_NAMESPACE "MovieScene"

namespace UE::MovieSceneTracks
{
class FArchiveClearExternalReferences : public FArchiveReplaceOrClearExternalReferences<UObject>
{
	using Super = FArchiveReplaceOrClearExternalReferences<UObject>;

	TArray<UObject*> RemovedObjects;
public:
	
	FArchiveClearExternalReferences(UObject* InSearchObject, const TMap<UObject*, UObject*>& InReplacementMap, UPackage* InDestPackage)
		: Super(InSearchObject, InReplacementMap, InDestPackage, EArchiveReplaceObjectFlags::NullPrivateRefs | EArchiveReplaceObjectFlags::DelayStart )
	{
		SerializeSearchObject();
	}

	const TArray<UObject*>& GetRemovedObjects() const { return RemovedObjects; }

	virtual FArchive& operator<<(UObject*& Obj) override
	{
		UObject* Original = Obj;
		Super::operator<<(Obj);

		if (Original && !Obj)
		{
			RemovedObjects.AddUnique(Original);
		}
		
		return *this;
	}
};

static void LogRemovedExternalObjects(UObject* ReferencingObject, const TArray<UObject*>& InRemovedObjects)
{
	if (InRemovedObjects.IsEmpty())
	{
		return;
	}

	const FString List = FString::JoinBy(InRemovedObjects, TEXT(", "), [](UObject* Object)
	{
		return ensure(Object) ? Object->GetPathName() : TEXT("null");
	});
	UE_LOGF(LogMovieScene, Warning, "While saving %ls, the following references were cleared because they were private and external: %ls",
		*ReferencingObject->GetPathName(), *List
		);
}
}

UObject* UMovieSceneSpawnableActorBindingBase::SpawnObjectInternal(UWorld* WorldContext, FName SpawnName, const FGuid& BindingId, int32 BindingIndex, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	const EObjectFlags SpawnObjectFlags = RF_Transient | RF_Transactional;

	TSubclassOf<AActor> ActorClass = GetActorClass();
	if (!ActorClass || ActorClass->HasAllClassFlags(CLASS_NewerVersionExists))
	{
		return nullptr;
	}

	AActor* ActorTemplate = GetActorTemplate();

	FString SpawnLabel;

#if WITH_EDITOR
	const IConsoleVariable* AllowSetActorLabelCvar = IConsoleManager::Get().FindConsoleVariable(TEXT("LevelSequence.EnableReadableActorLabelsForSpawnables"));
	const bool bAllowSetActorLabel = AllowSetActorLabelCvar && AllowSetActorLabelCvar->GetBool();
	
	// Historically, setting the actor label has caused performance issues in some scenarios (by causing async loading flushes); however, there's no
	// evidence for this anymore, so the cvar is here to turn off this behavior if needed.
	if ((WorldContext->WorldType == EWorldType::Editor) || bAllowSetActorLabel)
	{
		if (FMovieScenePossessable* Possessable = MovieScene.FindPossessable(BindingId))
		{
			UMovieSceneSequence*                Sequence          = MovieScene.GetTypedOuter<UMovieSceneSequence>();
			const FMovieSceneBindingReferences* BindingReferences = Sequence ? Sequence->GetBindingReferences() : nullptr;

			if (ActorTemplate && BindingReferences && BindingReferences->GetReferences(BindingId).Num() > 1)
			{
				// If there are multiple bound objects, use the Object Template actor label instead of the possessable name
				SpawnLabel = ActorTemplate->GetActorLabel();
			}
			else
			{
				SpawnLabel = Possessable->GetName();
			}
		}
		else
		{
			SpawnLabel = GetDesiredBindingName();
			if (SpawnLabel.Len() == 0)
			{
				SpawnLabel = SpawnName.ToString();
			}
		}
	}
#endif

	// Spawn the actor
	FActorSpawnParameters SpawnInfo;
	{
		SpawnInfo.Name = SpawnName;
		SpawnInfo.ObjectFlags = SpawnObjectFlags;
#if WITH_EDITOR
		SpawnInfo.InitialActorLabel = SpawnLabel;
#endif
		SpawnInfo.SpawnCollisionHandlingOverride = ESpawnActorCollisionHandlingMethod::AlwaysSpawn;
		// allow pre-construction variables to be set.
		SpawnInfo.bDeferConstruction = true;
		SpawnInfo.Template = ActorTemplate;
		SpawnInfo.OverrideLevel = WorldContext->PersistentLevel;
	}

	if (ActorTemplate)
	{
		// Disable all particle components so that they don't auto fire as soon as the actor is spawned. The particles should be triggered through the particle track.
		for (UActorComponent* Component : ActorTemplate->GetComponents())
		{
			if (UParticleSystemComponent* ParticleComponent = Cast<UParticleSystemComponent>(Component))
			{
				// The particle needs to be set inactive in case its template was active.
				ParticleComponent->SetActiveFlag(false);
				Component->bAutoActivate = false;
			}
		}
	}

	FTransform SpawnTransformToUse = GetSpawnTransform();

	AActor* SpawnedActor = WorldContext->SpawnActorAbsolute(ActorClass, SpawnTransformToUse, SpawnInfo);
	if (!SpawnedActor)
	{
		return nullptr;
	}

	if (bNetAddressableName)
	{
		SpawnedActor->SetNetAddressable();
	}

	if (UMovieSceneEntitySystemLinker* Linker = SharedPlaybackState->GetLinker())
	{
		if (UMovieSceneDeferredComponentMovementSystem* DeferredMovementSystem = Linker->FindSystem<UMovieSceneDeferredComponentMovementSystem>())
		{
			for (UActorComponent* ActorComponent : SpawnedActor->GetComponents())
			{
				if (USceneComponent* SceneComponent = Cast<USceneComponent>(ActorComponent))
				{
					DeferredMovementSystem->DeferMovementUpdates(SceneComponent);
				}
			}
		}
	}
	
	// The below block duplicates code in UMovieSceneSpawnableBinding, but this ensures that at least for this
	// type of binding, these tags are added before calling FinishSpawning, as some client code may check for this tag
	// in component initialization.
	{
	#if WITH_EDITOR
		SpawnedActor->bIsEditorPreviewActor = false;
	#endif

		// tag this actor so we know it was spawned by sequencer
		SpawnedActor->Tags.AddUnique(SequencerActorTag);
	}

	const bool bIsDefaultTransform = true;
	SpawnedActor->FinishSpawning(SpawnTransformToUse, bIsDefaultTransform);

	return SpawnedActor;
}

void UMovieSceneSpawnableActorBindingBase::DestroySpawnedObjectInternal(UObject* Object)
{
	AActor* Actor = Cast<AActor>(Object);
	if (!ensure(Actor))
	{
		return;
	}

	UWorld* World = Actor->GetWorld();
	if (World)
	{
		UE::Net::FScopedIgnoreStaticActorDestruction ScopedIgnoreDestruction; // net addressable actors normally have a destruction info created for them, we don't want that as these actors were spawned at runtime
		const bool bNetForce = false;
		const bool bShouldModifyLevel = false;
		World->DestroyActor(Actor, bNetForce, bShouldModifyLevel);
	}
}

void UMovieSceneSpawnableActorBindingBase::AutoSetNetAddressableName()
{
	bNetAddressableName = false;

	AActor* ActorTemplatePtr = GetActorTemplate();
	if (ActorTemplatePtr && ActorTemplatePtr->FindComponentByClass<UStaticMeshComponent>() != nullptr)
	{
		bNetAddressableName = true;
	}
}

FTransform UMovieSceneSpawnableActorBindingBase::GetSpawnTransform() const
{
	FTransform ReturnSpawnTransform;

	if (AActor* ActorTemplatePtr = GetActorTemplate())
	{
		if (USceneComponent* RootComponent = ActorTemplatePtr->GetRootComponent())
		{
			ReturnSpawnTransform.SetTranslation(RootComponent->GetRelativeLocation());
			ReturnSpawnTransform.SetRotation(RootComponent->GetRelativeRotation().Quaternion());
			ReturnSpawnTransform.SetScale3D(RootComponent->GetRelativeScale3D());
		}
	}
	return ReturnSpawnTransform;
}


ULevelStreaming* GetLevelStreamingHelper(const FName& DesiredLevelName, const UWorld* World)
{
	if (DesiredLevelName == NAME_None)
	{
		return nullptr;
	}

	const TArray<ULevelStreaming*>& StreamingLevels = World->GetStreamingLevels();
	FString SafeLevelNameString = DesiredLevelName.ToString();
	if (FPackageName::IsShortPackageName(SafeLevelNameString))
	{
		// Make sure MyMap1 and Map1 names do not resolve to a same streaming level
		SafeLevelNameString.InsertAt(0, '/');
	}

#if WITH_EDITOR
	FWorldContext* WorldContext = GEngine->GetWorldContextFromWorld(World);
	if (WorldContext && WorldContext->PIEInstance != INDEX_NONE)
	{
		SafeLevelNameString = UWorld::ConvertToPIEPackageName(SafeLevelNameString, WorldContext->PIEInstance);
	}
#endif


	for (ULevelStreaming* LevelStreaming : StreamingLevels)
	{
		if (LevelStreaming && LevelStreaming->GetWorldAssetPackageName().EndsWith(SafeLevelNameString, ESearchCase::IgnoreCase))
		{
			return LevelStreaming;
		}
	}

	return nullptr;
}

UWorld* UMovieSceneSpawnableActorBinding::GetWorldContext(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext();
	UWorld* WorldContext = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	FName DesiredLevelName = LevelName;
	if (DesiredLevelName != NAME_None)
	{
		if (WorldContext && WorldContext->GetFName() == DesiredLevelName)
		{
			// done, spawn into this world
		}
		else
		{
			ULevelStreaming* LevelStreaming = GetLevelStreamingHelper(DesiredLevelName, WorldContext);
			if (LevelStreaming && LevelStreaming->GetWorldAsset().IsValid())
			{
				WorldContext = LevelStreaming->GetWorldAsset().Get();
			}
			else
			{
				UE_LOGF(LogMovieScene, Warning, "Can't find sublevel '%ls' to spawn into, defaulting to Persistent level", *DesiredLevelName.ToString());
			}
		}
	}
	return WorldContext;
}

FName UMovieSceneSpawnableActorBindingBase::GetSpawnName(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	// We use the net addressable name for spawnable actors on any non-editor, non-standalone world (ie, all clients, servers and PIE worlds)

	UWorld* WorldContext = GetWorldContext(SharedPlaybackState);
	const bool bUseNetAddressableName = bNetAddressableName && WorldContext && (WorldContext->WorldType != EWorldType::Editor) && (WorldContext->GetNetMode() != ENetMode::NM_Standalone);
	FString DesiredBindingName = GetDesiredBindingName();
	if (FMovieScenePossessable* Possessable = MovieScene.FindPossessable(BindingId))
	{
		if (DesiredBindingName.IsEmpty())
		{
			DesiredBindingName = Possessable->GetName();
		}

#if WITH_EDITOR
		UClass* ActorClass = GetActorClass();
		if (bUseNetAddressableName)
		{
			return GetNetAddressableName(SharedPlaybackState, BindingId, TemplateID, DesiredBindingName);
		}
		else if (ensure(WorldContext != nullptr && WorldContext->PersistentLevel))
		{
			return MakeUniqueObjectName(WorldContext->PersistentLevel, ActorClass ? ActorClass : AActor::StaticClass(), *DesiredBindingName);
		}
#else
		return bUseNetAddressableName ? GetNetAddressableName(SharedPlaybackState, BindingId, TemplateID, DesiredBindingName) : NAME_None;
#endif
	}
	return NAME_None;
}

FName UMovieSceneSpawnableActorBindingBase::GetNetAddressableName(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, const FGuid& BindingId, FMovieSceneSequenceID SequenceID, const FString& BaseName) const
{
	UObject* AddressingContext = nullptr;

	if (IMovieScenePlayer* Player = UE::MovieScene::FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState))
	{
		AddressingContext = Player->AsUObject();
	}
	if (!AddressingContext)
	{
		if (UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext())
		{
			AddressingContext = PlaybackContext;
		}
	}

	if (!AddressingContext)
	{
		return NAME_None;
	}

	TStringBuilder<128> AddressableName;

	// Spawnable name
	AddressableName.Append(*BaseName, BaseName.Len());

	// SequenceID
	AddressableName.Appendf(TEXT("_0x%08X"), SequenceID.GetInternalValue());

	// Spawnable GUID
	AddressableName.Appendf(TEXT("_%08X%08X%08X%08X"), BindingId.A, BindingId.B, BindingId.C, BindingId.D);

	// Actor / player Name
	if (AActor* OuterActor = AddressingContext->GetTypedOuter<AActor>())
	{
		AddressableName.AppendChar(TEXT('_'));
		OuterActor->GetFName().AppendString(AddressableName);
	}
	else
	{
		AddressableName.AppendChar(TEXT('_'));
		AddressingContext->GetFName().AppendString(AddressableName);
	}

	return FName(AddressableName.Len(), AddressableName.GetData());
}

TSubclassOf<AActor> UMovieSceneSpawnableActorBinding::GetActorClass() const
{
	if (AActor* ActorTemplatePtr = GetActorTemplate())
	{
		return ActorTemplatePtr->GetClass();
	}
	return nullptr;
}


struct FIsSpawnable
{
	FIsSpawnable() : bIsSpawnable(false) {}
	explicit FIsSpawnable(bool bInIsSpawnable) : bIsSpawnable(bInIsSpawnable) {}

	bool IsDefault() const { return !bIsSpawnable; }

	bool bIsSpawnable;
};

void UMovieSceneSpawnableActorBinding::SetObjectTemplate(UObject* InObjectTemplate)
{
	ensure(InObjectTemplate == nullptr || InObjectTemplate->IsA<AActor>());
	checkf(InObjectTemplate == nullptr || !InObjectTemplate->HasAnyFlags(RF_ClassDefaultObject), TEXT("Setting CDOs as object templates is not supported. Please use the class directly."));
	ActorTemplate = Cast<AActor>(InObjectTemplate);
	if (ActorTemplate)
	{
		// TODO: We should move this out of FMovieSceneSpawnable eventually
		FMovieSceneSpawnable::MarkSpawnableTemplate(*ActorTemplate);
	}
	AutoSetNetAddressableName();
}

void UMovieSceneSpawnableActorBinding::CopyObjectTemplate(UObject* InSourceObject, UMovieSceneSequence& MovieSceneSequence)
{
	if (!InSourceObject)
	{
		return;
	}
	ensure(InSourceObject->IsA<AActor>());
	if (AActor* SourceActor = Cast<AActor>(InSourceObject))
	{
		const FName ActorName = ActorTemplate ? ActorTemplate->GetFName() : SourceActor->GetFName();

		if (ActorTemplate)
		{
			const FString NewName = MakeUniqueObjectName(MovieSceneSequence.GetMovieScene(), GetActorClass(), "ExpiredSpawnable").ToString();
			// Without REN_DontCreateRedirectors, we'd create a redirector causing a name collision when we call MakeSpawnableTemplateFromInstance.
			ActorTemplate->Rename(*NewName, nullptr, REN_DoNotDirty | REN_DontCreateRedirectors | REN_NonTransactional);
			ActorTemplate->MarkAsGarbage();
			ActorTemplate = nullptr;
		}

		ActorTemplate = Cast<AActor>(MovieSceneSequence.MakeSpawnableTemplateFromInstance(*SourceActor, ActorName));

		check(ActorTemplate);

		// TODO: We should move this out of FMovieSceneSpawnable eventually
		FMovieSceneSpawnable::MarkSpawnableTemplate(*ActorTemplate);

		MovieSceneSequence.MarkPackageDirty();
	}
}

bool UMovieSceneSpawnableActorBinding::SupportsBindingCreationFromObject(const UObject* SourceObject) const
{
	if (!SourceObject)
	{
		// In this case we would just make an empty binding
		return true;
	}
	else if (SourceObject->IsA<AActor>())
	{
		return true;
	}
	else if (const UBlueprint* SourceBlueprint = Cast<const UBlueprint>(SourceObject))
	{
		return SourceBlueprint->GeneratedClass->IsChildOf(AActor::StaticClass());
	}
#if WITH_EDITORONLY_DATA
	else if (const UBlueprintGeneratedClass* SourceBlueprintGeneratedClass = Cast<const UBlueprintGeneratedClass>(SourceObject))
	{
		if (const UBlueprint* BlueprintGeneratedBy = Cast<const UBlueprint>(SourceBlueprintGeneratedClass->ClassGeneratedBy))
		{
			return BlueprintGeneratedBy->GeneratedClass->IsChildOf(AActor::StaticClass());
		}
	}
#endif
	else if (const UClass* InClass = Cast<const UClass>(SourceObject))
	{
		return InClass->IsChildOf(AActor::StaticClass());
	}

	return false;
}

UMovieSceneCustomBinding* UMovieSceneSpawnableActorBinding::CreateNewCustomBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	UMovieSceneSpawnableActorBinding* NewCustomBinding = nullptr;

	const FName TemplateName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), SourceObject ? SourceObject->GetFName() : TEXT("EmptyBinding"));
	const FName InstancedBindingName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), *FString(TemplateName.ToString() + TEXT("_CustomBinding")));

	auto CreateBinding = [&]()
	{
		return NewObject<UMovieSceneSpawnableActorBinding>(&OwnerMovieScene, GetClass(), InstancedBindingName, RF_Transactional);
	};

	// Deal with creating a spawnable from an instance of an actor
	if (AActor* Actor = Cast<AActor>(SourceObject))
	{
		// Remove any previous tags- new ones will be added as necessary during spawning
		Actor->Tags.Remove(SequencerActorTag);
		Actor->Tags.Remove(SequencerPreviewActorTag);

		// If the source actor is not transactional, temporarily add the flag to ensure that the duplicated object is created with the transactional flag.
		// This is necessary for the creation of the object to exist in the transaction buffer for multi-user workflows
		const bool bWasTransactional = Actor->HasAnyFlags(RF_Transactional);
		if (!bWasTransactional)
		{
			Actor->SetFlags(RF_Transactional);
		}

		NewCustomBinding = CreateBinding();

		if (NewCustomBinding)
		{
			AActor* SpawnedActor = Cast<AActor>(StaticDuplicateObject(Actor, &OwnerMovieScene, TemplateName, RF_AllFlags & ~RF_Transient));
			SpawnedActor->DetachFromActor(FDetachmentTransformRules(EDetachmentRule::KeepRelative, false));
#if WITH_EDITORONLY_DATA
			SpawnedActor->bIsEditorPreviewActor = false;
#endif
			NewCustomBinding->SetObjectTemplate(SpawnedActor);

			if (!bWasTransactional)
			{
				Actor->ClearFlags(RF_Transactional);
			}

			// This achieves that the template actor keeps no reference to any external private objects.
			// Such references would cause a crash when saving the owning UMovieScene / sequence.
			// Example: An actor that has a reference to another actor in the level.
			// SpawnedActor keep referencing the actor in the original map, which is a private external object.
			// FArchiveReplaceOrClearExternalReferences handles recursively serializing subobjects, such as components.
			UPackage* NewPackage = SpawnedActor->GetOutermost();
			const UE::MovieSceneTracks::FArchiveClearExternalReferences ReplaceActorInvalidReferences(SpawnedActor, {}, NewPackage);
			// User should be warned in case they notice weird behaviour (e.g. if the spawnable actor is selected in details panel,
			// saving nulls out the references which the user may notice).
			UE::MovieSceneTracks::LogRemovedExternalObjects(this, ReplaceActorInvalidReferences.GetRemovedObjects());
		}
	}
	// If it's a blueprint, we need some special handling
	else if (UBlueprint* SourceBlueprint = Cast<UBlueprint>(SourceObject))
	{
		if (!OwnerMovieScene.GetClass()->IsChildOf(SourceBlueprint->GeneratedClass->ClassWithin) || !SourceBlueprint->GeneratedClass->IsChildOf(AActor::StaticClass()))
		{
			return nullptr;
		}
		NewCustomBinding = CreateBinding();
		if (NewCustomBinding)
		{
			NewCustomBinding->SetObjectTemplate(NewObject<UObject>(NewCustomBinding, SourceBlueprint->GeneratedClass, TemplateName, RF_Transactional));
		}
	}
#if WITH_EDITORONLY_DATA
	else if (UBlueprintGeneratedClass* SourceBlueprintGeneratedClass = Cast<UBlueprintGeneratedClass>(SourceObject))
	{
		if (UBlueprint* BlueprintGeneratedBy = Cast<UBlueprint>(SourceBlueprintGeneratedClass->ClassGeneratedBy))
		{
			if (!OwnerMovieScene.GetClass()->IsChildOf(BlueprintGeneratedBy->GeneratedClass->ClassWithin) || !BlueprintGeneratedBy->GeneratedClass->IsChildOf(AActor::StaticClass()))
			{
				return nullptr;
			}

			NewCustomBinding = CreateBinding();

			if (NewCustomBinding)
			{
				NewCustomBinding->SetObjectTemplate(NewObject<UObject>(NewCustomBinding, BlueprintGeneratedBy->GeneratedClass, TemplateName, RF_Transactional));
			}
		}
	}
#endif

	if (!NewCustomBinding)
	{
		if (UClass* InClass = Cast<UClass>(SourceObject ? SourceObject : AActor::StaticClass()))
		{
			if (!InClass->IsChildOf(AActor::StaticClass()))
			{		
				return nullptr;
			}

			NewCustomBinding = CreateBinding();
			if (NewCustomBinding)
			{
				NewCustomBinding->SetObjectTemplate(NewObject<UObject>(&OwnerMovieScene, InClass, TemplateName, RF_Transactional));
			}
		}
	}

	return NewCustomBinding;
}

#if WITH_EDITOR
bool UMovieSceneSpawnableActorBinding::SupportsConversionFromBinding(const FMovieSceneBindingReference& BindingReference, const UObject* SourceObject) const
{
	return SupportsBindingCreationFromObject(SourceObject);
}

UMovieSceneCustomBinding* UMovieSceneSpawnableActorBinding::CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	return CreateNewCustomBinding(SourceObject, OwnerMovieScene);
}

FText UMovieSceneSpawnableActorBinding::GetBindingTypePrettyName() const
{
	return LOCTEXT("MovieSceneSpawnableActorBinding", "Spawnable Actor");
}
#endif

void UMovieSceneSpawnableActorBinding::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	// Post duplication, the Actor Template will still be pointing to the original Sequence.
	// We therefore need to rebuild the object template
	if (ActorTemplate != nullptr)
	{
		// Only duplicate the inner ActorTemplate if this binding has been bound into a different valid UMovieScene.
		if (UMovieScene* OuterMovieScene = GetTypedOuter<UMovieScene>(); 
			OuterMovieScene != nullptr && ActorTemplate->GetTypedOuter<UMovieScene>() != OuterMovieScene)
		{
			// Reparent the actor template to the duplicated movie scene
			ActorTemplate = Cast<AActor>(StaticDuplicateObject(ActorTemplate, OuterMovieScene));
			if (ActorTemplate)
			{
				FMovieSceneSpawnable::MarkSpawnableTemplate(*ActorTemplate);
			}
		}
	}
}

#undef LOCTEXT_NAMESPACE
