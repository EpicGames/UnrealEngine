// Copyright Epic Games, Inc. All Rights Reserved.

#include "Bindings/MovieSceneSpawnableDirectorBlueprintBinding.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "MovieScene.h"
#include "MovieSceneDynamicBindingInvoker.h"
#include "Engine/World.h"
#include "Engine/Level.h"
#include "MovieSceneSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSpawnableDirectorBlueprintBinding)

#define LOCTEXT_NAMESPACE "MovieScene"

UObject* UMovieSceneSpawnableDirectorBlueprintBinding::SpawnObjectInternal(UWorld* WorldContext, FName SpawnName, const FGuid& BindingId, int32 BindingIndex, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	FMovieSceneDynamicBindingResolveResult ResolveResult = FMovieSceneDynamicBindingInvoker::ResolveDynamicBinding(SharedPlaybackState, MovieScene.GetTypedOuter<UMovieSceneSequence>(), TemplateID, BindingId, DynamicBinding);

	for (TObjectPtr<UObject> Object : ResolveResult.Objects)
	{
		if (Object.Get())
		{
			return Object.Get();
		}
	}
	return nullptr;
}

void UMovieSceneSpawnableDirectorBlueprintBinding::DestroySpawnedObjectInternal(UObject* Object)
{
	AActor* Actor = Cast<AActor>(Object);
	if (!Actor)
	{
		// TODO: Consider doing something here for non actors? Does this all need to get moved to a base class?
		return;
	}

#if WITH_EDITOR
	if (GIsEditor)
	{
		// Explicitly remove RF_Transactional on spawned actors since we don't want to trasact spawn/destroy events
		// This particular UObject will have RF_Transactional cleared by the caller, but we need to cleared it on the components.
		for (UActorComponent* Component : Actor->GetComponents())
		{
			if (Component)
			{
				Component->ClearFlags(RF_Transactional);
			}
		}
	}
#endif

	UWorld* World = Actor->GetWorld();
	if (World)
	{
		const bool bNetForce = false;
		const bool bShouldModifyLevel = false;
		World->DestroyActor(Actor, bNetForce, bShouldModifyLevel);
	}
}

UWorld* UMovieSceneSpawnableDirectorBlueprintBinding::GetWorldContext(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext();
	UWorld* WorldContext = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
	return WorldContext;
}

FName UMovieSceneSpawnableDirectorBlueprintBinding::GetSpawnName(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	// We use the net addressable name for spawnable actors on any non-editor, non-standalone world (ie, all clients, servers and PIE worlds)

	UWorld* WorldContext = GetWorldContext(SharedPlaybackState);
	FString DesiredBindingName = GetDesiredBindingName();
	if (FMovieScenePossessable* Possessable = MovieScene.FindPossessable(BindingId))
	{
		if (DesiredBindingName.IsEmpty())
		{
			DesiredBindingName = Possessable->GetName();
		}
	}

#if WITH_EDITOR
	UClass* ObjectClass = GetBoundObjectClass();
	if (ensure(WorldContext != nullptr && WorldContext->PersistentLevel))
	{
		return MakeUniqueObjectName(WorldContext->PersistentLevel.Get(), ObjectClass ? ObjectClass : UObject::StaticClass(), *DesiredBindingName);
	}
#endif
	return *DesiredBindingName;
}

bool UMovieSceneSpawnableDirectorBlueprintBinding::SupportsBindingCreationFromObject(const UObject* SourceObject) const
{
	return true;
}

UMovieSceneCustomBinding* UMovieSceneSpawnableDirectorBlueprintBinding::CreateNewCustomBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	UMovieSceneSpawnableDirectorBlueprintBinding* NewCustomBinding = nullptr;

	const FName TemplateName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), SourceObject ? SourceObject->GetFName() : GetClass()->GetFName());
	const FName InstancedBindingName = MakeUniqueObjectName(&OwnerMovieScene, UObject::StaticClass(), *FString(TemplateName.ToString() + TEXT("_CustomBinding")));

	return NewObject<UMovieSceneSpawnableDirectorBlueprintBinding>(&OwnerMovieScene, UMovieSceneSpawnableDirectorBlueprintBinding::StaticClass(), InstancedBindingName, RF_Transactional);
}

#if WITH_EDITOR
bool UMovieSceneSpawnableDirectorBlueprintBinding::SupportsConversionFromBinding(const FMovieSceneBindingReference& BindingReference, const UObject* SourceObject) const
{
	return SupportsBindingCreationFromObject(SourceObject);
}

UMovieSceneCustomBinding* UMovieSceneSpawnableDirectorBlueprintBinding::CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene)
{
	return CreateNewCustomBinding(SourceObject, OwnerMovieScene);
}

FText UMovieSceneSpawnableDirectorBlueprintBinding::GetBindingTypePrettyName() const
{
	return LOCTEXT("UMovieSceneSpawnableDirectorBlueprintBinding", "Spawnable from Director Blueprint");
}

FText UMovieSceneSpawnableDirectorBlueprintBinding::GetBindingTrackIconTooltip() const
{
	return LOCTEXT("CustomSpawnableDirectorBlueprintTooltip", "This item is spawned by sequencer by a user-specified Director Blueprint endpoint.");
}
#endif

void UMovieSceneSpawnableDirectorBlueprintBinding::PostDuplicate(EDuplicateMode::Type DuplicateMode)
{
	Super::PostDuplicate(DuplicateMode);

	// If we were duplicated into a different package we can't reference the old function any more
	//    For now we null it out, but it would be good to copy the endpoint as well (we currently can't do that
	//    because there is no way to generically access the correct director BP class from UMovieSceneSequence
	if (DynamicBinding.Function && GetTypedOuter<UMovieScene>() != nullptr)
	{
		UPackage* FunctionPackage = DynamicBinding.Function->GetPackage();
		UPackage* Package = GetPackage();

		if (Package != FunctionPackage)
		{
			DynamicBinding.Function = nullptr;
			DynamicBinding.ResolveParamsProperty = nullptr;
		#if WITH_EDITORONLY_DATA
			DynamicBinding.WeakEndpoint = nullptr;
		#endif
		}
	}
}

#undef LOCTEXT_NAMESPACE
