// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneActorReferenceChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneActorReferenceChannel)

bool FMovieSceneActorReferenceChannel::Evaluate(FFrameTime InTime, FMovieSceneActorReferenceKey& OutValue) const
{
	if (KeyTimes.Num())
	{
		const int32 Index = FMath::Max(0, Algo::UpperBound(KeyTimes, InTime.FrameNumber)-1);
		OutValue = KeyValues[Index];
		return true;
	}

	OutValue = DefaultValue;
	return true;
}

void FMovieSceneActorReferenceChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneActorReferenceChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneActorReferenceChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneActorReferenceChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneActorReferenceChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneActorReferenceChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	// Insert a key at the current time to maintain evaluation
	if (GetData().GetTimes().Num() > 0)
	{
		FMovieSceneActorReferenceKey Value;
		Evaluate(InTime, Value);
		GetData().UpdateOrAddKey(InTime, Value);
	}

	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

FKeyHandle FMovieSceneActorReferenceChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneActorReferenceChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}

void FMovieSceneActorReferenceChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneActorReferenceChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneActorReferenceChannel::GetNumKeys() const
{
	return KeyTimes.Num();
}

void FMovieSceneActorReferenceChannel::Reset()
{
	KeyTimes.Reset();
	KeyValues.Reset();
	KeyHandles.Reset();
	DefaultValue = FMovieSceneActorReferenceKey();
}

void FMovieSceneActorReferenceChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}

void FMovieSceneActorReferenceChannel::ClearDefault()
{
	DefaultValue = FMovieSceneActorReferenceKey();
}

