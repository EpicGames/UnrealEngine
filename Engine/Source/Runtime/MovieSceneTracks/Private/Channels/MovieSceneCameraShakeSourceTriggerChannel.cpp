// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneCameraShakeSourceTriggerChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraShakeSourceTriggerChannel)

void FMovieSceneCameraShakeSourceTriggerChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneCameraShakeSourceTriggerChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneCameraShakeSourceTriggerChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneCameraShakeSourceTriggerChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneCameraShakeSourceTriggerChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneCameraShakeSourceTriggerChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneCameraShakeSourceTriggerChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneCameraShakeSourceTriggerChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneCameraShakeSourceTriggerChannel::GetNumKeys() const
{
	return KeyTimes.Num();
}

void FMovieSceneCameraShakeSourceTriggerChannel::Reset()
{
	KeyTimes.Reset();
	KeyValues.Reset();
	KeyHandles.Reset();
}

void FMovieSceneCameraShakeSourceTriggerChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}


