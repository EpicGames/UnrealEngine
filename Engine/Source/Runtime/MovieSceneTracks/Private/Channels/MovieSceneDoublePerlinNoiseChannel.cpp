// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneDoublePerlinNoiseChannel.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDoublePerlinNoiseChannel)

FMovieSceneDoublePerlinNoiseChannel::FMovieSceneDoublePerlinNoiseChannel()
{
}

double FMovieSceneDoublePerlinNoiseChannel::Evaluate(double InSeconds) const
{
	return Evaluate(PerlinNoiseParams, InSeconds);
}

bool FMovieSceneDoublePerlinNoiseChannel::Evaluate(const UMovieSceneSection* InSection, FFrameTime InTime, double& OutValue) const
{
	if (ensure(InSection))
	{
		UMovieScene* MovieScene = InSection->GetTypedOuter<UMovieScene>();
		if (MovieScene)
		{
			const FFrameRate TickResolution = MovieScene->GetTickResolution();
			const double Seconds = TickResolution.AsSeconds(InTime);
			OutValue = Evaluate(Seconds);
			return true;
		}
	}
	return false;
}

double FMovieSceneDoublePerlinNoiseChannel::Evaluate(const FPerlinNoiseParams& InParams, double InSeconds)
{
	double Result = static_cast<double>(FMath::PerlinNoise1D((InSeconds + InParams.Offset) * InParams.Frequency));
	Result *= InParams.Amplitude;
	return Result;
}

