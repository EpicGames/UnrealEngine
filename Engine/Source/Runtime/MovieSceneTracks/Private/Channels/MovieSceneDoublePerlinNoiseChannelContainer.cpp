// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneDoublePerlinNoiseChannelContainer.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Misc/Guid.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDoublePerlinNoiseChannelContainer)

void UMovieSceneDoublePerlinNoiseChannelContainer::InitializeOverride(FMovieSceneChannel* InChannel)
{
	PerlinNoiseChannel.PerlinNoiseParams.RandomizeOffset();
}

bool UMovieSceneDoublePerlinNoiseChannelContainer::SupportsOverride(FName DefaultChannelTypeName) const
{
	return FMovieSceneDoubleChannel::StaticStruct()->GetFName() == DefaultChannelTypeName;
}

void UMovieSceneDoublePerlinNoiseChannelContainer::ImportEntityImpl(const UE::MovieScene::FChannelOverrideEntityImportParams& OverrideParams, const UE::MovieScene::FEntityImportParams& ImportParams, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();
	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	TComponentTypeID<double> ResultComponent = OverrideParams.ResultComponent.ReinterpretCast<double>();

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(TracksComponents->DoublePerlinNoiseChannel, PerlinNoiseChannel.PerlinNoiseParams)
		.Add(ResultComponent, TNumericLimits<double>::Max())
	);
}

#if WITH_EDITOR
FMovieSceneChannelHandle UMovieSceneDoublePerlinNoiseChannelContainer::AddChannelProxy(FName ChannelName, FMovieSceneChannelProxyData& ProxyData, const FMovieSceneChannelMetaData& MetaData)
{
	FMovieSceneChannelMetaData NoiseMetaData(MetaData);
	NoiseMetaData.bCanBeKeyed = false;
	return ProxyData.AddWithDefaultEditorData(PerlinNoiseChannel, NoiseMetaData);
}
#else
void UMovieSceneDoublePerlinNoiseChannelContainer::AddChannelProxy(FName ChannelName, FMovieSceneChannelProxyData& ProxyData)
{
	ProxyData.Add(PerlinNoiseChannel);
}
#endif

