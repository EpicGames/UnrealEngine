// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneEvent.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEvent)

UClass* FMovieSceneEvent::GetBoundObjectPropertyClass() const 
{ 
	if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Ptrs.BoundObjectProperty.Get()))
	{
		return ObjectProperty->PropertyClass; 
	}
	else if (FInterfaceProperty* InterfaceProperty = CastField<FInterfaceProperty>(Ptrs.BoundObjectProperty.Get()))
	{
		return InterfaceProperty->InterfaceClass;
	}
	return nullptr; 
}

