// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneEventChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEventChannel)

void FMovieSceneEventChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneEventChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneEventChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneEventChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneEventChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneEventChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneEventChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}


TRange<FFrameNumber> FMovieSceneEventChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneEventChannel::GetNumKeys() const
{
	return KeyTimes.Num();
}

void FMovieSceneEventChannel::Reset()
{
	KeyTimes.Reset();
	KeyValues.Reset();
	KeyHandles.Reset();
}

void FMovieSceneEventChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}

FKeyHandle FMovieSceneEventChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneEventChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}
