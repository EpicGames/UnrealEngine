// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneStringChannel.h"
#include "Curves/StringCurve.h"
#include "MovieSceneFwd.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneFrameMigration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneStringChannel)

const FString* FMovieSceneStringChannel::Evaluate(FFrameTime InTime) const
{
	if (Times.Num())
	{
		const int32 Index = FMath::Max(0, Algo::UpperBound(Times, InTime.FrameNumber)-1);
		return &Values[Index];
	}
	
	return bHasDefaultValue ? &DefaultValue : nullptr;
}

bool FMovieSceneStringChannel::SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot)
{
	static const FName StringCurveName("StringCurve");
	if (Tag.GetType().IsStruct(StringCurveName))
	{
		FStringCurve StringCurve;
		FStringCurve::StaticStruct()->SerializeItem(Slot, &StringCurve, nullptr);

		FString NewDefault = StringCurve.GetDefaultValue();
		if (NewDefault.Len())
		{
			bHasDefaultValue = true;
			DefaultValue = MoveTemp(NewDefault);
		}

		FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();

		Times.Reserve(StringCurve.GetNumKeys());
		Values.Reserve(StringCurve.GetNumKeys());
		int32 Index = 0;
		for (const FStringCurveKey& Key : StringCurve.GetKeys())
		{
			FFrameNumber KeyTime = UpgradeLegacyMovieSceneTime(nullptr, LegacyFrameRate, Key.Time);

			FString Val(Key.Value);
			ConvertInsertAndSort<FString>(Index++, KeyTime, Val, Times, Values);
		}
		return true;
	}

	return false;
}

void FMovieSceneStringChannel::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneStringChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneStringChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneStringChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneStringChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneStringChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	// Insert a key at the current time to maintain evaluation
	if (GetData().GetTimes().Num() > 0)
	{
		if (const FString* CurrentValue = Evaluate(InTime))
		{
			FString Value(*CurrentValue);
			GetData().UpdateOrAddKey(InTime, *Value);
		}
	}

	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

FKeyHandle FMovieSceneStringChannel::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneStringChannel::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}

void FMovieSceneStringChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneStringChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneStringChannel::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneStringChannel::Reset()
{
	Times.Reset();
	Values.Reset();
	KeyHandles.Reset();
	bHasDefaultValue = false;
}

void FMovieSceneStringChannel::Optimize(const FKeyDataOptimizationParams& InParameters)
{
	UE::MovieScene::Optimize(this, InParameters);
}

void FMovieSceneStringChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}

void FMovieSceneStringChannel::ClearDefault()
{
	bHasDefaultValue = false;
}
