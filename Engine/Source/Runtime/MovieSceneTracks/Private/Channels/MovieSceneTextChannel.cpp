// Copyright Epic Games, Inc. All Rights Reserved.

#include "Channels/MovieSceneTextChannel.h"

#if WITH_EDITOR
#include "UObject/Package.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTextChannel)

#if WITH_EDITOR
void FMovieSceneTextChannel::SetPackage(UPackage* InPackage)
{
	Package = InPackage;
}

UPackage* FMovieSceneTextChannel::GetPackage() const
{
	return Package.Get();
}
#endif

const FText* FMovieSceneTextChannel::Evaluate(FFrameTime InTime) const
{
	if (Times.Num())
	{
		const int32 Index = FMath::Max(0, Algo::UpperBound(Times, InTime.FrameNumber)-1);
		return &Values[Index];
	}
	return bHasDefaultValue ? &DefaultValue : nullptr;
}

void FMovieSceneTextChannel::GetKeys(const TRange<FFrameNumber>& WithinRange
	, TArray<FFrameNumber>* OutKeyTimes
	, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneTextChannel::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneTextChannel::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneTextChannel::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneTextChannel::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneTextChannel::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	// Insert a key at the current time to maintain evaluation
	if (GetData().GetTimes().Num() > 0)
	{
		if (const FText* CurrentValue = Evaluate(InTime))
		{
			FText Value(*CurrentValue);
			GetData().UpdateOrAddKey(InTime, Value);
		}
	}
	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneTextChannel::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneTextChannel::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneTextChannel::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneTextChannel::Reset()
{
	Times.Reset();
	Values.Reset();
	KeyHandles.Reset();
	bHasDefaultValue = false;
}

void FMovieSceneTextChannel::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}

void FMovieSceneTextChannel::Optimize(const FKeyDataOptimizationParams& InParameters)
{
	UE::MovieScene::Optimize(this, InParameters);
}

void FMovieSceneTextChannel::ClearDefault()
{
	bHasDefaultValue = false;
}
