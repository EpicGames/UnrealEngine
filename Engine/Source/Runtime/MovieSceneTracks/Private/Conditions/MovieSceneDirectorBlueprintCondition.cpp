// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/MovieSceneDirectorBlueprintCondition.h"
#include "CoreMinimal.h"
#include "Engine/World.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/SequenceDirectorPlaybackCapability.h"
#include "MovieSceneSequence.h"
#include "UObject/UnrealType.h"
#include "MovieScene.h"
#if WITH_EDITOR
#include "EdGraphSchema_K2.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDirectorBlueprintCondition)

bool FMovieSceneDirectorBlueprintConditionInvoker::EvaluateDirectorBlueprintCondition(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, const FMovieSceneDirectorBlueprintConditionData& DirectorBlueprintConditionData)
{
	using namespace UE::MovieScene;

	UFunction* ConditionFunc = DirectorBlueprintConditionData.Function.Get();
	if (!ConditionFunc)
	{
		// No condition specified, default to succeeding the condition
		return true;
	}

	// Auto-add the director playback capability, which is just really a cache for director instances after
	// they've been created by the sequences in the hierarchy.
	FSequenceDirectorPlaybackCapability* DirectorCapability = SharedPlaybackState->FindCapability<FSequenceDirectorPlaybackCapability>();
	if (!DirectorCapability)
	{
		TSharedRef<FSharedPlaybackState> MutableState = ConstCastSharedRef<FSharedPlaybackState>(SharedPlaybackState);
		DirectorCapability = &MutableState->AddCapability<FSequenceDirectorPlaybackCapability>();
	}

	UObject* DirectorInstance = DirectorCapability->GetOrCreateDirectorInstance(SharedPlaybackState, SequenceID);
	if (!DirectorInstance)
	{
#if !NO_LOGGING
		UE_LOGF(LogMovieScene, Warning,
			"%ls: Failed to evaluate director blueprint condition '%ls' because no director instance was available.",
			*SharedPlaybackState->GetRootSequence()->GetName(), *ConditionFunc->GetName());
#endif
		// Fallback to default behavior.
		return true;
	}

	UE_LOGF(LogMovieScene, VeryVerbose,
		"%ls: Evaluating director blueprint condition '%ls' with function '%ls'.",
		*SharedPlaybackState->GetRootSequence()->GetName(), *LexToString(BindingGuid), *ConditionFunc->GetName());

	TArray<TObjectPtr<UObject>> BoundObjects;
	Algo::Transform(SharedPlaybackState->FindBoundObjects(BindingGuid, SequenceID), BoundObjects, [](const TWeakObjectPtr<> BoundObject) { return BoundObject.Get(); });

	FMovieSceneConditionContext ConditionContext{ SharedPlaybackState->GetPlaybackContext(), FMovieSceneBindingProxy(BindingGuid, SharedPlaybackState->GetSequence(SequenceID)), BoundObjects };

	return InvokeDirectorBlueprintCondition(DirectorInstance, DirectorBlueprintConditionData, ConditionContext);
}

bool FMovieSceneDirectorBlueprintConditionInvoker::InvokeDirectorBlueprintCondition(UObject* DirectorInstance, const FMovieSceneDirectorBlueprintConditionData& DirectorBlueprintConditionData, const FMovieSceneConditionContext& ConditionContext)
{
	bool Result = false;

	// Do some basic checks.
	UFunction* ConditionFunc = DirectorBlueprintConditionData.Function.Get();
	if (!ensure(ConditionFunc))
	{
		return Result;
	}

#if WITH_EDITOR
	// Not sure why this isn't being checked further down, but check manually here
	if (!ConditionFunc->HasMetaData(FBlueprintMetadata::MD_CallInEditor) || ConditionFunc->GetMetaData(FBlueprintMetadata::MD_CallInEditor) != TEXT("true"))
	{
		if (ConditionContext.WorldContext)
		{
			const UWorld* World = ConditionContext.WorldContext->GetWorld();
			if (World && World->IsEditorWorld() && !World->IsPlayInEditor())
			{
				return Result;
			}
		}
	}
#endif

	// Parse all function parameters.
	uint8* Parameters = (uint8*)FMemory_Alloca(ConditionFunc->ParmsSize + ConditionFunc->MinAlignment);
	Parameters = Align(Parameters, ConditionFunc->MinAlignment);

	// Initialize parameters.
	FMemory::Memzero(Parameters, ConditionFunc->ParmsSize);

	FBoolProperty* ReturnProp = nullptr;
	for (TFieldIterator<FProperty> It(ConditionFunc); It; ++It)
	{
		FProperty* LocalProp = *It;
		checkSlow(LocalProp);
		if (!LocalProp->HasAnyPropertyFlags(CPF_ZeroConstructor) && LocalProp->HasAllPropertyFlags(CPF_Parm))
		{
			LocalProp->InitializeValue_InContainer(Parameters);
		}

		if (LocalProp->HasAnyPropertyFlags(CPF_ReturnParm))
		{
			ensureMsgf(ReturnProp == nullptr,
				TEXT("Found more than one return parameter in blueprint condition resolver function!"));
			ReturnProp = CastFieldChecked<FBoolProperty>(LocalProp);
		}
	}

	// Set the condition context parameter struct if we need to pass it to the function.
	if (FProperty* ConditionContextProp = DirectorBlueprintConditionData.ConditionContextProperty.Get())
	{
		ConditionContextProp->SetValue_InContainer(Parameters, &ConditionContext);
	}

#if WITH_EDITOR
	// In the editor we need to be more forgiving, because we might have temporarily invalid states, such as
	// when undo-ing operations.
	if (ReturnProp != nullptr)
#else
	if (ensureMsgf(ReturnProp != nullptr,
		TEXT("The director blueprint condition evaluation function has no return value of type bool")))
#endif
	{
		// Invoke the function.
		DirectorInstance->ProcessEvent(ConditionFunc, Parameters);

		// Grab the result value.
		ReturnProp->GetValue_InContainer(Parameters, static_cast<void*>(&Result));
	}

	// Destroy parameters.
	for (TFieldIterator<FProperty> It(ConditionFunc); It; ++It)
	{
		FProperty* LocalProp = *It;
		checkSlow(LocalProp);

		if (LocalProp->HasAllPropertyFlags(CPF_Parm))
		{
			It->DestroyValue_InContainer(Parameters);
		}
	}

	return Result;
}

bool UMovieSceneDirectorBlueprintCondition::EvaluateConditionInternal(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	return FMovieSceneDirectorBlueprintConditionInvoker::EvaluateDirectorBlueprintCondition(BindingGuid, SequenceID, SharedPlaybackState, DirectorBlueprintConditionData);
}
