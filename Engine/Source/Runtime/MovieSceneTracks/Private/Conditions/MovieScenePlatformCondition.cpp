// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/MovieScenePlatformCondition.h"
#include "CoreMinimal.h"
#include "Engine/World.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "Kismet/GameplayStatics.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePlatformCondition)

bool UMovieScenePlatformCondition::EvaluateConditionInternal(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	FString PlatformName = UGameplayStatics::GetPlatformName();
	return ValidPlatforms.Contains(*PlatformName);
}
