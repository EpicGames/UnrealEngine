// Copyright Epic Games, Inc. All Rights Reserved.

#include "Conditions/MovieSceneScalabilityCondition.h"
#include "CoreMinimal.h"
#include "Scalability.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneScalabilityCondition)

bool UMovieSceneScalabilityCondition::EvaluateConditionInternal(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const
{
	using namespace Scalability;
	const FQualityLevels& QualityLevels = GetQualityLevels();
	
	int32 CurrentLevel = 0;

	switch (Group)
	{
		 case EMovieSceneScalabilityConditionGroup::ViewDistance:
			CurrentLevel = QualityLevels.ViewDistanceQuality;
			break;
		 case EMovieSceneScalabilityConditionGroup::AntiAliasing:
			CurrentLevel = QualityLevels.AntiAliasingQuality;
			break;
		 case EMovieSceneScalabilityConditionGroup::Shadow:
			CurrentLevel = QualityLevels.ShadowQuality;
			break;
		 case EMovieSceneScalabilityConditionGroup::GlobalIllumination:
			CurrentLevel = QualityLevels.GlobalIlluminationQuality;
			break;
		 case EMovieSceneScalabilityConditionGroup::Reflection:
			CurrentLevel = QualityLevels.ReflectionQuality;
			break;
		 case EMovieSceneScalabilityConditionGroup::PostProcess:
			CurrentLevel = QualityLevels.PostProcessQuality;
			break;
		 case EMovieSceneScalabilityConditionGroup::Texture:
			CurrentLevel = QualityLevels.TextureQuality;
			break;
		 case EMovieSceneScalabilityConditionGroup::Effects:
			CurrentLevel = QualityLevels.EffectsQuality;
			break;
		 case EMovieSceneScalabilityConditionGroup::Foliage:
			CurrentLevel = QualityLevels.FoliageQuality;
			break;
		 case EMovieSceneScalabilityConditionGroup::Shading:
			CurrentLevel = QualityLevels.ShadingQuality;
			break;
		 case EMovieSceneScalabilityConditionGroup::Landscape:
			CurrentLevel = QualityLevels.LandscapeQuality;
			break;
	}

	int32 CompareValue = (int32)Level;

	switch (Operator)
	{
		case EMovieSceneScalabilityConditionOperator::LessThan:
			return CurrentLevel < CompareValue;
		case EMovieSceneScalabilityConditionOperator::LessThanOrEqualTo:
			return CurrentLevel <= CompareValue;
		case EMovieSceneScalabilityConditionOperator::EqualTo:
			return CurrentLevel == CompareValue;
		case EMovieSceneScalabilityConditionOperator::GreaterThanOrEqualTo:
			return CurrentLevel >= CompareValue;
		case EMovieSceneScalabilityConditionOperator::GreaterThan:
			return CurrentLevel > CompareValue;
	}

	return false;
}
