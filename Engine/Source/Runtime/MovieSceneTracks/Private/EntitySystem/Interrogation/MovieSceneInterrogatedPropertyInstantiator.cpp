// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePropertyRegistry.h"
#include "EntitySystem/MovieSceneInitialValueSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#include "Algo/IndexOf.h"
#include "Algo/Find.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneInterrogatedPropertyInstantiator)


UMovieSceneInterrogatedPropertyInstantiatorSystem::UMovieSceneInterrogatedPropertyInstantiatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	BuiltInComponents = FBuiltInComponentTypes::Get();

	RecomposerImpl.OnGetPropertyInfo = FOnGetPropertyRecomposerPropertyInfo::CreateUObject(
				this, &UMovieSceneInterrogatedPropertyInstantiatorSystem::FindPropertyFromSource);

	RelevantComponent = BuiltInComponents->Interrogation.InputKey;
	SystemCategories |= FSystemInterrogator::GetInterrogationCategory();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentProducer(GetClass(), BuiltInComponents->BlendChannelInput);
		DefineComponentProducer(GetClass(), BuiltInComponents->SymbolicTags.CreatesEntities);

		DefineImplicitPrerequisite(GetClass(), UMovieSceneInitialValueSystem::StaticClass());
	}

	CleanFastPathMask.Set(BuiltInComponents->Interrogation.OutputKey);
}

bool UMovieSceneInterrogatedPropertyInstantiatorSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	return InLinker->EntityManager.Contains(FEntityComponentFilter().All({ BuiltInComponents->PropertyBinding, BuiltInComponents->Interrogation.InputKey }));
}

UE::MovieScene::FPropertyRecomposerPropertyInfo UMovieSceneInterrogatedPropertyInstantiatorSystem::FindPropertyFromSource(FMovieSceneEntityID EntityID, UObject* Object) const
{
	using namespace UE::MovieScene;

	if (const FPropertyInfo* Property = PropertyTracker.FindOutput(EntityID))
	{
		return FPropertyRecomposerPropertyInfo { Property->BlendChannel, Property->Blender.Get(), Property->PropertyEntityID };
	}
	return FPropertyRecomposerPropertyInfo::Invalid();
}

UMovieSceneInterrogatedPropertyInstantiatorSystem::FValueRecompositionResult UMovieSceneInterrogatedPropertyInstantiatorSystem::RecomposeBlendChannel(const UE::MovieScene::FPropertyDefinition& PropertyDefinition, int32 ChannelCompositeIndex, const UE::MovieScene::FDecompositionQuery& InQuery, double InCurrentValue)
{
	using namespace UE::MovieScene;

	FValueRecompositionResult Result(InCurrentValue, InQuery.Entities.Num());

	if (InQuery.Entities.Num() == 0)
	{
		return Result;
	}

	const FPropertyInfo* Property = PropertyTracker.FindOutput(InQuery.Entities[0]);
	if (!Property || Property->BlendChannel == INVALID_BLEND_CHANNEL)
	{
		return Result;
	}

	UMovieSceneBlenderSystem* Blender = Property->Blender.Get();
	if (!Blender)
	{
		return Result;
	}

	FValueDecompositionParams Params;
	Params.Query = InQuery;
	Params.PropertyEntityID = Property->PropertyEntityID;
	Params.DecomposeBlendChannel = Property->BlendChannel;
	Params.PropertyTag = PropertyDefinition.PropertyType;

	TArrayView<const FPropertyCompositeDefinition> Composites = BuiltInComponents->PropertyRegistry.GetComposites(PropertyDefinition);
	check(Composites.IsValidIndex(ChannelCompositeIndex));

	PropertyDefinition.Handler->RecomposeBlendChannel(PropertyDefinition, Composites, ChannelCompositeIndex, Params, Blender, InCurrentValue, Result.Values);

	return Result;
}


bool UMovieSceneInterrogatedPropertyInstantiatorSystem::PropertySupportsFastPath(TArrayView<const FMovieSceneEntityID> Inputs, FPropertyInfo* Output) const
{
	using namespace UE::MovieScene;

	// Properties that are already blended, or are currently animated must use the blend path
	if (Output->BlendChannel != INVALID_BLEND_CHANNEL || Output->PropertyEntityID.IsValid())
	{
		return false;
	}
	else if (Inputs.Num() > 1)
	{
		return false;
	}

	for (FMovieSceneEntityID Input : Inputs)
	{
		FComponentMask Type = Linker->EntityManager.GetEntityType(Input);
		if (Type.Contains(BuiltInComponents->Tags.RelativeBlend) ||
			Type.Contains(BuiltInComponents->Tags.AdditiveBlend) ||
			Type.Contains(BuiltInComponents->Tags.OverrideBlend) ||
			Type.Contains(BuiltInComponents->Tags.AdditiveFromBaseBlend) ||
			Type.Contains(BuiltInComponents->WeightAndEasingResult))
		{
			return false;
		}
	}
	return true;
}

UClass* UMovieSceneInterrogatedPropertyInstantiatorSystem::ResolveBlenderClass(const UE::MovieScene::FPropertyDefinition& PropertyDefinition, TArrayView<const FMovieSceneEntityID> Inputs) const
{
	using namespace UE::MovieScene;

	UClass* BlenderClass = PropertyDefinition.BlenderSystemClass;

	for (FMovieSceneEntityID Input : Inputs)
	{
		TOptionalComponentReader<TSubclassOf<UMovieSceneBlenderSystem>> BlenderTypeComponent = Linker->EntityManager.ReadComponent(Input, BuiltInComponents->BlenderType);
		if (BlenderTypeComponent)
		{
			BlenderClass = BlenderTypeComponent->Get();
			break;
		}
	}
	
	if (!ensureMsgf(BlenderClass, TEXT("No default blender class specified on property, and no custom blender specified on entities. Falling back to double blender.")))
	{
		BlenderClass = UMovieScenePiecewiseDoubleBlenderSystem::StaticClass();
	}
	
	return BlenderClass;
}

void UMovieSceneInterrogatedPropertyInstantiatorSystem::InitializeOutput(UE::MovieScene::FInterrogationKey Key, TArrayView<const FMovieSceneEntityID> Inputs, FPropertyInfo* Output, UE::MovieScene::FEntityOutputAggregate Aggregate)
{
	using namespace UE::MovieScene;

	UpdateOutput(Key, Inputs, Output, Aggregate);
}

void UMovieSceneInterrogatedPropertyInstantiatorSystem::UpdateOutput(UE::MovieScene::FInterrogationKey Key, TArrayView<const FMovieSceneEntityID> Inputs, FPropertyInfo* Output, UE::MovieScene::FEntityOutputAggregate Aggregate)
{
	using namespace UE::MovieScene;

	check(Inputs.Num() > 0);

	auto FindPropertyIndex = [this, Input = Inputs[0]](const FPropertyDefinition& InDefinition)
	{
		return this->Linker->EntityManager.HasComponent(Input, InDefinition.PropertyType);
	};
	TArrayView<const FPropertyDefinition> Properties = BuiltInComponents->PropertyRegistry.GetProperties();

	const FPropertyDefinition* PropertyDefinition = Algo::FindByPredicate(Properties, FindPropertyIndex);
	check(PropertyDefinition);

	if (PropertySupportsFastPath(Inputs, Output))
	{
		Linker->EntityManager.AddComponent(Inputs[0], BuiltInComponents->Interrogation.OutputKey, Key);

		if (PropertyDefinition->MetaDataTypes.Num() > 0)
		{
			FComponentMask NewMask;
			for (FComponentTypeID Component : PropertyDefinition->MetaDataTypes)
			{
				NewMask.Set(Component);
			}
			Linker->EntityManager.AddComponents(Inputs[0], NewMask);
		}
		return;
	}

	TArrayView<const FPropertyCompositeDefinition> Composites = BuiltInComponents->PropertyRegistry.GetComposites(*PropertyDefinition);

	// Find the blender class to use
	UClass* BlenderClass = ResolveBlenderClass(*PropertyDefinition, Inputs);

	UMovieSceneBlenderSystem* ExistingBlender = Output->Blender.Get();
	if (ExistingBlender && BlenderClass != ExistingBlender->GetClass())
	{
		const FMovieSceneBlendChannelID BlendChannel(ExistingBlender->GetBlenderSystemID(), Output->BlendChannel);
		ExistingBlender->ReleaseBlendChannel(BlendChannel);
		Output->BlendChannel = INVALID_BLEND_CHANNEL;
	}

	UMovieSceneBlenderSystem* BlenderSystem = CastChecked<UMovieSceneBlenderSystem>(Linker->LinkSystem(BlenderClass));
	Output->Blender = BlenderSystem;

	const bool bWasAlreadyBlended = Output->BlendChannel != INVALID_BLEND_CHANNEL;
	if (!bWasAlreadyBlended)
	{
		const FMovieSceneBlendChannelID BlendChannel = Output->Blender->AllocateBlendChannel();
		Output->BlendChannel = BlendChannel.ChannelID;
	}

	FComponentMask NewMask;
	FComponentMask OldMask;

	if (PropertyDefinition->MetaDataTypes.Num() > 0)
	{
		for (FComponentTypeID Component : PropertyDefinition->MetaDataTypes)
		{
			NewMask.Set(Component);
		}
	}

	if (!bWasAlreadyBlended)
	{
		NewMask.Set(PropertyDefinition->InitialValueType);

		for (int32 CompositeIndex = 0; CompositeIndex < Composites.Num(); ++CompositeIndex)
		{
			const FComponentTypeID CompositeType = Composites[CompositeIndex].ComponentTypeID;

			const bool bHasChannel = Algo::FindByPredicate(Inputs, [this, CompositeType](FMovieSceneEntityID Input) { return this->Linker->EntityManager.HasComponent(Input, CompositeType); }) != nullptr;
			if (bHasChannel)
			{
				NewMask.Set(CompositeType);
			}
		}
		NewMask.Set(PropertyDefinition->PropertyType);

		const FMovieSceneBlendChannelID BlendChannel(Output->Blender->GetBlenderSystemID(), Output->BlendChannel);

		// Never seen this property before
		FMovieSceneEntityID NewEntityID = FEntityBuilder()
		.Add(BuiltInComponents->Interrogation.OutputKey, Key)
		.Add(BuiltInComponents->BlendChannelOutput, BlendChannel)
		.AddTag(BuiltInComponents->Tags.NeedsLink)
		.AddTag(BlenderSystem->GetBlenderTypeTag())
		.AddMutualComponents()
		.CreateEntity(&Linker->EntityManager, NewMask);

		if (Output->PropertyEntityID)
		{
			// Move any migratable components over from the existing fast-path entity
			Linker->EntityManager.CopyComponents(Output->PropertyEntityID, NewEntityID, Linker->EntityManager.GetComponents()->GetMigrationMask());

			// Add blend inputs on the first contributor, which was using the fast-path
			Linker->EntityManager.AddComponent(Output->PropertyEntityID, BuiltInComponents->BlendChannelInput, BlendChannel);
			Linker->EntityManager.RemoveComponents(Output->PropertyEntityID, CleanFastPathMask);
		}

		Output->PropertyEntityID = NewEntityID;
	}
	else
	{
		FComponentMask NewEntityType = Linker->EntityManager.GetEntityType(Output->PropertyEntityID);

		// Ensure the property has only the exact combination of components that constitute its animation
		for (int32 CompositeIndex = 0; CompositeIndex < Composites.Num(); ++CompositeIndex)
		{
			const FComponentTypeID CompositeType = Composites[CompositeIndex].ComponentTypeID;

			const bool bHasChannel = Algo::FindByPredicate(Inputs, [this, CompositeType](FMovieSceneEntityID Input) { return this->Linker->EntityManager.HasComponent(Input, CompositeType); }) != nullptr;
			NewEntityType[CompositeType] = bHasChannel;
		}

		NewEntityType.Set(PropertyDefinition->PropertyType);

		Linker->EntityManager.ChangeEntityType(Output->PropertyEntityID, NewEntityType);
	}

	// Ensure contributors all have the necessary blend inputs and tags
	for (FMovieSceneEntityID Input : Inputs)
	{
		const FMovieSceneBlendChannelID BlendChannel(Output->Blender->GetBlenderSystemID(), Output->BlendChannel);
		Linker->EntityManager.AddComponent(Input, BuiltInComponents->BlendChannelInput, BlendChannel);
		Linker->EntityManager.AddComponent(Input, BlenderSystem->GetBlenderTypeTag());
		Linker->EntityManager.RemoveComponents(Input, CleanFastPathMask);
	}
}

void UMovieSceneInterrogatedPropertyInstantiatorSystem::DestroyOutput(UE::MovieScene::FInterrogationKey Key, FPropertyInfo* Output, UE::MovieScene::FEntityOutputAggregate Aggregate)
{
	if (Output->BlendChannel != INVALID_BLEND_CHANNEL)
	{
		if (UMovieSceneBlenderSystem* Blender = Output->Blender.Get())
		{
			const FMovieSceneBlendChannelID BlendChannel(Blender->GetBlenderSystemID(), Output->BlendChannel);
			Blender->ReleaseBlendChannel(BlendChannel);
		}
		Linker->EntityManager.AddComponents(Output->PropertyEntityID, BuiltInComponents->FinishedMask);
	}
}

void UMovieSceneInterrogatedPropertyInstantiatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	PropertyTracker.Initialize(this);

	using namespace UE::MovieScene;

	{
		TArrayView<const FPropertyDefinition> AllProperties = BuiltInComponents->PropertyRegistry.GetProperties();

		auto LinkCallback =  [this, AllProperties](const FEntityAllocation* Allocation, TRead<FInterrogationKey> InterrogationChannels)
		{
			const int32 PropertyDefinitionIndex = Algo::IndexOfByPredicate(AllProperties, [=](const FPropertyDefinition& InDefinition){ return Allocation->HasComponent(InDefinition.PropertyType); });
			if (PropertyDefinitionIndex != INDEX_NONE)
			{
				this->PropertyTracker.VisitLinkedAllocation(Allocation, InterrogationChannels);
			}
		};
		auto UnlinkCallback = [this, AllProperties](const FEntityAllocation* Allocation)
		{
			const int32 PropertyDefinitionIndex = Algo::IndexOfByPredicate(AllProperties, [=](const FPropertyDefinition& InDefinition){ return Allocation->HasComponent(InDefinition.PropertyType); });
			if (PropertyDefinitionIndex != INDEX_NONE)
			{
				this->PropertyTracker.VisitUnlinkedAllocation(Allocation);
			}
		};

		// Visit newly or re-linked entities
		FEntityTaskBuilder()
		.Read(BuiltInComponents->Interrogation.InputKey)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.Iterate_PerAllocation(&Linker->EntityManager, LinkCallback);

		FEntityTaskBuilder()
		.FilterAll({ BuiltInComponents->Interrogation.InputKey, BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerAllocation(&Linker->EntityManager, UnlinkCallback);
	}

	PropertyTracker.ProcessInvalidatedOutputs(Linker, *this);
}

