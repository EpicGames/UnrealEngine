// Copyright Epic Games, Inc. All Rights Reserved.

#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInitialValueCache.h"

#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneSequence.h"
#include "IMovieScenePlayer.h"

#include "Tracks/MovieScene3DTransformTrack.h"

#include "GameFramework/Actor.h"
#include "Tracks/IMovieSceneTransformOrigin.h"
#include "UObject/Package.h"


namespace UE
{
namespace MovieScene
{



struct FImportedInterrogationEntityKey
{
	FInterrogationKey InterrogationKey;
	FMovieSceneEvaluationFieldEntityKey Entity;

	friend bool operator==(FImportedInterrogationEntityKey A, FImportedInterrogationEntityKey B)
	{
		return A.InterrogationKey == B.InterrogationKey && A.Entity == B.Entity;
	}
	friend bool operator!=(FImportedInterrogationEntityKey A, FImportedInterrogationEntityKey B)
	{
		return !(A == B);
	}
	friend uint32 GetTypeHash(const FImportedInterrogationEntityKey& In)
	{
		return HashCombine(GetTypeHash(In.InterrogationKey), GetTypeHash(In.Entity));
	}
};

struct FSystemInterrogatorEntityTracker
{
	/** Ledger for all imported and manufactured entities */
	TMap<FImportedInterrogationEntityKey, FMovieSceneEntityID> ImportedEntities;

	void TrackEntity(const FInterrogationKey& InterrogationKey, const FMovieSceneEvaluationFieldEntityKey& EntityKey, FMovieSceneEntityID EntityID)
	{
		ImportedEntities.Add(FImportedInterrogationEntityKey{ InterrogationKey, EntityKey }, EntityID);
	}

	FMovieSceneEntityID FindTrackedEntity(const FInterrogationKey& InterrogationKey, const FMovieSceneEvaluationFieldEntityKey& EntityKey)
	{
		return ImportedEntities.FindRef(FImportedInterrogationEntityKey{ InterrogationKey, EntityKey });
	}

	void Reset()
	{
		ImportedEntities.Reset();
	}
};


FInterrogationChannels::FInterrogationChannels()
{
	// Always add a bit for the default channel
	ActiveChannelBits.Add(false);
}

FInterrogationChannels::~FInterrogationChannels()
{}

void FInterrogationChannels::Reset()
{
	Interrogations.Empty();
	ActiveChannelBits.Empty();

	// Always add a bit for the default channel
	ActiveChannelBits.Add(false);

	ObjectToChannel.Empty();

	SparseChannelInfo.Empty();
}

FInterrogationChannel FInterrogationChannels::AllocateChannel(FInterrogationChannel ParentChannel, const FMovieScenePropertyBinding& PropertyBinding)
{
	if (!ensureMsgf(ActiveChannelBits.Num() < MAX_int32, TEXT("Reached the maximum available number of interrogation channels")))
	{
		return FInterrogationChannel::Invalid();
	}

	FInterrogationChannel Channel = FInterrogationChannel::FromIndex(ActiveChannelBits.Num());
	ActiveChannelBits.Add(false);

	if (ParentChannel || !PropertyBinding.PropertyPath.IsNone())
	{
		FInterrogationChannelInfo& ChannelInfo = SparseChannelInfo.Get(Channel);
		ChannelInfo.ParentChannel   = ParentChannel;
		ChannelInfo.PropertyBinding = PropertyBinding;
	}
	return Channel;
}

FInterrogationChannel FInterrogationChannels::AllocateChannel(UObject* Object, const FMovieScenePropertyBinding& PropertyBinding)
{
	return AllocateChannel(Object, FInterrogationChannel::Invalid(), PropertyBinding);
}

FInterrogationChannel FInterrogationChannels::AllocateChannel(UObject* Object, FInterrogationChannel ParentChannel, const FMovieScenePropertyBinding& PropertyBinding)
{
	FInterrogationChannel NewChannel = AllocateChannel(ParentChannel, PropertyBinding);
	if (NewChannel)
	{
		ObjectToChannel.Add(Object, NewChannel);
		SparseChannelInfo.Get(NewChannel).WeakObject = Object;
	}
	return NewChannel;
}

FInterrogationChannel FInterrogationChannels::FindChannel(UObject* Object)
{
	return ObjectToChannel.FindRef(Object);
}

void FInterrogationChannels::ActivateChannel(FInterrogationChannel InChannel)
{
	ActiveChannelBits[InChannel.AsIndex()] = true;
}

void FInterrogationChannels::DeactivateChannel(FInterrogationChannel InChannel)
{
	ActiveChannelBits[InChannel.AsIndex()] = false;
}

int32 FInterrogationChannels::AddInterrogation(const FInterrogationParams& Params)
{
	if (!ensureMsgf(Interrogations.Num() != MAX_int32, TEXT("Reached the maximum available number of interrogation channels")))
	{
		return INDEX_NONE;
	}

	const int32 InterrogationIndex = Interrogations.Num();
	Interrogations.Add(Params);

	return InterrogationIndex;
}

void FInterrogationChannels::QueryWorldSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, USceneComponent* SceneComponent, TArray<FTransform>& OutTransforms) const
{
	FInterrogationChannel Channel = ObjectToChannel.FindRef(SceneComponent);
	if (Channel)
	{
		QueryWorldSpaceTransforms(Linker, Channel, OutTransforms);
	}
}

void FInterrogationChannels::QueryLocalSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, USceneComponent* SceneComponent, TArray<FIntermediate3DTransform>& OutTransforms) const
{
	FInterrogationChannel Channel = ObjectToChannel.FindRef(SceneComponent);
	if (Channel)
	{
		QueryLocalSpaceTransforms(Linker, Channel, OutTransforms);
	}
}

void FInterrogationChannels::QueryWorldSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, FInterrogationChannel Channel, TArray<FTransform>& OutTransforms) const
{
	TBitArray<> ChannelMask(false, Channel.AsIndex()+1);
	ChannelMask[Channel.AsIndex()] = true;
	QueryWorldSpaceTransforms(Linker, ChannelMask, [&OutTransforms](FInterrogationChannel)-> TArray<FTransform>& { return OutTransforms; });
}

void FInterrogationChannels::QueryLocalSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, FInterrogationChannel Channel, TArray<FIntermediate3DTransform>& OutTransforms) const
{
	TBitArray<> ChannelMask(false, Channel.AsIndex()+1);
	ChannelMask[Channel.AsIndex()] = true;
	QueryLocalSpaceTransforms(Linker, ChannelMask, [&OutTransforms](FInterrogationChannel) -> TArray<FIntermediate3DTransform>& { return OutTransforms; });
}

void FInterrogationChannels::QueryWorldSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, TSparseArray<TArray<FTransform>>& OutTransformsByChannel) const
{
	TBitArray<> AllChannels(true, GetNumChannels());

	OutTransformsByChannel.Reserve(GetNumChannels());
	for (int32 Index = 0; Index < GetNumChannels(); ++Index)
	{
		if (!OutTransformsByChannel.IsValidIndex(Index))
		{
			OutTransformsByChannel.Insert(Index, TArray<FTransform>());
		}
		OutTransformsByChannel[Index].Reset(Interrogations.Num());
	}

	QueryWorldSpaceTransforms(Linker, AllChannels, [&OutTransformsByChannel](FInterrogationChannel Channel) -> TArray<FTransform>& { return OutTransformsByChannel[Channel.AsIndex()]; });
}

void FInterrogationChannels::QueryLocalSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, TSparseArray<TArray<FIntermediate3DTransform>>& OutTransformsByChannel) const
{
	TBitArray<> AllChannels(true, GetNumChannels());

	OutTransformsByChannel.Reserve(GetNumChannels());
	for (int32 Index = 0; Index < GetNumChannels(); ++Index)
	{
		if (!OutTransformsByChannel.IsValidIndex(Index))
		{
			OutTransformsByChannel.Insert(Index, TArray<FIntermediate3DTransform>());
		}
		OutTransformsByChannel[Index].Reset(Interrogations.Num());
	}

	QueryLocalSpaceTransforms(Linker, AllChannels, [&OutTransformsByChannel](FInterrogationChannel Channel) -> TArray<FIntermediate3DTransform>& { return OutTransformsByChannel[Channel.AsIndex()]; });
}

void FInterrogationChannels::QueryWorldSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, const TBitArray<>& ChannelsToQuery, TSparseArray<TArray<FTransform>>& OutTransformsByChannel) const
{
	for (TConstSetBitIterator<> ChannelBit(ChannelsToQuery); ChannelBit; ++ChannelBit)
	{
		if (!OutTransformsByChannel.IsValidIndex(ChannelBit.GetIndex()))
		{
			OutTransformsByChannel.Insert(ChannelBit.GetIndex(), TArray<FTransform>());
		}
		OutTransformsByChannel[ChannelBit.GetIndex()].Reset(Interrogations.Num());
	}

	QueryWorldSpaceTransforms(Linker, ChannelsToQuery, [&OutTransformsByChannel](FInterrogationChannel Channel) -> TArray<FTransform>& { return OutTransformsByChannel[Channel.AsIndex()]; });
}

void FInterrogationChannels::QueryLocalSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, const TBitArray<>& ChannelsToQuery, TSparseArray<TArray<FIntermediate3DTransform>>& OutTransformsByChannel) const
{
	for (TConstSetBitIterator<> ChannelBit(ChannelsToQuery); ChannelBit; ++ChannelBit)
	{
		if (!OutTransformsByChannel.IsValidIndex(ChannelBit.GetIndex()))
		{
			OutTransformsByChannel.Insert(ChannelBit.GetIndex(), TArray<FIntermediate3DTransform>());
		}
		OutTransformsByChannel[ChannelBit.GetIndex()].Reset(Interrogations.Num());
	}

	QueryLocalSpaceTransforms(Linker, ChannelsToQuery, [&OutTransformsByChannel](FInterrogationChannel Channel) -> TArray<FIntermediate3DTransform>& { return OutTransformsByChannel[Channel.AsIndex()]; });
}

template<typename GetOutputForChannelType>
void FInterrogationChannels::QueryWorldSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, const TBitArray<>& ChannelsToQuery, GetOutputForChannelType&& OnGetOutputForChannel) const
{
	const int32 NumTransforms = Interrogations.Num();
	const int32 NumChannels   = GetNumChannels();

	TSparseArray<TArray<FTransform>> VariableTransformsByChannel;
	TSparseArray<FTransform> BaseValues;

	TBitArray<> PredicateBits(false, NumChannels);

	// Array of channel depths where 0 means no children
	TArray<int32> NumRecursiveChildren;
	NumRecursiveChildren.SetNum(NumChannels);

	// Populate the map with initial values
	for (TConstSetBitIterator<> ChannelBit(ChannelsToQuery); ChannelBit; ++ChannelBit)
	{
		int32 NumChildren = 0;

		FInterrogationChannel Channel = FInterrogationChannel::FromIndex(ChannelBit.GetIndex());
		if ((bool)ActiveChannelBits[Channel.AsIndex()] == false && SparseChannelInfo.FindObject(Channel) == nullptr)
		{
			continue;
		}

		OnGetOutputForChannel(Channel).SetNum(NumTransforms);

		// Populate the arrays with current values for all objects in the hierarchy
		while (Channel.IsValid())
		{
			const int32 ThisChannelIndex = Channel.AsIndex();
			PredicateBits[ThisChannelIndex] = true;

			NumRecursiveChildren[ThisChannelIndex] = FMath::Max(NumRecursiveChildren[ThisChannelIndex], NumChildren);

			// Update the component's current value
			FTransform BaseValue = FTransform::Identity;

			const FInterrogationChannelInfo* ChannelInfo = SparseChannelInfo.Find(Channel);
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(ChannelInfo ? ChannelInfo->WeakObject.Get() : nullptr))
			{
				BaseValue = SceneComponent->GetRelativeTransform();
			}

			// Keep track of base values
			if (!BaseValues.IsValidIndex(ThisChannelIndex))
			{
				BaseValues.Insert(ThisChannelIndex, BaseValue);
			}

			// If this channel has variable data, we allocate space for it now
			if (ActiveChannelBits[ThisChannelIndex] == true && !VariableTransformsByChannel.IsValidIndex(ThisChannelIndex))
			{
				VariableTransformsByChannel.Insert(ThisChannelIndex, TArray<FTransform>());
				VariableTransformsByChannel[ThisChannelIndex].SetNum(Interrogations.Num());
			}

			Channel = ChannelInfo ? ChannelInfo->ParentChannel : FInterrogationChannel::Invalid();
			++NumChildren;
		}
	}


	FBuiltInComponentTypes*          Components       = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	// Gather fully animated transforms - this code is simple
	{
		auto PopulateComplete = [&VariableTransformsByChannel, PredicateBits](FInterrogationKey InterrogationKey,
			double LocationX, double LocationY, double LocationZ,
			double RotationX, double RotationY, double RotationZ,
			double ScaleX, double ScaleY, double ScaleZ)
		{
			const int32 ChannelIndex = InterrogationKey.Channel.AsIndex();

			if (PredicateBits[ChannelIndex] == true)
			{
				VariableTransformsByChannel[ChannelIndex][InterrogationKey.InterrogationIndex] = FTransform(
					FRotator(RotationY, RotationZ, RotationX),
					FVector(LocationX, LocationY, LocationZ),
					FVector(ScaleX, ScaleY, ScaleZ)
				);
			}
		};

		FEntityTaskBuilder()
		.Read(Components->Interrogation.OutputKey)
		.ReadAllOf(
			Components->DoubleResult[0], Components->DoubleResult[1], Components->DoubleResult[2],
			Components->DoubleResult[3], Components->DoubleResult[4], Components->DoubleResult[5],
			Components->DoubleResult[6], Components->DoubleResult[7], Components->DoubleResult[8])
		.FilterAll({ TracksComponents->ComponentTransform.PropertyTag })
		.Iterate_PerEntity(&Linker->EntityManager, PopulateComplete);
	}

	// Gather partially animated transforms - this code is more complex
	{
		FComponentMask CompleteMask;
		for (int32 Index = 0; Index < 9; ++Index)
		{
			CompleteMask.Set(Components->DoubleResult[Index]);
		}

		FEntityComponentFilter Filter;
		Filter.Any(CompleteMask);
		Filter.Deny(CompleteMask);

		if (Linker->EntityManager.Contains(Filter))
		{
			auto PopulatePartial = [&VariableTransformsByChannel, &PredicateBits, &BaseValues, Components](const FEntityAllocation* Allocation, const FInterrogationKey* Keys)
			{
				const int32 Num = Allocation->Num();

				TOptionalComponentReader<double> DoubleComponents[9];
				for (int32 Index = 0; Index < 9; ++Index)
				{
					DoubleComponents[Index] = Allocation->TryReadComponents(Components->DoubleResult[Index]);
				}

				for (int32 ComponentIndex = 0; ComponentIndex < Num; ++ComponentIndex)
				{
					FInterrogationKey InterrogationKey = Keys[ComponentIndex];
					const int32 ChannelIndex = InterrogationKey.Channel.AsIndex();
					if (PredicateBits[ChannelIndex] == true)
					{
						FTransform Transform = BaseValues[ChannelIndex];

						FVector   Location = Transform.GetTranslation();
						FRotator  Rotation = Transform.GetRotation().Rotator();
						FVector   Scale    = Transform.GetScale3D();

						if (DoubleComponents[0]) { Location.X     = DoubleComponents[0][ComponentIndex]; }
						if (DoubleComponents[1]) { Location.Y     = DoubleComponents[1][ComponentIndex]; }
						if (DoubleComponents[2]) { Location.Z     = DoubleComponents[2][ComponentIndex]; }
						if (DoubleComponents[3]) { Rotation.Roll  = DoubleComponents[3][ComponentIndex]; }
						if (DoubleComponents[4]) { Rotation.Pitch = DoubleComponents[4][ComponentIndex]; }
						if (DoubleComponents[5]) { Rotation.Yaw   = DoubleComponents[5][ComponentIndex]; }
						if (DoubleComponents[6]) { Scale.X        = DoubleComponents[6][ComponentIndex]; }
						if (DoubleComponents[7]) { Scale.Y        = DoubleComponents[7][ComponentIndex]; }
						if (DoubleComponents[8]) { Scale.Z        = DoubleComponents[8][ComponentIndex]; }

						VariableTransformsByChannel[ChannelIndex][InterrogationKey.InterrogationIndex] = FTransform(Rotation, Location, Scale);
					}
				}
			};

			FEntityTaskBuilder()
			.Read(Components->Interrogation.OutputKey)
			.FilterAny(CompleteMask)
			.FilterOut(CompleteMask)
			.FilterAll({ TracksComponents->ComponentTransform.PropertyTag })
			.Iterate_PerAllocation(&Linker->EntityManager, PopulatePartial);
		}
	}


	// Final accumulation
	{
		// Build and populate an array of depths so we can sort parents first
		TArray<FInterrogationChannel> ChannelsByDepth;
		for (TConstSetBitIterator<> ChannelBit(PredicateBits); ChannelBit; ++ChannelBit)
		{
			ChannelsByDepth.Add(FInterrogationChannel::FromIndex(ChannelBit.GetIndex()));
		}
		Algo::Sort(ChannelsByDepth, [&NumRecursiveChildren](FInterrogationChannel A, FInterrogationChannel B){ return NumRecursiveChildren[A.AsIndex()] > NumRecursiveChildren[B.AsIndex()]; });

		for (FInterrogationChannel Channel : ChannelsByDepth)
		{
			TArray<FTransform>& ChannelOutput = OnGetOutputForChannel(Channel);

			const int32 ChannelIndex = Channel.AsIndex();


			FInterrogationChannel Parent = SparseChannelInfo.FindParent(Channel);
			FTransform OffsetFromAnimatedParent = FTransform::Identity;

			bool bProcessedTransforms = false;

			// Find a parent to accumulate from
			while (Parent)
			{
				// Does this parent have a variable transform over time?
				if (VariableTransformsByChannel.IsValidIndex(Parent.AsIndex()))
				{
					TArray<FTransform>& ParentTransforms = VariableTransformsByChannel[Parent.AsIndex()];

					// Does this child have a variable transform over time?
					if (VariableTransformsByChannel.IsValidIndex(ChannelIndex))
					{
						TArray<FTransform>& ChildTransforms = VariableTransformsByChannel[ChannelIndex];
						for (int32 Index = 0; Index < Interrogations.Num(); ++Index)
						{
							ChildTransforms[Index] = ChildTransforms[Index] * OffsetFromAnimatedParent * ParentTransforms[Index];
						}
						ChannelOutput = ChildTransforms;
					}
					else
					{
						for (int32 Index = 0; Index < Interrogations.Num(); ++Index)
						{
							ChannelOutput[Index] = BaseValues[ChannelIndex] * OffsetFromAnimatedParent * ParentTransforms[Index];
						}
					}

					bProcessedTransforms = true;
					break;
				}

				OffsetFromAnimatedParent = BaseValues[Parent.AsIndex()] * OffsetFromAnimatedParent;
				Parent = SparseChannelInfo.FindParent(Parent);
			}

			if (!bProcessedTransforms)
			{
				if (VariableTransformsByChannel.IsValidIndex(ChannelIndex))
				{
					TArray<FTransform>& ChildTransforms = VariableTransformsByChannel[ChannelIndex];
					for (int32 Index = 0; Index < Interrogations.Num(); ++Index)
					{
						ChannelOutput[Index] = ChildTransforms[Index] * OffsetFromAnimatedParent;
					}
				}
				// This channel does not have a variable output -> it's constant the whole way
				else for (FTransform& Transform : ChannelOutput)
				{
					Transform = OffsetFromAnimatedParent;
				}
			}
		}
	}
}

template<typename GetOutputForChannelType>
void FInterrogationChannels::QueryLocalSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, const TBitArray<>& ChannelsToQuery, GetOutputForChannelType&& OnGetOutputForChannel) const
{
	const int32 NumTransforms = Interrogations.Num();

	FBuiltInComponentTypes*          Components       = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	TSparseArray<FIntermediate3DTransform> BaseValues;
	TBitArray<> PredicateBits(false, GetNumChannels());

	// Populate the output structure with initial values
	for (TConstSetBitIterator<> ChannelBit(ChannelsToQuery); ChannelBit; ++ChannelBit)
	{
		FInterrogationChannel Channel = FInterrogationChannel::FromIndex(ChannelBit.GetIndex());
		if ((bool)ActiveChannelBits[Channel.AsIndex()] == false && SparseChannelInfo.FindObject(Channel) == nullptr)
		{
			continue;
		}

		FIntermediate3DTransform BaseValue;

		if (USceneComponent* SceneComponent = Cast<USceneComponent>(SparseChannelInfo.FindObject(Channel)))
		{
			BaseValue = FIntermediate3DTransform(SceneComponent->GetRelativeLocation(), SceneComponent->GetRelativeRotation(), SceneComponent->GetRelativeScale3D());
		}

		if (!BaseValues.IsValidIndex(ChannelBit.GetIndex()))
		{
			BaseValues.Insert(ChannelBit.GetIndex(), BaseValue);
		}

		if (ActiveChannelBits[ChannelBit.GetIndex()] == true)
		{
			OnGetOutputForChannel(Channel).SetNum(NumTransforms);
			PredicateBits[ChannelBit.GetIndex()] = true;
		}
		else
		{
			// If the channel doesn't have any variable data, just copy the base value over
			TArray<FIntermediate3DTransform>& Array = OnGetOutputForChannel(Channel);
			Array.SetNum(NumTransforms);
			for (FIntermediate3DTransform& Transform : Array)
			{
				Transform = BaseValue;
			}
		}
	}

	// Gather fully animated transforms - this code is simple
	{
		auto PopulateComplete = [&OnGetOutputForChannel, &PredicateBits](FInterrogationKey InterrogationKey,
			double LocationX, double LocationY, double LocationZ,
			double RotationX, double RotationY, double RotationZ,
			double ScaleX, double ScaleY, double ScaleZ)
		{
			if (PredicateBits[InterrogationKey.Channel.AsIndex()] == true)
			{
				OnGetOutputForChannel(InterrogationKey.Channel)[InterrogationKey.InterrogationIndex] = FIntermediate3DTransform(
					LocationX, LocationY, LocationZ,
					RotationY, RotationZ, RotationX,
					ScaleX, ScaleY, ScaleZ
				);
			}
		};

		FEntityTaskBuilder()
		.Read(Components->Interrogation.OutputKey)
		.ReadAllOf(
			Components->DoubleResult[0], Components->DoubleResult[1], Components->DoubleResult[2],
			Components->DoubleResult[3], Components->DoubleResult[4], Components->DoubleResult[5],
			Components->DoubleResult[6], Components->DoubleResult[7], Components->DoubleResult[8])
		.FilterAll({ TracksComponents->ComponentTransform.PropertyTag })
		.Iterate_PerEntity(&Linker->EntityManager, PopulateComplete);
	}

	// Gather partially animated transforms - this code is more complex
	{
		FComponentMask CompleteMask;
		for (int32 Index = 0; Index < 9; ++Index)
		{
			CompleteMask.Set(Components->DoubleResult[Index]);
		}

		FEntityComponentFilter Filter;
		Filter.Any(CompleteMask);
		Filter.Deny(CompleteMask);

		if (Linker->EntityManager.Contains(Filter))
		{
			auto PopulatePartial = [&OnGetOutputForChannel, &PredicateBits, &BaseValues, Components](const FEntityAllocation* Allocation, const FInterrogationKey* Keys)
			{
				const int32 Num = Allocation->Num();

				TOptionalComponentReader<double> DoubleComponents[9];
				for (int32 Index = 0; Index < 9; ++Index)
				{
					DoubleComponents[Index] = Allocation->TryReadComponents(Components->DoubleResult[Index]);
				}

				for (int32 ComponentIndex = 0; ComponentIndex < Num; ++ComponentIndex)
				{
					FInterrogationKey InterrogationKey = Keys[ComponentIndex];
					if (PredicateBits[InterrogationKey.Channel.AsIndex()] == true)
					{
						FIntermediate3DTransform Transform = BaseValues[ComponentIndex];

						if (DoubleComponents[0]) { Transform.T_X = DoubleComponents[0][ComponentIndex]; }
						if (DoubleComponents[1]) { Transform.T_Y = DoubleComponents[1][ComponentIndex]; }
						if (DoubleComponents[2]) { Transform.T_Z = DoubleComponents[2][ComponentIndex]; }
						if (DoubleComponents[3]) { Transform.R_X = DoubleComponents[3][ComponentIndex]; }
						if (DoubleComponents[4]) { Transform.R_Y = DoubleComponents[4][ComponentIndex]; }
						if (DoubleComponents[5]) { Transform.R_Z = DoubleComponents[5][ComponentIndex]; }
						if (DoubleComponents[6]) { Transform.S_X = DoubleComponents[6][ComponentIndex]; }
						if (DoubleComponents[7]) { Transform.S_Y = DoubleComponents[7][ComponentIndex]; }
						if (DoubleComponents[8]) { Transform.S_Z = DoubleComponents[8][ComponentIndex]; }

						OnGetOutputForChannel(InterrogationKey.Channel)[InterrogationKey.InterrogationIndex] = Transform;
					}
				}
			};

			FEntityTaskBuilder()
			.Read(Components->Interrogation.OutputKey)
			.FilterAny(CompleteMask)
			.FilterOut(CompleteMask)
			.FilterAll({ TracksComponents->ComponentTransform.PropertyTag })
			.Iterate_PerAllocation(&Linker->EntityManager, PopulatePartial);
		}
	}
}

void FInterrogationChannels::QueryTransformOrigins(UMovieSceneEntitySystemLinker* Linker, const FMovieSceneSequenceHierarchy* Hierarchy, TArray<FTransform>& OutTransformOrigins, TArray<FMovieSceneSequenceID>& FocusedSubsequenceHierarchy, const UObject* InstanceData) const
{
	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();

	TArray<TPair<FMovieSceneSequenceID, FMovieSceneSequenceID>> ChildToParentMapping;

	// This array will be treated as an NxM, two-dimensional array that will hold the transform origins for all sequences in the focused hierarchy
	// N is the number of interrogations, and M is the number of Sequences in the currently focused hierarchy in Sequencer
	TArray<TOptional<FTransform>> TransformOrigins;
	// Reserve room for every sequence in the focused hierarchy to have entries for every time being interrogated.
	TransformOrigins.Init(TOptional<FTransform>(), Interrogations.Num() * FocusedSubsequenceHierarchy.Num());

	const IMovieSceneTransformOrigin* RawInterface = Cast<const IMovieSceneTransformOrigin>(InstanceData);

	// Read the world context actor's instance data.
	const bool bHasInterface = RawInterface || (InstanceData && InstanceData->GetClass()->ImplementsInterface(UMovieSceneTransformOrigin::StaticClass()));
	if (bHasInterface)
	{
		// Retrieve the current origin and use that for every sampled time for the root sequence transforms.
		const FTransform TransformOrigin = RawInterface ? RawInterface->GetTransformOrigin() : IMovieSceneTransformOrigin::Execute_BP_GetTransformOrigin(InstanceData);

		for (int32 OriginIndex = 0; OriginIndex < Interrogations.Num(); OriginIndex++)
		{
			TransformOrigins[OriginIndex] = TransformOrigin;
		}
	}
	
	// Read all transform origins from subsequences.
	auto GatherTransformOrigins = [Components, Hierarchy, &ChildToParentMapping, &TransformOrigins, &FocusedSubsequenceHierarchy, &InstanceData, this]
		(const FEntityAllocation* Allocation, const FMovieSceneSequenceID* SequenceIDs, const FInterrogationKey* InputKeys)
	{
		const int32 NumAllocations = Allocation->Num();
		const int32 StrideSize = Interrogations.Num();

		TOptionalComponentReader<double> DoubleComponents[6];
		for (int32 Index = 0; Index < 6; ++Index)
		{
			DoubleComponents[Index] = Allocation->TryReadComponents(Components->DoubleResult[Index]);
		}

		for (int32 ComponentIndex = 0; ComponentIndex < NumAllocations; ++ComponentIndex)
		{
			const FMovieSceneSequenceID CurrentSequence = SequenceIDs[ComponentIndex];
			const FInterrogationKey InterrogationKey = InputKeys[ComponentIndex];

			if (const FMovieSceneSequenceHierarchyNode* HierarchyNode = Hierarchy->FindNode(CurrentSequence))
			{
				// If one of the sequences isn't in the focused hierarchy, ignore it. This would be a sibling subsequence on the same track, and blending is not supported.
				// @todo support blending, and full subsequence hierarchies
				if (FocusedSubsequenceHierarchy.IndexOfByKey(CurrentSequence) == INDEX_NONE || FocusedSubsequenceHierarchy.IndexOfByKey(HierarchyNode->ParentID) == INDEX_NONE)
				{
					continue;
				}
				ChildToParentMapping.AddUnique(TPair<FMovieSceneSequenceID, FMovieSceneSequenceID>(CurrentSequence, HierarchyNode->ParentID));
			}
		
			FVector Translation = FVector::ZeroVector;
			FRotator Rotation = FRotator::ZeroRotator;

			if (DoubleComponents[0]) { Translation.X = DoubleComponents[0][ComponentIndex]; }
			if (DoubleComponents[1]) { Translation.Y = DoubleComponents[1][ComponentIndex]; }
			if (DoubleComponents[2]) { Translation.Z = DoubleComponents[2][ComponentIndex]; }
			if (DoubleComponents[3]) { Rotation.Roll = DoubleComponents[3][ComponentIndex]; }
			if (DoubleComponents[4]) { Rotation.Pitch = DoubleComponents[4][ComponentIndex]; }
			if (DoubleComponents[5]) { Rotation.Yaw = DoubleComponents[5][ComponentIndex]; }

			const int32 SequenceIndex = FocusedSubsequenceHierarchy.IndexOfByKey(CurrentSequence);
			if (SequenceIndex == INDEX_NONE)
			{
				continue;
			}
			const int32 TransformIndex = InterrogationKey.InterrogationIndex + (SequenceIndex * StrideSize);
			const FTransform TransformOrigin =  FTransform(Rotation, Translation);
			
			TransformOrigins[TransformIndex] = TransformOrigin;
		}
		
	};
	
	FEntityTaskBuilder()
	.Read(Components->SequenceID)
	.Read(Components->Interrogation.InputKey)
	.FilterAll({Components->Tags.SubInstance})
	.FilterAll({Components->Tags.ImportedEntity})
	.Iterate_PerAllocation(&Linker->EntityManager, GatherTransformOrigins);
	
	// Sort parent first
	Algo::Sort(ChildToParentMapping, [](TPair<FMovieSceneSequenceID, FMovieSceneSequenceID> A, TPair<FMovieSceneSequenceID, FMovieSceneSequenceID> B){return A.Key == B.Value;});

	// compose transform origins
	for (const TPair<FMovieSceneSequenceID, FMovieSceneSequenceID>& ChildParentPair : ChildToParentMapping)
	{
		const int32 ParentIndexStart = FocusedSubsequenceHierarchy.IndexOfByKey(ChildParentPair.Value) * Interrogations.Num();
		const int32 ChildIndexStart = FocusedSubsequenceHierarchy.IndexOfByKey(ChildParentPair.Key) * Interrogations.Num();

		for (int32 InterrogationIndex = 0; InterrogationIndex < Interrogations.Num(); ++InterrogationIndex)
		{
			const int32 ParentIndex = ParentIndexStart + InterrogationIndex;
			const int32 ChildIndex = ChildIndexStart + InterrogationIndex;

			// If the focused view in the sequence editor goes beyond the range of the subsequence, there will be interrogations where the subsequence isn't evaluated.
			if (TransformOrigins[ParentIndex].IsSet())
			{
				const FTransform ParentTransform = TransformOrigins[ParentIndex].GetValue();

				const bool bIsChildSet = TransformOrigins[ChildIndex].IsSet();

				TransformOrigins[ChildIndex] = bIsChildSet ? TransformOrigins[ChildIndex].GetValue() * ParentTransform : ParentTransform;
			}
		}
	}

	const int32 FocusedSequenceIndex = (FocusedSubsequenceHierarchy.Num() - 1) * Interrogations.Num();
	
	for (int32 OutTransformIndex = 0; OutTransformIndex < Interrogations.Num(); ++OutTransformIndex)
	{
		FTransform CurrentTransform = TransformOrigins[FocusedSequenceIndex + OutTransformIndex].IsSet() ? TransformOrigins[FocusedSequenceIndex + OutTransformIndex].GetValue() : FTransform::Identity;
		OutTransformOrigins.Add(CurrentTransform);
	}
}

EEntitySystemCategory FSystemInterrogator::GetInterrogationCategory()
{
	static EEntitySystemCategory Interrogation = UMovieSceneEntitySystem::RegisterCustomSystemCategory();
	return Interrogation;
}

EEntitySystemCategory FSystemInterrogator::GetExcludedFromInterrogationCategory()
{
	static EEntitySystemCategory ExcludedFromInterrogation = UMovieSceneEntitySystem::RegisterCustomSystemCategory();
	return ExcludedFromInterrogation;
}

FSystemInterrogator::FSystemInterrogator()
{
	InitialValueCache = FInitialValueCache::GetGlobalInitialValues();

	Linker = NewObject<UMovieSceneEntitySystemLinker>(GetTransientPackage());
	Linker->SetLinkerRole(EEntitySystemLinkerRole::Interrogation);
	Linker->GetSystemFilter().DisallowCategory(FSystemInterrogator::GetExcludedFromInterrogationCategory());

	Linker->AddExtension(IInterrogationExtension::GetExtensionID(), static_cast<IInterrogationExtension*>(this));
	Linker->AddExtension(InitialValueCache.Get());
}

FSystemInterrogator::~FSystemInterrogator()
{
}

void FSystemInterrogator::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObject(Linker);
}

FString FSystemInterrogator::GetReferencerName() const
{
	return TEXT("FSystemInterrogator");
}

void FSystemInterrogator::Reset()
{
	if (EntityTracker)
	{
		EntityTracker->Reset();
	}

	EntitiesScratch.Empty();
	EntityComponentField = FMovieSceneEntityComponentField();

	Channels.Reset();
	ExtraMetaData.Reset();

	Linker->Reset();
}

void FSystemInterrogator::ImportTrack(UMovieSceneTrack* Track, FInterrogationChannel InChannel, FMovieSceneSequenceID SequenceID)
{
	ImportTrack(Track, Track->FindObjectBindingGuid(), InChannel, SequenceID);
}

void FSystemInterrogator::ImportTrack(UMovieSceneTrack* Track, const FGuid& ObjectBindingID, FInterrogationChannel InChannel, FMovieSceneSequenceID SequenceID)
{
	if (Track->IsEvalDisabled())
	{
		return;
	}

	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &Linker->EntityManager);

	const FMovieSceneTrackEvaluationField& EvaluationField = Track->GetEvaluationField();

	FMovieSceneEntityComponentFieldBuilder FieldBuilder(&EntityComponentField);
	FieldBuilder.GetSharedMetaData().ObjectBindingID = ObjectBindingID;

	// Creating a new builder allocates a new piece of shared metadata. We will use that shared metadata's index
	// to keep track of the extra metadata, since the extra metadata is also shared.
	const int32 SharedMetaDataIndex = FieldBuilder.GetSharedMetaDataIndex();
	UMovieSceneSequence* OwningSequence = Track->GetTypedOuter<UMovieSceneSequence>();
	if (!ExtraMetaData.IsValidIndex(SharedMetaDataIndex))
	{
		ExtraMetaData.Insert(SharedMetaDataIndex, FExtraMetaData{ FInterrogationInstance { OwningSequence }, InChannel, SequenceID });
	}
	else
	{
		const FExtraMetaData& InterrogationMetaData = ExtraMetaData[SharedMetaDataIndex];
		ensure(InterrogationMetaData.InterrogationInstance.Sequence == OwningSequence && InterrogationMetaData.InterrogationChannel == InChannel);
	}

	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : EvaluationField.Entries)
	{
		IMovieSceneEntityProvider* EntityProvider = Cast<IMovieSceneEntityProvider>(Entry.Section);

		if (!EntityProvider || Entry.Range.IsEmpty())
		{
			continue;
		}

		if (Track->IsRowEvalDisabled(Entry.Section->GetRowIndex()))
		{
			continue;
		}

		FMovieSceneEvaluationFieldEntityMetaData MetaData;

		MetaData.ForcedTime = Entry.ForcedTime;
		MetaData.Flags      = Entry.Flags;
		MetaData.bEvaluateInSequencePreRoll  = Track->EvalOptions.bEvaluateInPreroll;
		MetaData.bEvaluateInSequencePostRoll = Track->EvalOptions.bEvaluateInPostroll;

		MetaData.Condition = MovieSceneHelpers::GetSequenceCondition(Track, Entry.Section);

		if (!EntityProvider->PopulateEvaluationField(Entry.Range, MetaData, &FieldBuilder))
		{
			const int32 EntityIndex   = FieldBuilder.FindOrAddEntity(Entry.Section, 0);
			const int32 MetaDataIndex = FieldBuilder.AddMetaData(MetaData);
			FieldBuilder.AddPersistentEntity(Entry.Range, EntityIndex, MetaDataIndex);
		}
	}

	Channels.ActivateChannel(InChannel);
}

int32 FSystemInterrogator::AddInterrogation(const FInterrogationParams& Params)
{
	const int32 NewInterrogationIndex = Channels.AddInterrogation(Params);
	if (NewInterrogationIndex == INDEX_NONE)
	{
		return INDEX_NONE;
	}

	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &Linker->EntityManager);

	TRange<FFrameNumber> UnusedEntityRange;

	// Update the entities that exist at this frame
	EntitiesScratch.Reset();
	EntityComponentField.QueryPersistentEntities(Params.Time.FrameNumber, UnusedEntityRange, EntitiesScratch);

	for (const FMovieSceneEvaluationFieldEntityQuery& Query : EntitiesScratch)
	{
		InterrogateEntity(NewInterrogationIndex, Query);
	}

	return NewInterrogationIndex;
}

void FSystemInterrogator::InterrogateEntity(int32 InterrogationIndex, const FMovieSceneEvaluationFieldEntityQuery& Query)
{
	UObject* EntityOwner = Query.Entity.Key.EntityOwner.Get();
	IMovieSceneEntityProvider* Provider = Cast<IMovieSceneEntityProvider>(EntityOwner);
	if (!Provider)
	{
		return;
	}

	FEntityImportParams Params;
	Params.EntityID = Query.Entity.Key.EntityID;
	Params.EntityMetaData = EntityComponentField.FindMetaData(Query);
	Params.SharedMetaData = EntityComponentField.FindSharedMetaData(Query);

	Params.InterrogationKey.Channel = FInterrogationChannel::Default();
	Params.InterrogationKey.InterrogationIndex = InterrogationIndex;

	const int32 SharedMetaDataIndex = Query.Entity.SharedMetaDataIndex;
	if (SharedMetaDataIndex != INDEX_NONE && ensure(ExtraMetaData.IsValidIndex(SharedMetaDataIndex)))
	{
		const FExtraMetaData& InterrogationMetaData = ExtraMetaData[SharedMetaDataIndex];
		Params.InterrogationKey.Channel = InterrogationMetaData.InterrogationChannel;
		Params.InterrogationInstance = InterrogationMetaData.InterrogationInstance;
		Params.Sequence.SequenceID = InterrogationMetaData.InterrogationSequenceID;
	}

	FImportedEntity ImportedEntity;
	Provider->InterrogateEntity(Linker, Params, &ImportedEntity);

	if (!ImportedEntity.IsEmpty())
	{
		if (UMovieSceneSection* Section = Cast<UMovieSceneSection>(EntityOwner))
		{
			Section->BuildDefaultComponents(Linker, Params, &ImportedEntity);
		}

		const FMovieSceneEntityID NewEntityID = ImportedEntity.Manufacture(Params, &Linker->EntityManager);
		if (EntityTracker)
		{
			EntityTracker->TrackEntity(Params.InterrogationKey, Query.Entity.Key, NewEntityID);
		}
	}
}

void FSystemInterrogator::Update()
{
	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &Linker->EntityManager);

	Linker->EntityManager.AddMutualComponents();
	Linker->LinkRelevantSystems();

	TArrayView<const FInterrogationParams> Interrogations = Channels.GetInterrogations();

	// Compute evaluation time (in frames) for all entities that need it.
	FEntityTaskBuilder()
	.Read(FBuiltInComponentTypes::Get()->Interrogation.InputKey)
	.Write(FBuiltInComponentTypes::Get()->EvalTime)
	.FilterNone({ FBuiltInComponentTypes::Get()->Tags.FixedTime })
	.Iterate_PerEntity(
		&Linker->EntityManager, 
		[&Interrogations](const FInterrogationKey& InterrogationKey, FFrameTime& OutEvalTime)
		{
			const FInterrogationParams& InterrogationParams(Interrogations[InterrogationKey.InterrogationIndex]);
			OutEvalTime = InterrogationParams.Time;
		});

	// Compute evaluation time (in seconds) for all entities that need it.
	FEntityTaskBuilder()
	.Read(FBuiltInComponentTypes::Get()->Interrogation.InputKey)
	.Read(FBuiltInComponentTypes::Get()->Interrogation.Instance)
	.Write(FBuiltInComponentTypes::Get()->EvalSeconds)
	.FilterNone({ FBuiltInComponentTypes::Get()->Tags.FixedTime })
	.Iterate_PerEntity(
		&Linker->EntityManager, 
		[&Interrogations](const FInterrogationKey& InterrogationKey, const FInterrogationInstance& InterrogationInstance, double& OutEvalSeconds)
		{
			const FInterrogationParams& InterrogationParams(Interrogations[InterrogationKey.InterrogationIndex]);
			if (ensure(InterrogationInstance.Sequence && InterrogationInstance.Sequence->GetMovieScene()))
			{
				const FFrameRate& TickResolution = InterrogationInstance.Sequence->GetMovieScene()->GetTickResolution();
				OutEvalSeconds = TickResolution.AsSeconds(InterrogationParams.Time);
			}
		});

	TSharedPtr<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();
	Runner->Flush();

	Linker->EntityManager.IncrementSystemSerial();
}

void FSystemInterrogator::TrackImportedEntities(bool bInTrackImportedEntities)
{
	if (bInTrackImportedEntities && !EntityTracker)
	{
		EntityTracker = MakeUnique<FSystemInterrogatorEntityTracker>();
	}
	else if (!bInTrackImportedEntities)
	{
		EntityTracker.Reset();
	}
}

FMovieSceneEntityID FSystemInterrogator::FindEntityFromOwner(FInterrogationKey InterrogationKey, UObject* Owner, uint32 EntityID) const
{
	if (!ensureMsgf(EntityTracker.IsValid(), TEXT("FindEntityFromOwner called on an interrogator that was not tracking entities")))
	{
		return FMovieSceneEntityID::Invalid();
	}

	return EntityTracker->FindTrackedEntity(InterrogationKey, FMovieSceneEvaluationFieldEntityKey { Owner, EntityID });;
}

FInterrogationChannel FSystemInterrogator::ImportLocalTransforms(USceneComponent* SceneComponent, IMovieScenePlayer* InPlayer, FMovieSceneSequenceID SequenceID)
{
	check(SceneComponent);

	FMovieSceneEvaluationState* State = InPlayer->GetEvaluationState();
	UMovieSceneSequence* Sequence = State->FindSequence(SequenceID);
	if (!Sequence)
	{
		return FInterrogationChannel::Invalid();
	}

	FInterrogationChannel ParentChannel;
	if (USceneComponent* AttachParent = SceneComponent->GetAttachParent())
	{
		ParentChannel = Channels.FindChannel(AttachParent);
	}

	FInterrogationChannel Channel = Channels.FindChannel(SceneComponent);
	if (!Channel.IsValid())
	{
		Channel = Channels.AllocateChannel(SceneComponent, ParentChannel, FMovieScenePropertyBinding("Transform", TEXT("Transform")));
	}

	if (!Channel.IsValid())
	{
		return FInterrogationChannel::Invalid();
	}

	// Find the binding that corresponds to the component directly
	FGuid ObjectBindingID = State->FindCachedObjectId(*SceneComponent, SequenceID, *InPlayer);
	if (ObjectBindingID.IsValid())
	{
		const FMovieSceneBinding* Binding = Sequence->GetMovieScene()->FindBinding(ObjectBindingID);
		if (ensure(Binding))
		{
			ImportTransformTracks(*Binding, Channel);
		}
	}

	// Also blend in any transforms that exist for this scene component's actor as well (if it is the root)
	AActor* Owner = SceneComponent->GetOwner();
	if (SceneComponent == Owner->GetRootComponent())
	{
		FGuid OwnerObjectBindingID = State->FindCachedObjectId(*Owner, SequenceID, *InPlayer);
		if (OwnerObjectBindingID.IsValid())
		{
			const FMovieSceneBinding* Binding = Sequence->GetMovieScene()->FindBinding(OwnerObjectBindingID);
			if (ensure(Binding))
			{
				ImportTransformTracks(*Binding, Channel);
			}
		}
	}

	return Channel;
}

FInterrogationChannel FSystemInterrogator::ImportTransformHierarchy(USceneComponent* SceneComponent, IMovieScenePlayer* InPlayer, FMovieSceneSequenceID SequenceID)
{
	check(SceneComponent);

	if (USceneComponent* AttachParent = SceneComponent->GetAttachParent())
	{
		ImportTransformHierarchy(AttachParent, InPlayer, SequenceID);
	}

	return ImportLocalTransforms(SceneComponent, InPlayer, SequenceID);
}

void FSystemInterrogator::ImportTransformTracks(const FMovieSceneBinding& Binding, FInterrogationChannel Channel)
{
	for (UMovieSceneTrack* Track : Binding.GetTracks())
	{
		if (Track->IsA<UMovieScene3DTransformTrack>())
		{
			ImportTrack(Track, Binding.GetObjectGuid(), Channel);
		}
	}
}

void FSystemInterrogator::FindPropertyOutputEntityIDs(const FPropertyDefinition& PropertyDefinition, FInterrogationChannel Channel, TArray<FMovieSceneEntityID>& OutEntityIDs) const
{
	UMovieSceneInterrogatedPropertyInstantiatorSystem* PropertyInstantiator = Linker->LinkSystem<UMovieSceneInterrogatedPropertyInstantiatorSystem>();
	check(PropertyInstantiator);

	TArrayView<const FInterrogationParams> Interrogations = Channels.GetInterrogations();
	OutEntityIDs.SetNum(Interrogations.Num());

	for (int32 Index = 0; Index < Interrogations.Num(); ++Index)
	{
		FInterrogationKey Key(Channel, Index);
		const UMovieSceneInterrogatedPropertyInstantiatorSystem::FPropertyInfo* PropertyInfo = PropertyInstantiator->FindPropertyInfo(Key);
		if (ensure(PropertyInfo))
		{
			if (PropertyInfo->PropertyEntityID.IsValid())
			{
				OutEntityIDs[Index] = PropertyInfo->PropertyEntityID;
			}
			else
			{
				TArray<FMovieSceneEntityID> InputEntityIDs;
				PropertyInstantiator->FindEntityIDs(Key, InputEntityIDs);
				if (ensure(InputEntityIDs.Num() == 1))
				{
					OutEntityIDs[Index] = InputEntityIDs[0];
				}
			}
		}
	}
}

} // namespace MovieScene
} // namespace UE
