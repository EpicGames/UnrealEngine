// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/CameraCutPlaybackCapability.h"

namespace UE::MovieScene
{

UE_DEFINE_MOVIESCENE_PLAYBACK_CAPABILITY(FCameraCutPlaybackCapability)

} // namespace UE::MovieScene

