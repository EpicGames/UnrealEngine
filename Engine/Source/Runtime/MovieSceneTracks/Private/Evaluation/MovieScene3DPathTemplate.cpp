// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieScene3DPathTemplate.h"
#include "Evaluation/MovieSceneTemplateCommon.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTracksPropertyTypes.h"
#include "GameFramework/Actor.h"
#include "Components/SplineComponent.h"
#include "Evaluation/MovieSceneEvaluation.h"
#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DPathTemplate)


DECLARE_CYCLE_STAT(TEXT("Path Track Evaluate"), MovieSceneEval_PathTrack_Evaluate, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Path Track Token Execute"), MovieSceneEval_PathTrack_TokenExecute, STATGROUP_MovieSceneEval);


/** A movie scene execution token that stores a specific spline component, and an operand */
struct F3DPathExecutionToken
	: IMovieSceneExecutionToken
{
	F3DPathExecutionToken(FMovieSceneObjectBindingID InPathBindingID, float InTiming, MovieScene3DPathSection_Axis InFrontAxisEnum, MovieScene3DPathSection_Axis InUpAxisEnum, bool bInFollow, bool bInReverse, bool bInForceUpright) 
		: PathBindingID(InPathBindingID)
		, Timing(InTiming)
		, FrontAxisEnum(InFrontAxisEnum)
		, UpAxisEnum(InUpAxisEnum)
		, bFollow(bInFollow)
		, bReverse(bInReverse)
		, bForceUpright(bInForceUpright)
	{}

	void Eval(USceneComponent* SceneComponent, USplineComponent* SplineComponent, FVector& OutTranslation, FRotator& OutRotation)
	{
		if (Timing < 0.f)
		{
			Timing = 0.f;
		}
		else if (Timing > 1.f)
		{
			Timing = 1.f;
		}

		if (bReverse)
		{
			Timing = 1.f - Timing;
		}

		const bool UseConstantVelocity = true;
		OutTranslation = SplineComponent->GetWorldLocationAtTime(Timing, UseConstantVelocity);
		OutRotation = SplineComponent->GetWorldRotationAtTime(Timing, UseConstantVelocity);
	
		FMatrix NewRotationMatrix = FRotationMatrix(OutRotation);

		FVector UpAxis(0, 0, 0);
		if (UpAxisEnum == MovieScene3DPathSection_Axis::X)
		{
			UpAxis = FVector(1, 0, 0);
		}
		else if (UpAxisEnum == MovieScene3DPathSection_Axis::NEG_X)
		{
			UpAxis = FVector(-1, 0, 0);
		}
		else if (UpAxisEnum == MovieScene3DPathSection_Axis::Y)
		{
			UpAxis = FVector(0, 1, 0);
		}
		else if (UpAxisEnum == MovieScene3DPathSection_Axis::NEG_Y)
		{
			UpAxis = FVector(0, -1, 0);
		}
		else if (UpAxisEnum == MovieScene3DPathSection_Axis::Z)
		{
			UpAxis = FVector(0, 0, 1);
		}
		else if (UpAxisEnum == MovieScene3DPathSection_Axis::NEG_Z)
		{
			UpAxis = FVector(0, 0, -1);
		}

		FVector FrontAxis(0, 0, 0);
		if (FrontAxisEnum == MovieScene3DPathSection_Axis::X)
		{
			FrontAxis = FVector(1, 0, 0);
		}
		else if (FrontAxisEnum == MovieScene3DPathSection_Axis::NEG_X)
		{
			FrontAxis = FVector(-1, 0, 0);
		}
		else if (FrontAxisEnum == MovieScene3DPathSection_Axis::Y)
		{
			FrontAxis = FVector(0, 1, 0);
		}
		else if (FrontAxisEnum == MovieScene3DPathSection_Axis::NEG_Y)
		{
			FrontAxis = FVector(0, -1, 0);
		}
		else if (FrontAxisEnum == MovieScene3DPathSection_Axis::Z)
		{
			FrontAxis = FVector(0, 0, 1);
		}
		else if (FrontAxisEnum == MovieScene3DPathSection_Axis::NEG_Z)
		{
			FrontAxis = FVector(0, 0, -1);
		}

		// Negate the front axis because the spline rotation comes in reversed
		FrontAxis *= FVector(-1, -1, -1);

		FMatrix AxisRotator = FRotationMatrix::MakeFromXZ(FrontAxis, UpAxis);
		NewRotationMatrix = AxisRotator * NewRotationMatrix;
		OutRotation = NewRotationMatrix.Rotator();

		if (bForceUpright)
		{
			OutRotation.Pitch = 0.f;
			OutRotation.Roll = 0.f;
		}

		if (!bFollow)
		{
			OutRotation = SceneComponent->GetRelativeTransform().GetRotation().Rotator();
		}
	}

	/** Execute this token, operating on all objects referenced by 'Operand' */
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PathTrack_TokenExecute)

		USplineComponent* SplineComponent = nullptr;

		// Find the first spline component on any of the objects resolved through the path binding ID
		for (TWeakObjectPtr<> PathTarget : PathBindingID.ResolveBoundObjects(Operand.SequenceID, Player))
		{
			if (AActor* Actor = Cast<AActor>(PathTarget.Get()))
			{
				SplineComponent = Actor->FindComponentByClass<USplineComponent>();
				if (SplineComponent)
				{
					break;
				}
			}
		}

		if (SplineComponent == nullptr)
		{
			return;
		}

		for (TWeakObjectPtr<> Object : Player.FindBoundObjects(Operand))
		{
			UObject* ObjectPtr = Object.Get();
			USceneComponent* SceneComponent = ObjectPtr ? MovieSceneHelpers::SceneComponentFromRuntimeObject(ObjectPtr) : nullptr;

			if (SceneComponent)
			{
				Player.SavePreAnimatedState(*SceneComponent, FMobilityTokenProducer::GetAnimTypeID(), FMobilityTokenProducer());
				Player.SavePreAnimatedState(*SceneComponent, F3DTransformTokenProducer::GetAnimTypeID(), F3DTransformTokenProducer());

				FVector Location;
				FRotator Rotation;
				Eval(SceneComponent, SplineComponent, Location, Rotation);
				
				SceneComponent->SetMobility(EComponentMobility::Movable);

				UE::MovieScene::FIntermediate3DTransform Transform(Location, Rotation, FVector(1.0));
				UE::MovieScene::FIntermediate3DTransform::ApplyTranslationAndRotationTo(SceneComponent, Transform);
			}
		}
	}
	
	FMovieSceneObjectBindingID PathBindingID;
	float Timing;
	MovieScene3DPathSection_Axis FrontAxisEnum;
	MovieScene3DPathSection_Axis UpAxisEnum;
	bool bFollow;
	bool bReverse;
	bool bForceUpright;
};

FMovieScene3DPathSectionTemplate::FMovieScene3DPathSectionTemplate(const UMovieScene3DPathSection& Section)
	: PathBindingID(Section.GetConstraintBindingID())
	, TimingCurve(Section.GetTimingChannel())
	, FrontAxisEnum(Section.GetFrontAxisEnum())
	, UpAxisEnum(Section.GetUpAxisEnum())
	, bFollow(Section.GetFollow())
	, bReverse(Section.GetReverse())
	, bForceUpright(Section.GetForceUpright())
{
}

void FMovieScene3DPathSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_PathTrack_Evaluate)

	float Timing = 0.f;
	if (TimingCurve.Evaluate(Context.GetTime(), Timing))
	{
		ExecutionTokens.Add(F3DPathExecutionToken(PathBindingID, Timing, FrontAxisEnum, UpAxisEnum, bFollow, bReverse, bForceUpright));
	}
}

