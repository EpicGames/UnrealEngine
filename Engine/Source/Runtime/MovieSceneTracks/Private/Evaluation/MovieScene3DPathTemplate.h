// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Misc/Guid.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Sections/MovieScene3DPathSection.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieScene3DPathTemplate.generated.h"

USTRUCT()
struct FMovieScene3DPathSectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()
	
	FMovieScene3DPathSectionTemplate() 
		: FrontAxisEnum(MovieScene3DPathSection_Axis::X)
		, UpAxisEnum(MovieScene3DPathSection_Axis::X)
		, bFollow(false)
		, bReverse(false)
		, bForceUpright(false)
	{}
	FMovieScene3DPathSectionTemplate(const UMovieScene3DPathSection& Section);

	/** The object binding ID of the path we should attach to */
	UPROPERTY()
	FMovieSceneObjectBindingID PathBindingID;

	/** The timing curve */
	UPROPERTY()
	FMovieSceneFloatChannel TimingCurve;

	/** Front Axis */
	UPROPERTY()
	MovieScene3DPathSection_Axis FrontAxisEnum;

	/** Up Axis */
	UPROPERTY()
	MovieScene3DPathSection_Axis UpAxisEnum;

	/** Follow Curve */
	UPROPERTY()
	uint32 bFollow:1;

	/** Reverse Timing */
	UPROPERTY()
	uint32 bReverse:1;

	/** Force Upright */
	UPROPERTY()
	uint32 bForceUpright:1;

private:

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;
};
