// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneBaseCacheTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBaseCacheTemplate)

DECLARE_CYCLE_STAT(TEXT("Chaos Cache Evaluate"), MovieSceneEval_BaseCache_Evaluate, STATGROUP_MovieSceneEval);
DECLARE_CYCLE_STAT(TEXT("Chaos Cache Token Execute"), MovieSceneEval_BaseCache_TokenExecute, STATGROUP_MovieSceneEval);

float FMovieSceneBaseCacheSectionTemplateParameters::MapTimeToAnimation(const FMovieSceneBaseCacheParams& BaseParams, float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const
{
	const float SequenceLength = ComponentDuration;
	const FFrameTime AnimationLength = SequenceLength * InTickResolution;
	const int32 LengthInFrames = AnimationLength.FrameNumber.Value + (int)(AnimationLength.GetSubFrame() + 0.5f) + 1;
	
	//we only play end if we are not looping, and assuming we are looping if Length is greater than default length;
	const bool bLooping = (SectionEndTime.Value - SectionStartTime.Value + BaseParams.StartFrameOffset + BaseParams.EndFrameOffset) > LengthInFrames;

	InPosition = FMath::Clamp(InPosition, FFrameTime(SectionStartTime), FFrameTime(SectionEndTime - 1));

	const float SectionPlayRate = BaseParams.PlayRate;
	const float AnimPlayRate = FMath::IsNearlyZero(SectionPlayRate) ? 1.0f : SectionPlayRate;

	const float FirstLoopSeqLength = SequenceLength - InTickResolution.AsSeconds(BaseParams.FirstLoopStartFrameOffset + BaseParams.StartFrameOffset + BaseParams.EndFrameOffset);
	const float SeqLength = SequenceLength - InTickResolution.AsSeconds(BaseParams.StartFrameOffset + BaseParams.EndFrameOffset);

	float AnimPosition = FFrameTime::FromDecimal((InPosition - SectionStartTime).AsDecimal() * AnimPlayRate) / InTickResolution;
	AnimPosition += InTickResolution.AsSeconds(BaseParams.FirstLoopStartFrameOffset);
	if (SeqLength > 0.f && (bLooping || !FMath::IsNearlyEqual(AnimPosition, SeqLength, 1e-4f)))
	{
		AnimPosition = FMath::Fmod(AnimPosition, SeqLength);
	}
	AnimPosition += InTickResolution.AsSeconds(BaseParams.StartFrameOffset);
	if (BaseParams.bReverse)
	{
		AnimPosition = SequenceLength - AnimPosition;
	}

	return AnimPosition;
}
