// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneCameraShakePreviewer.h"

#if WITH_EDITOR

#include "Camera/CameraModifier_CameraShake.h"
#include "Camera/CameraShakeSourceComponent.h"
#include "LevelEditorViewport.h"
#include "MovieSceneFwd.h"

FCameraShakePreviewer::FCameraShakePreviewer(UWorld* InWorld)
	: World(InWorld)
	, LastDeltaTime(0.f)
	, LastLocationModifier(FVector::ZeroVector)
	, LastRotationModifier(FRotator::ZeroRotator)
	, LastFOVModifier(0.f)
{
	// Handle camera shakes being recompiled.
	FCoreUObjectDelegates::OnObjectsReplaced.AddRaw(this, &FCameraShakePreviewer::OnObjectsReplaced);
}

FCameraShakePreviewer::~FCameraShakePreviewer()
{
	if (!ensureMsgf(RegisteredViewportClients.Num() == 0, TEXT("Forgot to call UnRegisterViewModifiers!")))
	{
		UnRegisterViewModifiers();
	}

	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
}

UCameraShakeBase* FCameraShakePreviewer::AddCameraShake(const FCameraShakePreviewerAddParams& Params)
{
	FCameraShakeBaseStartParams StartParams;
	StartParams.Scale = Params.Scale;
	StartParams.PlaySpace = Params.PlaySpace;
	StartParams.UserPlaySpaceRot = Params.UserPlaySpaceRot;
	StartParams.DurationOverride = Params.DurationOverride;

	UCameraShakeBase* NewShake = NewObject<UCameraShakeBase>(World, Params.ShakeClass);
	NewShake->StartShake(StartParams);

	ActiveShakes.Add({ StartParams, NewShake, Params.SourceComponent, Params.GlobalStartTime });

	UE_LOGF(LogMovieScene, Verbose, "CameraShakePreviewer::AddCameraShake '%ls'", *GetNameSafe(NewShake));

	return NewShake;
}

void FCameraShakePreviewer::RemoveCameraShake(UCameraShakeBase* ShakeInstance)
{
	const bool bImmediately = true;
	for (int32 i = ActiveShakes.Num() - 1; i >= 0; --i)
	{
		FPreviewCameraShakeInfo& ActiveShake = ActiveShakes[i];
		if (ActiveShake.ShakeInstance == ShakeInstance)
		{
			UE_LOGF(LogMovieScene, Verbose, "CameraShakePreviewer::RemoveCameraShake '%ls'", *GetNameSafe(ActiveShake.ShakeInstance));

			ActiveShake.ShakeInstance->StopShake(bImmediately);
			ActiveShake.ShakeInstance->TeardownShake();
			ActiveShakes.RemoveAt(i, 1);
			break;
		}
	}
}

void FCameraShakePreviewer::RemoveAllCameraShakesFromSource(const UCameraShakeSourceComponent* SourceComponent)
{
	int32 NumRemoved = 0;
	const bool bImmediately = true;
	for (int32 i = ActiveShakes.Num() - 1; i >= 0; --i)
	{
		FPreviewCameraShakeInfo& ActiveShake = ActiveShakes[i];
		if (ActiveShake.SourceComponent.Get() == SourceComponent && ActiveShake.ShakeInstance != nullptr)
		{
			ActiveShake.ShakeInstance->StopShake(bImmediately);
			ActiveShake.ShakeInstance->TeardownShake();
			ActiveShakes.RemoveAt(i, 1);
			++NumRemoved;
		}
	}
	UE_LOGF(LogMovieScene, Verbose, "CameraShakePreviewer::RemoveAllCameraShakesFromSource '%ls', %d removed", 
			*GetNameSafe(SourceComponent), NumRemoved);
}

void FCameraShakePreviewer::RemoveAllCameraShakes()
{
	const bool bImmediately = true;
	for (FPreviewCameraShakeInfo& ActiveShake : ActiveShakes)
	{
		if (ActiveShake.ShakeInstance)
		{
			ActiveShake.ShakeInstance->StopShake(bImmediately);
			ActiveShake.ShakeInstance->TeardownShake();
		}
	}
	UE_LOGF(LogMovieScene, Verbose, "CameraShakePreviewer::RemoveAllCameraShakes, %d removed", ActiveShakes.Num());
	ActiveShakes.Empty();
}

void FCameraShakePreviewer::GetActiveCameraShakes(TArray<FActiveCameraShakeInfo>& ActiveCameraShakes) const
{
	for (const FPreviewCameraShakeInfo& ActiveShake : ActiveShakes)
	{
		FActiveCameraShakeInfo ShakeInfo;
		ShakeInfo.ShakeInstance = ActiveShake.ShakeInstance;
		ShakeInfo.ShakeSource = ActiveShake.SourceComponent;
		ActiveCameraShakes.Add(ShakeInfo);
	}
}

void FCameraShakePreviewer::Update(float DeltaTime, bool bIsPlaying)
{
	LastDeltaTime = DeltaTime;
	LastScrubTime.Reset();

	if (!bIsPlaying)
	{
		ResetModifiers();
	}
}

void FCameraShakePreviewer::Scrub(float ScrubTime)
{
	LastDeltaTime.Reset();
	LastScrubTime = ScrubTime;

	ResetModifiers();
}

void FCameraShakePreviewer::ResetModifiers()
{
	LastLocationModifier = FVector::ZeroVector;
	LastRotationModifier = FRotator::ZeroRotator;
	LastFOVModifier = 0.f;

	LastPostProcessSettings.Reset();
	LastPostProcessBlendWeights.Reset();
}

void FCameraShakePreviewer::ModifyView(FEditorViewportViewModifierParams& Params)
{
	OnModifyView(Params);
}

void FCameraShakePreviewer::OnModifyView(FEditorViewportViewModifierParams& Params)
{
	FMinimalViewInfo& InOutPOV(Params.ViewInfo);
	const FMinimalViewInfo OriginalPOV(Params.ViewInfo);

	// This is a simpler version of what UCameraModifier_CameraShake does, with extra
	// support for scrubbing.
	if (LastDeltaTime.IsSet() || LastScrubTime.IsSet())
	{
		LastPostProcessSettings.Reset();
		LastPostProcessBlendWeights.Reset();

		for (FPreviewCameraShakeInfo& ActiveShake : ActiveShakes)
		{
			if (ActiveShake.ShakeInstance != nullptr)
			{
				float CurShakeAlpha = 1.f;

				if (ActiveShake.SourceComponent.IsValid())
				{
					const UCameraShakeSourceComponent* SourceComponent = ActiveShake.SourceComponent.Get();
					const float AttenuationFactor = SourceComponent->GetAttenuationFactor(InOutPOV.Location);
					CurShakeAlpha *= AttenuationFactor;
				}

				if (LastDeltaTime.IsSet())
				{
					ActiveShake.ShakeInstance->UpdateAndApplyCameraShake(LastDeltaTime.GetValue(), CurShakeAlpha, InOutPOV);
				}
				else if (LastScrubTime.IsSet())
				{
					float RelativeScrubTime = LastScrubTime.GetValue() - ActiveShake.StartTime;
					ActiveShake.ShakeInstance->ScrubAndApplyCameraShake(RelativeScrubTime, CurShakeAlpha, InOutPOV);
				}

				if (InOutPOV.PostProcessBlendWeight > 0.f)
				{
					Params.AddPostProcessBlend(InOutPOV.PostProcessSettings, InOutPOV.PostProcessBlendWeight);
					LastPostProcessSettings.Add(InOutPOV.PostProcessSettings);
					LastPostProcessBlendWeights.Add(InOutPOV.PostProcessBlendWeight);
				}
				InOutPOV.PostProcessSettings = FPostProcessSettings();
				InOutPOV.PostProcessBlendWeight = 0.f;
			}
		}

		LastLocationModifier = InOutPOV.Location - OriginalPOV.Location;
		LastRotationModifier = InOutPOV.Rotation - OriginalPOV.Rotation;
		LastFOVModifier = InOutPOV.FOV - OriginalPOV.FOV;

		LastDeltaTime.Reset();
		LastScrubTime.Reset();

		// Delete any obsolete shakes.
		for (int32 i = ActiveShakes.Num() - 1; i >= 0; i--)
		{
			const FPreviewCameraShakeInfo& ShakeInfo = ActiveShakes[i];
			if (ShakeInfo.ShakeInstance == nullptr || ShakeInfo.ShakeInstance->IsFinished() || ShakeInfo.SourceComponent.IsStale())
			{
				if (ShakeInfo.ShakeInstance != nullptr)
				{
					ShakeInfo.ShakeInstance->TeardownShake();
				}

				UE_LOGF(LogMovieScene, Verbose, "CameraShakePreviewer::OnModifyView, '%ls' finished or stale", 
						*GetNameSafe(ShakeInfo.ShakeInstance));
				ActiveShakes.RemoveAt(i, 1);
			}
		}
	}
	else
	{
		InOutPOV.Location += LastLocationModifier;
		InOutPOV.Rotation += LastRotationModifier;
		InOutPOV.FOV += LastFOVModifier;

		for (int32 PPIndex = 0; PPIndex < LastPostProcessSettings.Num(); ++PPIndex)
		{
			Params.AddPostProcessBlend(LastPostProcessSettings[PPIndex], LastPostProcessBlendWeights[PPIndex]);
		}
	}
}

void FCameraShakePreviewer::RegisterViewModifiers(bool bIgnoreDuplicateRegistration)
{
	RegisterViewModifiers([](FLevelEditorViewportClient*) { return true; }, bIgnoreDuplicateRegistration);
}

void FCameraShakePreviewer::RegisterViewModifiers(FViewportFilter ViewportFilter, bool bIgnoreDuplicateRegistration)
{
	if (GEditor == nullptr)
	{
		return;
	}

	// Register our view modifier on all appropriate viewports, and remember which viewports we did that on.
	// We will later make sure to unregister on the same list, except for any viewport that somehow disappeared since,
	// which we will be notified about with the OnLevelViewportClientListChanged event.
	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{		
		if (LevelVC && 
				LevelVC->GetViewMode() != VMI_Unknown &&
				LevelVC->GetWorld() == World &&
				ViewportFilter(LevelVC))
		{
			RegisterViewModifier(LevelVC, bIgnoreDuplicateRegistration);
		}
	}
}

void FCameraShakePreviewer::RegisterViewModifier(FLevelEditorViewportClient* ViewportClient, bool bIgnoreDuplicateRegistration)
{
	if (RegisteredViewportClients.Contains(ViewportClient))
	{
		// Already registered to this viewport.
		ensureMsgf(bIgnoreDuplicateRegistration, TEXT("Given viewport is already registered."));
		return;
	}

	UE_LOGF(LogMovieScene, Verbose, "CameraShakePreviewer::RegisterViewModifier");
	RegisteredViewportClients.Add(ViewportClient);
	ViewportClient->ViewModifiers.AddRaw(this, &FCameraShakePreviewer::OnModifyView);
	if (RegisteredViewportClients.Num() == 1)
	{
		// If this is the first viewport, start listening to viewports changing.
		GEditor->OnLevelViewportClientListChanged().AddRaw(this, &FCameraShakePreviewer::OnLevelViewportClientListChanged);
	}
}


void FCameraShakePreviewer::UnRegisterViewModifiers()
{
	if (GEditor == nullptr)
	{
		return;
	}

	GEditor->OnLevelViewportClientListChanged().RemoveAll(this);

	for (FLevelEditorViewportClient* ViewportClient : RegisteredViewportClients)
	{
		ViewportClient->ViewModifiers.RemoveAll(this);
	}
	UE_LOGF(LogMovieScene, Verbose, "CameraShakePreviewer::UnRegisterViewModifiers, %d unregistered", 
			RegisteredViewportClients.Num());
	RegisteredViewportClients.Reset();
}

void FCameraShakePreviewer::UnRegisterViewModifier(FLevelEditorViewportClient* ViewportClient)
{
	const int32 NumRemoved = RegisteredViewportClients.Remove(ViewportClient);
	if (ensureMsgf(NumRemoved > 0, TEXT("The given viewport client wasn't registered.")))
	{
		UE_LOGF(LogMovieScene, Verbose, "CameraShakePreviewer::UnRegisterViewModifier");

		// If this is the last viewport, stop listening to viewports changing.
		if (RegisteredViewportClients.IsEmpty())
		{
			GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
		}

		ViewportClient->ViewModifiers.RemoveAll(this);
	}
}

void FCameraShakePreviewer::OnLevelViewportClientListChanged()
{
	if (GEditor != nullptr)
	{
		// If any viewports were removed while we were playing, simply get rid of them from our list of
		// registered viewports.
		TSet<FLevelEditorViewportClient*> PreviousViewportClients(RegisteredViewportClients);
		TSet<FLevelEditorViewportClient*> NewViewportClients(GEditor->GetLevelViewportClients());
		RegisteredViewportClients = PreviousViewportClients.Intersect(NewViewportClients).Array();

		// Unregister ourselves from the viewport-change callback if we don't have any registered 
		// viewports anymore.
		if (RegisteredViewportClients.IsEmpty())
		{
			GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
		}

		// Remove ourselves from the viewport that went away. It's probably not necessary since they
		// will be destroyed, but better be safe.
		TSet<FLevelEditorViewportClient*> RemovedViewportClients = PreviousViewportClients.Difference(NewViewportClients);
		for (FLevelEditorViewportClient* RemovedViewportClient : RemovedViewportClients)
		{
			RemovedViewportClient->ViewModifiers.RemoveAll(this);
		}
	}
}

void FCameraShakePreviewer::AddReferencedObjects(FReferenceCollector& Collector)
{
	for (FPreviewCameraShakeInfo& ActiveShake : ActiveShakes)
	{
		if (ActiveShake.ShakeInstance)
		{
			Collector.AddReferencedObject(ActiveShake.ShakeInstance);
		}
	}
}

void FCameraShakePreviewer::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	const bool bImmediately = true;
	for (int32 i = ActiveShakes.Num() - 1; i >= 0; i--)
	{
		FPreviewCameraShakeInfo& ActiveShake = ActiveShakes[i];
		if (UObject* const * NewShakeInstance = ReplacementMap.Find(ActiveShake.ShakeInstance))
		{
			// If a camera shake gets recompiled, stop the old version, and start the
			// new version with the same parameters.
			ActiveShake.ShakeInstance->StopShake(bImmediately);
			ActiveShake.ShakeInstance->TeardownShake();
			
			ActiveShake.ShakeInstance = Cast<UCameraShakeBase>(*NewShakeInstance);
			if (ensure(ActiveShake.ShakeInstance))
			{
				ActiveShake.ShakeInstance->StartShake(ActiveShake.StartParams);
			}
		}
	}
}

#endif

