// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneEventTemplate.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "MovieSceneSequence.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "EngineGlobals.h"
#include "MovieScene.h"
#include "Evaluation/MovieSceneEvaluation.h"
#include "IMovieScenePlayer.h"
#include "Algo/Accumulate.h"
#include "Engine/LevelScriptActor.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEventTemplate)

#define LOCTEXT_NAMESPACE "MovieSceneEventTemplate"

DECLARE_CYCLE_STAT(TEXT("Event Track Token Execute"), MovieSceneEval_EventTrack_TokenExecute, STATGROUP_MovieSceneEval);

struct FMovieSceneEventData
{
	FMovieSceneEventData(const FEventPayload& InPayload, float InGlobalPosition) : Payload(InPayload), GlobalPosition(InGlobalPosition) {}

	FEventPayload Payload;
	float GlobalPosition;
};

/** A movie scene execution token that stores a specific transform, and an operand */
struct FEventTrackExecutionToken
	: IMovieSceneExecutionToken
{
	FEventTrackExecutionToken(TArray<FMovieSceneEventData> InEvents) : Events(MoveTemp(InEvents)) {}

	/** Execute this token, operating on all objects referenced by 'Operand' */
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_EventTrack_TokenExecute)
		
		TArray<float> PerformanceCaptureEventPositions;

		// Resolve event contexts to trigger the event on
		TArray<UObject*> EventContexts = Player.GetEventContexts();

		for (UObject* EventContextObject : EventContexts)
		{
			if (!EventContextObject)
			{
				continue;
			}

			for (FMovieSceneEventData& Event : Events)
			{
#if !UE_BUILD_SHIPPING
				if (Event.Payload.EventName == NAME_PerformanceCapture)
				{
					PerformanceCaptureEventPositions.Add(Event.GlobalPosition);
				}
#endif
				TriggerEvent(Event, *EventContextObject, Player);
			}
		}

#if !UE_BUILD_SHIPPING
		if (PerformanceCaptureEventPositions.Num())
		{
			UObject* PlaybackContext = Player.GetPlaybackContext();
			UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
			if (World)
			{
				FString LevelSequenceName = Player.GetEvaluationTemplate().GetSequence(MovieSceneSequenceID::Root)->GetName();
			
				for (float EventPosition : PerformanceCaptureEventPositions)
				{
					GEngine->PerformanceCapture(World, World->GetName(), LevelSequenceName, EventPosition);
				}
			}
		}
#endif	// UE_BUILD_SHIPPING
	}

	void TriggerEvent(FMovieSceneEventData& Event, UObject& EventContextObject, IMovieScenePlayer& Player)
	{
		UFunction* EventFunction = EventContextObject.FindFunction(Event.Payload.EventName);

		if (EventFunction == nullptr)
		{
			// Don't want to log out a warning for every event context.
			return;
		}
		else
		{
			FStructOnScope ParameterStruct(nullptr);
			Event.Payload.Parameters.GetInstance(ParameterStruct);

			uint8* Parameters = ParameterStruct.GetStructMemory();

			const UStruct* Struct = ParameterStruct.GetStruct();
			if (EventFunction->ReturnValueOffset != MAX_uint16)
			{
				UE_LOGF(LogMovieScene, Warning, "Sequencer Event Track: Cannot trigger events that return values (for event '%ls').", *Event.Payload.EventName.ToString());
				return;
			}
			else
			{
				TFieldIterator<FProperty> ParamIt(EventFunction);
				TFieldIterator<FProperty> ParamInstanceIt(Struct);
				for (int32 NumParams = 0; ParamIt || ParamInstanceIt; ++NumParams, ++ParamIt, ++ParamInstanceIt)
				{
					if (!ParamInstanceIt)
					{
						UE_LOGF(LogMovieScene, Warning, "Sequencer Event Track: Parameter count mismatch for event '%ls'. Required parameter of type '%ls' at index '%d'.", *Event.Payload.EventName.ToString(), *ParamIt->GetName(), NumParams);
						return;
					}
					else if (!ParamIt)
					{
						// Mismatch (too many params)
						UE_LOGF(LogMovieScene, Warning, "Sequencer Event Track: Parameter count mismatch for event '%ls'. Parameter struct contains too many parameters ('%ls' is superfluous at index '%d'.", *Event.Payload.EventName.ToString(), *ParamInstanceIt->GetName(), NumParams);
						return;
					}
					else if (!ParamInstanceIt->SameType(*ParamIt) || ParamInstanceIt->GetOffset_ForUFunction() != ParamIt->GetOffset_ForUFunction() || ParamInstanceIt->GetSize() != ParamIt->GetSize())
					{
						UE_LOGF(LogMovieScene, Warning, "Sequencer Event Track: Parameter type mismatch for event '%ls' ('%ls' != '%ls').",
							*Event.Payload.EventName.ToString(),
							*ParamInstanceIt->GetClass()->GetName(),
							*ParamIt->GetClass()->GetName()
						);
						return;
					}
				}
			}

			// Technically, anything bound to the event could mutate the parameter payload,
			// but we're going to treat that as misuse, rather than copy the parameters each time
			EventContextObject.ProcessEvent(EventFunction, Parameters);
		}
	}

	TArray<FMovieSceneEventData> Events;
};


FMovieSceneEventSectionTemplate::FMovieSceneEventSectionTemplate(const UMovieSceneEventSection& Section, const UMovieSceneEventTrack& Track)
	: EventData(Section.GetEventData())
	, bFireEventsWhenForwards(Track.bFireEventsWhenForwards)
	, bFireEventsWhenBackwards(Track.bFireEventsWhenBackwards)
{
}

void FMovieSceneEventSectionTemplate::EvaluateSwept(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const TRange<FFrameNumber>& SweptRange, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	// Don't allow events to fire when playback is in a stopped state. This can occur when stopping 
	// playback and returning the current position to the start of playback. It's not desireable to have 
	// all the events from the last playback position to the start of playback be fired.
	if (Context.GetStatus() == EMovieScenePlayerStatus::Stopped || Context.IsSilent())
	{
		return;
	}

	const bool bBackwards = Context.GetDirection() == EPlayDirection::Backwards;

	if ((!bBackwards && !bFireEventsWhenForwards) ||
		(bBackwards && !bFireEventsWhenBackwards))
	{
		return;
	}

	TArray<FMovieSceneEventData> Events;

	TArrayView<const FFrameNumber>  KeyTimes   = EventData.GetKeyTimes();
	TArrayView<const FEventPayload> KeyValues  = EventData.GetKeyValues();

	const int32 First = bBackwards ? KeyTimes.Num() - 1 : 0;
	const int32 Last = bBackwards ? 0 : KeyTimes.Num() - 1;
	const int32 Inc = bBackwards ? -1 : 1;


	// @todo: this is technically incorrect because it is converting a root-space time to seconds using the local-space frame rate,
	//        but this code only exists for legacy reasons so it has not been updated.
	TOptional<FFrameTime> RootTime = Context.GetRootToSequenceTransform().Inverse().TryTransformTime(Context.GetTime());

	const float PositionInSeconds = RootTime.Get(0) / Context.GetFrameRate();

	if (bBackwards)
	{
		// Trigger events backwards
		for (int32 KeyIndex = KeyTimes.Num() - 1; KeyIndex >= 0; --KeyIndex)
		{
			FFrameNumber Time = KeyTimes[KeyIndex];
			if (SweptRange.Contains(Time))
			{
				Events.Add(FMovieSceneEventData(KeyValues[KeyIndex], PositionInSeconds));
			}
		}
	}
	// Trigger events forwards
	else for (int32 KeyIndex = 0; KeyIndex < KeyTimes.Num(); ++KeyIndex)
	{
		FFrameNumber Time = KeyTimes[KeyIndex];
		if (SweptRange.Contains(Time))
		{
			Events.Add(FMovieSceneEventData(KeyValues[KeyIndex], PositionInSeconds));
		}
	}


	if (Events.Num())
	{
		ExecutionTokens.Add(FEventTrackExecutionToken(MoveTemp(Events)));
	}
}

#undef LOCTEXT_NAMESPACE
