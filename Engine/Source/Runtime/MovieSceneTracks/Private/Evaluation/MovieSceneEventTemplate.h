// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sections/MovieSceneEventSection.h"
#include "Sections/MovieSceneEventRepeaterSection.h"
#include "Sections/MovieSceneEventTriggerSection.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "MovieSceneObjectBindingID.h"

#include "MovieSceneEventTemplate.generated.h"

class UMovieSceneEventTrack;
struct EventData;

USTRUCT()
struct FMovieSceneEventSectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()
	
	FMovieSceneEventSectionTemplate() : bFireEventsWhenForwards(false), bFireEventsWhenBackwards(false) {}
	FMovieSceneEventSectionTemplate(const UMovieSceneEventSection& Section, const UMovieSceneEventTrack& Track);

	UPROPERTY()
	FMovieSceneEventSectionData EventData;

	UPROPERTY()
	uint32 bFireEventsWhenForwards : 1;

	UPROPERTY()
	uint32 bFireEventsWhenBackwards : 1;

private:

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void EvaluateSwept(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const TRange<FFrameNumber>& SweptRange, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;
};
