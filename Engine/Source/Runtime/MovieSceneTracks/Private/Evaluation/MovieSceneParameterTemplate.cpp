// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneParameterTemplate.h"
#include "Tracks/MovieSceneMaterialTrack.h"
#include "Components/DecalComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Evaluation/MovieSceneEvaluation.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneParameterTemplate)


FMovieSceneParameterSectionTemplate::FMovieSceneParameterSectionTemplate(const UMovieSceneParameterSection& Section)
	: Scalars(Section.GetScalarParameterNamesAndCurves())
	, Bools(Section.GetBoolParameterNamesAndCurves())
	, Vector2Ds(Section.GetVector2DParameterNamesAndCurves())
	, Vectors(Section.GetVectorParameterNamesAndCurves())
	, Colors(Section.GetColorParameterNamesAndCurves())
	, Transforms(Section.GetTransformParameterNamesAndCurves())
{
}

void FMovieSceneParameterSectionTemplate::EvaluateCurves(const FMovieSceneContext& Context, FEvaluatedParameterSectionValues& Values) const
{
	const FFrameTime Time = Context.GetTime();

	for ( const FScalarParameterNameAndCurve& Scalar : Scalars )
	{
		float Value = 0;
		if (Scalar.ParameterCurve.Evaluate(Time, Value))
		{
			Values.ScalarValues.Emplace(Scalar.ParameterName, Value);
		}
	}

	for (const FBoolParameterNameAndCurve& Bool : Bools)
	{
		bool Value = false;
		if (Bool.ParameterCurve.Evaluate(Time, Value))
		{
			Values.BoolValues.Emplace(Bool.ParameterName, Value);
		}
	}

	for (const FVector2DParameterNameAndCurves& Vector : Vector2Ds)
	{
		FVector2f Value(ForceInitToZero);

		bool bAnyEvaluated = false;
		bAnyEvaluated |= Vector.XCurve.Evaluate(Time, Value.X);
		bAnyEvaluated |= Vector.YCurve.Evaluate(Time, Value.Y);

		if (bAnyEvaluated)
		{
			Values.Vector2DValues.Emplace(Vector.ParameterName, FVector2D(Value));
		}
	}


	for ( const FVectorParameterNameAndCurves& Vector : Vectors )
	{
		FVector3f Value(ForceInitToZero);

		bool bAnyEvaluated = false;
		bAnyEvaluated |= Vector.XCurve.Evaluate(Time, Value.X);
		bAnyEvaluated |= Vector.YCurve.Evaluate(Time, Value.Y);
		bAnyEvaluated |= Vector.ZCurve.Evaluate(Time, Value.Z);

		if (bAnyEvaluated)
		{
			Values.VectorValues.Emplace(Vector.ParameterName, (FVector)Value);
		}
	}

	for ( const FColorParameterNameAndCurves& Color : Colors )
	{
		FLinearColor ColorValue = FLinearColor::White;

		bool bAnyEvaluated = false;
		bAnyEvaluated |= Color.RedCurve.Evaluate(  Time, ColorValue.R);
		bAnyEvaluated |= Color.GreenCurve.Evaluate(Time, ColorValue.G);
		bAnyEvaluated |= Color.BlueCurve.Evaluate( Time, ColorValue.B);
		bAnyEvaluated |= Color.AlphaCurve.Evaluate(Time, ColorValue.A);

		if (bAnyEvaluated)
		{
			Values.ColorValues.Emplace(Color.ParameterName, ColorValue);
		}
	}

	for (const FTransformParameterNameAndCurves& Transform : Transforms)
	{
		FVector3f Translation, Scale(FVector3f::OneVector);
		FRotator3f Rotator;
		bool bAnyEvaluated = false;
		bAnyEvaluated |= Transform.Translation[0].Evaluate(Time,Translation[0]);
		bAnyEvaluated |= Transform.Translation[1].Evaluate(Time, Translation[1]);
		bAnyEvaluated |= Transform.Translation[2].Evaluate(Time, Translation[2]);

		bAnyEvaluated |= Transform.Rotation[0].Evaluate(Time, Rotator.Roll);
		bAnyEvaluated |= Transform.Rotation[1].Evaluate(Time, Rotator.Pitch);
		bAnyEvaluated |= Transform.Rotation[2].Evaluate(Time, Rotator.Yaw);

		//mz todo quat interp...

		bAnyEvaluated |= Transform.Scale[0].Evaluate(Time, Scale[0]);
		bAnyEvaluated |= Transform.Scale[1].Evaluate(Time, Scale[1]);
		bAnyEvaluated |= Transform.Scale[2].Evaluate(Time, Scale[2]);

		if (bAnyEvaluated)
		{
			FTransformParameterNameAndValue NameAndValue(Transform.ParameterName, (FVector)Translation, FRotator(Rotator), (FVector)Scale);
			Values.TransformValues.Emplace(NameAndValue);
		}
	}
}

