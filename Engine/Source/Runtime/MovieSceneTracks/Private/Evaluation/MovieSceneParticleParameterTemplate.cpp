// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneParticleParameterTemplate.h"
#include "Tracks/MovieSceneParticleParameterTrack.h"
#include "Particles/Emitter.h"
#include "Particles/ParticleSystemComponent.h"
#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneParticleParameterTemplate)

DECLARE_CYCLE_STAT(TEXT("Particle Parameter Track Token Execute"), MovieSceneEval_ParticleParameterTrack_TokenExecute, STATGROUP_MovieSceneEval);

template<typename T>
struct TNameAndValue
{
	FName Name;
	T Value;
};

struct FParticleParameterPreAnimatedToken : IMovieScenePreAnimatedToken
{
	FParticleParameterPreAnimatedToken() {}

	FParticleParameterPreAnimatedToken(FParticleParameterPreAnimatedToken&&) = default;
	FParticleParameterPreAnimatedToken& operator=(FParticleParameterPreAnimatedToken&&) = default;

	virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params) override
	{
		UParticleSystemComponent* ParticleSystemComponent = CastChecked<UParticleSystemComponent>(&Object);

		for (TNameAndValue<float>& Value : ScalarValues)
		{
			ParticleSystemComponent->SetFloatParameter(Value.Name, Value.Value);
		}
		
		for (TNameAndValue<FVector>& Value : VectorValues)
		{
			ParticleSystemComponent->SetVectorParameter(Value.Name, Value.Value);
		}

		for (TNameAndValue<FLinearColor>& Value : ColorValues)
		{
			ParticleSystemComponent->SetColorParameter(Value.Name, Value.Value);
		}
	}

	TArray< TNameAndValue<float> > ScalarValues;
	TArray< TNameAndValue<FVector> > VectorValues;
	TArray< TNameAndValue<FLinearColor> > ColorValues;
};



struct FParticleParameterPreAnimatedTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override
	{
		UParticleSystemComponent* ParticleSystemComponent = CastChecked<UParticleSystemComponent>(&Object);

		FParticleParameterPreAnimatedToken Token;

		for (const FParticleSysParam& Param : ParticleSystemComponent->GetAsyncInstanceParameters())
		{
			switch (Param.ParamType)
			{
			case PSPT_Scalar:
				Token.ScalarValues.Add( TNameAndValue<float>{ Param.Name, Param.Scalar } );
				break;
			case PSPT_Vector:
				Token.VectorValues.Add( TNameAndValue<FVector>{ Param.Name, Param.Vector } );
				break;
			case PSPT_Color:
				Token.ColorValues.Add( TNameAndValue<FLinearColor>{ Param.Name, Param.Color } );
				break;
			}
		}

		return MoveTemp(Token);
	}
};

struct FParticleParameterExecutionToken : IMovieSceneExecutionToken
{
	FParticleParameterExecutionToken() = default;

	FParticleParameterExecutionToken(FParticleParameterExecutionToken&&) = default;
	FParticleParameterExecutionToken& operator=(FParticleParameterExecutionToken&&) = default;

	// Non-copyable
	FParticleParameterExecutionToken(const FParticleParameterExecutionToken&) = delete;
	FParticleParameterExecutionToken& operator=(const FParticleParameterExecutionToken&) = delete;

	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player)
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_ParticleParameterTrack_TokenExecute)

		for (TWeakObjectPtr<>& WeakObject : Player.FindBoundObjects(Operand))
		{
			UParticleSystemComponent* ParticleSystemComponent = Cast<UParticleSystemComponent>(WeakObject.Get());
			if (!ParticleSystemComponent)
			{
				AEmitter* Emitter = Cast<AEmitter>(WeakObject.Get());
				ParticleSystemComponent = Emitter ? Emitter->GetParticleSystemComponent() : nullptr;
			}

			if (!ParticleSystemComponent)
			{
				continue;
			}

			Player.SavePreAnimatedState(*ParticleSystemComponent, TMovieSceneAnimTypeID<FParticleParameterExecutionToken>(), FParticleParameterPreAnimatedTokenProducer());

			for ( const FScalarParameterNameAndValue& ScalarNameAndValue : Values.ScalarValues )
			{
				ParticleSystemComponent->SetFloatParameter( ScalarNameAndValue.ParameterName, ScalarNameAndValue.Value );
			}
			for ( const FVectorParameterNameAndValue& VectorNameAndValue : Values.VectorValues )
			{
				ParticleSystemComponent->SetVectorParameter( VectorNameAndValue.ParameterName, VectorNameAndValue.Value );
			}
			for ( const FColorParameterNameAndValue& ColorNameAndValue : Values.ColorValues )
			{
				ParticleSystemComponent->SetColorParameter( ColorNameAndValue.ParameterName, ColorNameAndValue.Value );
			}
		}
	}
	
	FEvaluatedParameterSectionValues Values;
};

FMovieSceneParticleParameterSectionTemplate::FMovieSceneParticleParameterSectionTemplate(const UMovieSceneParameterSection& Section, const UMovieSceneParticleParameterTrack& Track)
	: FMovieSceneParameterSectionTemplate(Section)
{
}

void FMovieSceneParticleParameterSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	FParticleParameterExecutionToken ExecutionToken;

	EvaluateCurves(Context, ExecutionToken.Values);

	ExecutionTokens.Add(MoveTemp(ExecutionToken));
}

