// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Evaluation/MovieSceneParameterTemplate.h"
#include "MovieSceneParticleParameterTemplate.generated.h"

class UMovieSceneParticleParameterTrack;

USTRUCT()
struct FMovieSceneParticleParameterSectionTemplate : public FMovieSceneParameterSectionTemplate
{
	GENERATED_BODY()

	FMovieSceneParticleParameterSectionTemplate() {}
	FMovieSceneParticleParameterSectionTemplate(const UMovieSceneParameterSection& Section, const UMovieSceneParticleParameterTrack& Track);

private:

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;
};
