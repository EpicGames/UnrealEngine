// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneParticleTemplate.h"
#include "Sections/MovieSceneParticleSection.h"
#include "Particles/Emitter.h"
#include "Particles/ParticleSystemComponent.h"
#include "Evaluation/MovieSceneEvaluation.h"
#include "MovieSceneTimeHelpers.h"
#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneParticleTemplate)


DECLARE_CYCLE_STAT(TEXT("Particle Track Token Execute"), MovieSceneEval_ParticleTrack_TokenExecute, STATGROUP_MovieSceneEval);

static UFXSystemComponent* GetFXSystemComponentFromObject(UObject* Object)
{
	if (AEmitter* Emitter = Cast<AEmitter>(Object))
	{
		return Emitter->GetParticleSystemComponent();
	}
	else
	{
		return Cast<UFXSystemComponent>(Object);
	}
}

/** A movie scene pre-animated token that stores a pre-animated active state */
struct FActivePreAnimatedToken : IMovieScenePreAnimatedToken
{
	FActivePreAnimatedToken(UObject& InObject)
	{
		bCurrentlyActive = false;

		if (AEmitter* Emitter = Cast<AEmitter>(&InObject))
		{
			bCurrentlyActive = Emitter->bCurrentlyActive;
		}
		else
		{
			UFXSystemComponent* FXSystemComponent = GetFXSystemComponentFromObject(&InObject);
			if (FXSystemComponent != nullptr)
			{
				bCurrentlyActive = FXSystemComponent->IsActive();
			}
		}
	}

	virtual void RestoreState(UObject& InObject, const UE::MovieScene::FRestoreStateParams& Params) override
	{
		UFXSystemComponent* FXSystemComponent = GetFXSystemComponentFromObject(&InObject);
		if (FXSystemComponent)
		{
			FXSystemComponent->SetActive(bCurrentlyActive, true);
		}
	}

private:
	bool bCurrentlyActive;
};

struct FActiveTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	static FMovieSceneAnimTypeID GetAnimTypeID() 
	{
		return TMovieSceneAnimTypeID<FActiveTokenProducer>();
	}

private:
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override
	{
		return FActivePreAnimatedToken(Object);
	}
};


/** A movie scene execution token that stores a specific transform, and an operand */
struct FParticleTrackExecutionToken
	: IMovieSceneExecutionToken
{
	FParticleTrackExecutionToken(EParticleKey InParticleKey)
		: ParticleKey(InParticleKey)
	{}

	/** Execute this token, operating on all objects referenced by 'Operand' */
	virtual void Execute(const FMovieSceneContext& Context, const FMovieSceneEvaluationOperand& Operand, FPersistentEvaluationData& PersistentData, IMovieScenePlayer& Player) override
	{
		MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_ParticleTrack_TokenExecute)

		for (TWeakObjectPtr<> Object : Player.FindBoundObjects(Operand))
		{
			UObject* ObjectPtr = Object.Get();
			UFXSystemComponent* FXSystemComponent = GetFXSystemComponentFromObject(ObjectPtr);

			if (FXSystemComponent)
			{
				Player.SavePreAnimatedState(*ObjectPtr, FActiveTokenProducer::GetAnimTypeID(), FActiveTokenProducer());

				if ( ParticleKey == EParticleKey::Activate)
				{
					if ( !FXSystemComponent->IsActive() )
					{
						FXSystemComponent->SetActive(true, true);
					}
				}
				else if( ParticleKey == EParticleKey::Deactivate )
				{
					FXSystemComponent->SetActive(false, true);
				}
				else if ( ParticleKey == EParticleKey::Trigger )
				{
					FXSystemComponent->ActivateSystem(true);
				}
			}
		}
	}

	EParticleKey ParticleKey;
	TOptional<FKeyHandle> KeyHandle;
};

FMovieSceneParticleSectionTemplate::FMovieSceneParticleSectionTemplate(const UMovieSceneParticleSection& Section)
	: ParticleKeys(Section.ParticleKeys)
{
}

void FMovieSceneParticleSectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	const bool bPlaying = Context.GetDirection() == EPlayDirection::Forwards && Context.GetRange().Size<FFrameTime>() >= FFrameTime(0);

	if (!bPlaying)
	{
		ExecutionTokens.Add(FParticleTrackExecutionToken(EParticleKey::Deactivate));
	}
	else
	{
		TRange<FFrameNumber> PlaybackRange = Context.GetFrameNumberRange();

		TMovieSceneChannelData<const uint8> ChannelData = ParticleKeys.GetData();

		// Find the index of the key handle that exists before this time
		TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
		TArrayView<const uint8>        Values = ChannelData.GetValues();

		FFrameNumber ExclusiveEndFrame = UE::MovieScene::DiscreteExclusiveUpper(PlaybackRange.GetUpperBound());

		int32 LastKeyIndex = Algo::UpperBound(Times, ExclusiveEndFrame)-1;

		// If Algo::UpperBound returns a value greater or equal to the exclusive upper bound of the range, use the index before it
		if (LastKeyIndex >= 0 && Times[LastKeyIndex] >= ExclusiveEndFrame)
		{
			--LastKeyIndex;
		}

		if (LastKeyIndex >= 0)
		{
			EParticleKey ParticleKey((EParticleKey)Values[LastKeyIndex]);
			
			// If the particle key is a trigger, limit the activation/deactivation to within the playback range
			if (ParticleKey == EParticleKey::Trigger)
			{
				if (PlaybackRange.Contains(Times[LastKeyIndex]))
				{
					FParticleTrackExecutionToken NewToken(ParticleKey);
					ExecutionTokens.Add(MoveTemp(NewToken));
				}
			}
			// Otherwise, this is within the activation/deactivation keyframe range
			else
			{
				FParticleTrackExecutionToken NewToken(ParticleKey);
				ExecutionTokens.Add(MoveTemp(NewToken));
			}
		}
	}
}

