// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sections/MovieSceneParticleSection.h"
#include "Evaluation/MovieSceneEvalTemplate.h"

#include "MovieSceneParticleTemplate.generated.h"

class UMovieSceneParticleSection;

USTRUCT()
struct FMovieSceneParticleSectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()
	
	FMovieSceneParticleSectionTemplate() {}
	FMovieSceneParticleSectionTemplate(const UMovieSceneParticleSection& Section);

	UPROPERTY()
	FMovieSceneParticleChannel ParticleKeys;

private:

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;
};
