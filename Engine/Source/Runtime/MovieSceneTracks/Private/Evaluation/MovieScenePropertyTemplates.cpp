// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieScenePropertyTemplates.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Sections/MovieSceneByteSection.h"
#include "Sections/MovieSceneEnumSection.h"
#include "Sections/MovieSceneIntegerSection.h"
#include "Sections/MovieSceneVectorSection.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieScene.h"
#include "Evaluation/MovieSceneEvaluation.h"
#include "MovieSceneTemplateCommon.h"
#include "Evaluation/Blending/MovieSceneMultiChannelBlending.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePropertyTemplates)

namespace
{
	FName SanitizeBoolPropertyName(FName InPropertyName)
	{
		FString PropertyVarName = InPropertyName.ToString();
		PropertyVarName.RemoveFromStart("b", ESearchCase::CaseSensitive);
		return FName(*PropertyVarName);
	}
}

namespace UE::MovieScene
{
	MOVIESCENETRACKS_API extern bool GBooleanPropertyLegacyTemplate;
}

//	----------------------------------------------------------------------------
//	Boolean Property Template
FMovieSceneBoolPropertySectionTemplate::FMovieSceneBoolPropertySectionTemplate(const UMovieSceneBoolSection& Section, const UMovieScenePropertyTrack& Track)
	: FMovieScenePropertySectionTemplate(Track.GetPropertyName(), Track.GetPropertyPath().ToString())
	, BoolCurve(Section.GetChannel())
{
	PropertyData.PropertyName = SanitizeBoolPropertyName(PropertyData.PropertyName);
}

void FMovieSceneBoolPropertySectionTemplate::Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const
{
	if (!UE::MovieScene::GBooleanPropertyLegacyTemplate)
	{
		return;
	}

	// Only evaluate if the curve has any data
	bool Result = false;
	if (BoolCurve.Evaluate(Context.GetTime(), Result))
	{
		ExecutionTokens.Add(TPropertyTrackExecutionToken<bool>(Result));
	}
}

