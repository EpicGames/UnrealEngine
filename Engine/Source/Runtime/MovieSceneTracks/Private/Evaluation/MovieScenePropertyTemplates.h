// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Evaluation/MovieScenePropertyTemplate.h"
#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "EulerTransform.h"
#include "MovieScenePropertyTemplates.generated.h"

class UMovieSceneBoolSection;
class UMovieSceneByteSection;
class UMovieSceneFloatSection;
class UMovieSceneIntegerSection;
class UMovieScenePropertyTrack;
class UMovieSceneVectorSection;
class UMovieSceneEnumSection;
class UMovieScene3DTransformSection;


USTRUCT()
struct FMovieSceneBoolPropertySectionTemplate : public FMovieScenePropertySectionTemplate
{
	GENERATED_BODY()
	
	FMovieSceneBoolPropertySectionTemplate() {}
	FMovieSceneBoolPropertySectionTemplate(const UMovieSceneBoolSection& Section, const UMovieScenePropertyTrack& Track);

protected:

	virtual UScriptStruct& GetScriptStructImpl() const override { return *StaticStruct(); }
	virtual void SetupOverrides() override { EnableOverrides(RequiresSetupFlag); }
	virtual void Evaluate(const FMovieSceneEvaluationOperand& Operand, const FMovieSceneContext& Context, const FPersistentEvaluationData& PersistentData, FMovieSceneExecutionTokens& ExecutionTokens) const override;

	UPROPERTY()
	FMovieSceneBoolChannel BoolCurve;
};

