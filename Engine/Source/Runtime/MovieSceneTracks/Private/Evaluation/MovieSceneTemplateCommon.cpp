// Copyright Epic Games, Inc. All Rights Reserved.

#include "Evaluation/MovieSceneTemplateCommon.h"
#include "Components/SceneComponent.h"
#include "GameFramework/Actor.h"
#include "Misc/App.h"

/** A movie scene pre-animated token that stores a pre-animated mobility */
struct FMobilityPreAnimatedToken : IMovieScenePreAnimatedToken
{
	FMobilityPreAnimatedToken(USceneComponent& SceneComponent)
	{
		Mobility = SceneComponent.Mobility;
	}

	virtual void RestoreState(UObject& InObject, const UE::MovieScene::FRestoreStateParams& Params) override
	{
		USceneComponent* SceneComponent = CastChecked<USceneComponent>(&InObject);
		SceneComponent->SetMobility(Mobility);
	}

private:
	EComponentMobility::Type Mobility;
};

FMovieSceneAnimTypeID FMobilityTokenProducer::GetAnimTypeID()
{
	return TMovieSceneAnimTypeID<FMobilityTokenProducer>();
}

/** Cache the existing state of an object before moving it */
IMovieScenePreAnimatedTokenPtr FMobilityTokenProducer::CacheExistingState(UObject& Object) const
{
	USceneComponent* SceneComponent = CastChecked<USceneComponent>(&Object);
	return FMobilityPreAnimatedToken(*SceneComponent);
}

void F3DTransformTrackToken::Apply(USceneComponent& SceneComponent) const
{
	/* Cache initial absolute position */
	FVector PreviousPosition = SceneComponent.GetComponentLocation();

	// If this is a simulating component, teleport since sequencer takes over. 
	// Teleport will not have no velocity, but it's computed below by sequencer so that it will be correct for physics.
	AActor* Actor = SceneComponent.GetOwner();
	USceneComponent* RootComponent = Actor ? Actor->GetRootComponent() : nullptr;
	bool bIsSimulatingPhysics = RootComponent ? RootComponent->IsSimulatingPhysics() : false;

	SceneComponent.SetRelativeLocationAndRotation(Translation, Rotation, false, nullptr, bIsSimulatingPhysics ? ETeleportType::ResetPhysics : ETeleportType::None);
	SceneComponent.SetRelativeScale3D(Scale);

	// Force the location and rotation values to avoid Rot->Quat->Rot conversions
	SceneComponent.SetRelativeLocation_Direct(Translation);
	SceneComponent.SetRelativeRotation_Direct(Rotation);

	double DeltaTime = FApp::GetDeltaTime();
	if (DeltaTime > 0)
	{
		/* Get current absolute position and set component velocity */
		FVector CurrentPosition = SceneComponent.GetComponentLocation();
		FVector ComponentVelocity = (CurrentPosition - PreviousPosition) / DeltaTime;
		SceneComponent.ComponentVelocity = ComponentVelocity;
	}
};


/** A movie scene pre-animated token that stores a pre-animated transform */
struct F3DTransformTrackPreAnimatedToken : F3DTransformTrackToken, IMovieScenePreAnimatedToken
{
	F3DTransformTrackPreAnimatedToken(USceneComponent& SceneComponent)
	{
		FTransform ExistingTransform = SceneComponent.GetRelativeTransform();

		Translation = ExistingTransform.GetTranslation();
		Rotation = ExistingTransform.Rotator();
		Scale = ExistingTransform.GetScale3D();
	}

	virtual void RestoreState(UObject& InObject, const UE::MovieScene::FRestoreStateParams& Params) override
	{
		USceneComponent* SceneComponent = CastChecked<USceneComponent>(&InObject);

		Apply(*SceneComponent);
	}
};

FMovieSceneAnimTypeID F3DTransformTokenProducer::GetAnimTypeID()
{
	return TMovieSceneAnimTypeID<F3DTransformTokenProducer>();
}

/** Cache the existing state of an object before moving it */
IMovieScenePreAnimatedTokenPtr F3DTransformTokenProducer::CacheExistingState(UObject& Object) const
{
	USceneComponent* SceneComponent = CastChecked<USceneComponent>(&Object);
	return F3DTransformTrackPreAnimatedToken(*SceneComponent);
}

