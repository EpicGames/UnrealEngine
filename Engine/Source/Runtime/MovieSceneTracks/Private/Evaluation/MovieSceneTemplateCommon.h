// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Evaluation/MovieSceneAnimTypeID.h"
#include "MovieSceneExecutionToken.h"

class USceneComponent;

struct FMobilityTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	MOVIESCENETRACKS_API static FMovieSceneAnimTypeID GetAnimTypeID();

private:
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override;
};

/** A token that sets a component's relative transform */
struct F3DTransformTrackToken
{
	F3DTransformTrackToken() : Translation(FVector::ZeroVector), Rotation(FRotator::ZeroRotator), Scale(FVector::OneVector)
	{
	}

	F3DTransformTrackToken(FVector InTranslation, FRotator InRotation, FVector InScale)
		: Translation(InTranslation), Rotation(InRotation), Scale(InScale)
	{
	}

	FVector Translation;
	FRotator Rotation;
	FVector Scale;

	void Apply(USceneComponent& SceneComponent) const;
};

struct F3DTransformTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	MOVIESCENETRACKS_API static FMovieSceneAnimTypeID GetAnimTypeID();

private:
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override;
};
