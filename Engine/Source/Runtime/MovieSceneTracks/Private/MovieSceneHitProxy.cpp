// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneHitProxy.h"
#include "GenericPlatform/ICursor.h"

IMPLEMENT_HIT_PROXY(HMovieSceneKeyProxy, HHitProxy);

HMovieSceneKeyProxy::HMovieSceneKeyProxy(class UMovieSceneTrack* InTrack, const FTrajectoryKey& InKey)
	: HHitProxy(HPP_UI)
	, MovieSceneTrack(InTrack)
	, Key(InKey)
{}

EMouseCursor::Type HMovieSceneKeyProxy::GetMouseCursor()
{
	return EMouseCursor::Crosshairs;
}
