// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneSkeletalAnimationRootHitProxy.h"
#include "Components/SkeletalMeshComponent.h"
#include "GenericPlatform/ICursor.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"

IMPLEMENT_HIT_PROXY(HMovieSceneSkeletalAnimationRootHitProxy, HHitProxy);

HMovieSceneSkeletalAnimationRootHitProxy::HMovieSceneSkeletalAnimationRootHitProxy(UMovieSceneSkeletalAnimationSection* InSection, USkeletalMeshComponent* InSkelMeshComp)
	: HHitProxy(HPP_UI)
	, AnimSection(InSection)
	, SkelMeshComp(InSkelMeshComp)
{
}

EMouseCursor::Type HMovieSceneSkeletalAnimationRootHitProxy::GetMouseCursor()
{
	return EMouseCursor::CardinalCross;
}
