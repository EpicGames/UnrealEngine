// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTracksComponentTypes.h"
#include "Camera/CameraShakeBase.h"
#include "Camera/CameraShakeSourceComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "MovieSceneTracksCustomAccessors.h"
#include "Tracks/MovieSceneIntegerTrack.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieSceneDoubleTrack.h"
#include "Tracks/MovieSceneColorTrack.h"
#include "Tracks/MovieSceneVectorTrack.h"
#include "Tracks/MovieSceneTransformTrack.h"
#include "Tracks/MovieSceneEulerTransformTrack.h"
#include "Tracks/MovieSceneBoolTrack.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "Channels/MovieSceneUnpackedChannelValues.h"
#include "Systems/MovieScenePiecewiseBoolBlenderSystem.h"
#include "Systems/MovieScenePiecewiseByteBlenderSystem.h"
#include "Systems/MovieScenePiecewiseIntegerBlenderSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "EntitySystem/MovieScenePropertyComponentHandler.h"
#include "EntitySystem/MovieSceneVariantPropertyComponentHandler.h"
#include "EntitySystem/MovieSceneEntityFactoryTemplates.h"
#include "EntitySystem/MovieScenePropertyMetaData.inl"
#include "PreAnimatedState/MovieScenePreAnimatedComponentTransformStorage.h"
#include "Systems/MovieSceneColorPropertySystem.h"
#include "Systems/MovieSceneVectorPropertySystem.h"
#include "MovieSceneObjectBindingID.h"
#include "GameFramework/Actor.h"
#include "Materials/MaterialParameterCollection.h"
#include "Misc/App.h"
#include "PhysicsEngine/BodyInstance.h"
#include "EntitySystem/MovieSceneComponentTypeIDs.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTracksComponentTypes)

namespace UE
{
namespace MovieScene
{
/* ---------------------------------------------------------------------------
 * Integer conversion functions
 * ---------------------------------------------------------------------------*/
void ConvertOperationalProperty(int64 In, int8&  Out)
{
	static constexpr int8 Min = std::numeric_limits<int8>::lowest();
	static constexpr int8 Max = std::numeric_limits<int8>::max();
	if (In > static_cast<int64>(Max))
	{
		Out = Max;
	}
	else if (In < static_cast<int64>(Min))
	{
		Out = Min;
	}
	else
	{
		Out = static_cast<int8>(In);
	}
}
void ConvertOperationalProperty(int64 In, int16& Out)
{
	static constexpr int16 Min = std::numeric_limits<int16>::lowest();
	static constexpr int16 Max = std::numeric_limits<int16>::max();
	if (In > static_cast<int64>(Max))
	{
		Out = Max;
	}
	else if (In < static_cast<int64>(Min))
	{
		Out = Min;
	}
	else
	{
		Out = static_cast<int16>(In);
	}
}
void ConvertOperationalProperty(int64 In, int32& Out)
{
	static constexpr int32 Min = std::numeric_limits<int32>::lowest();
	static constexpr int32 Max = std::numeric_limits<int32>::max();
	if (In > static_cast<int64>(Max))
	{
		Out = Max;
	}
	else if (In < static_cast<int64>(Min))
	{
		Out = Min;
	}
	else
	{
		Out = static_cast<int32>(In);
	}
}

/* ---------------------------------------------------------------------------
 * Transform conversion functions
 * ---------------------------------------------------------------------------*/
void ConvertOperationalProperty(const FIntermediate3DTransform& In, FEulerTransform& Out)
{
	Out.Location = In.GetTranslation();
	Out.Rotation = In.GetRotation();
	Out.Scale = In.GetScale();
}
void ConvertOperationalProperty(const FEulerTransform& In, FIntermediate3DTransform& Out)
{
	Out = FIntermediate3DTransform(In.Location, In.Rotation, In.Scale);
}

void ConvertOperationalProperty(const FIntermediate3DTransform& In, FTransform& Out)
{
	Out = FTransform(In.GetRotation().Quaternion(), In.GetTranslation(), In.GetScale());
}
void ConvertOperationalProperty(const FTransform& In, FIntermediate3DTransform& Out)
{
	FVector Location = In.GetTranslation();
	FRotator Rotation = In.GetRotation().Rotator();
	FVector Scale = In.GetScale3D();

	Out = FIntermediate3DTransform(Location, Rotation, Scale);
}

/* ---------------------------------------------------------------------------
 * Color conversion functions
 * ---------------------------------------------------------------------------*/
void ConvertOperationalProperty(const FIntermediateColor& InColor, FColor& Out)
{
	Out = InColor.GetColor();
}

void ConvertOperationalProperty(const FIntermediateColor& InColor, FLinearColor& Out)
{
	Out = InColor.GetLinearColor();
}

void ConvertOperationalProperty(const FIntermediateColor& InColor, FSlateColor& Out)
{
	Out = InColor.GetSlateColor();
}

void ConvertOperationalProperty(const FColor& InColor, FIntermediateColor& OutIntermediate)
{
	OutIntermediate = FIntermediateColor(InColor);
}

void ConvertOperationalProperty(const FLinearColor& InColor, FIntermediateColor& OutIntermediate)
{
	OutIntermediate = FIntermediateColor(InColor);
}

void ConvertOperationalProperty(const FSlateColor& InColor, FIntermediateColor& OutIntermediate)
{
	OutIntermediate = FIntermediateColor(InColor);
}


/* ---------------------------------------------------------------------------
 * Vector conversion functions
 * ---------------------------------------------------------------------------*/
void ConvertOperationalProperty(const FFloatIntermediateVector& InVector, FVector2f& Out)
{
	Out = FVector2f(InVector.X, InVector.Y);
}

void ConvertOperationalProperty(const FFloatIntermediateVector& InVector, FVector3f& Out)
{
	Out = FVector3f(InVector.X, InVector.Y, InVector.Z);
}

void ConvertOperationalProperty(const FFloatIntermediateVector& InVector, FVector4f& Out)
{
	Out = FVector4f(InVector.X, InVector.Y, InVector.Z, InVector.W);
}

void ConvertOperationalProperty(const FVector2f& In, FFloatIntermediateVector& Out)
{
	Out = FFloatIntermediateVector(In.X, In.Y);
}

void ConvertOperationalProperty(const FVector3f& In, FFloatIntermediateVector& Out)
{
	Out = FFloatIntermediateVector(In.X, In.Y, In.Z);
}

void ConvertOperationalProperty(const FVector4f& In, FFloatIntermediateVector& Out)
{
	Out = FFloatIntermediateVector(In.X, In.Y, In.Z, In.W);
}

void ConvertOperationalProperty(const FDoubleIntermediateVector& InVector, FVector2d& Out)
{
	Out = FVector2d(InVector.X, InVector.Y);
}

void ConvertOperationalProperty(const FDoubleIntermediateVector& InVector, FVector3d& Out)
{
	Out = FVector3d(InVector.X, InVector.Y, InVector.Z);
}

void ConvertOperationalProperty(const FDoubleIntermediateVector& InVector, FVector4d& Out)
{
	Out = FVector4d(InVector.X, InVector.Y, InVector.Z, InVector.W);
}

void ConvertOperationalProperty(const FVector2d& In, FDoubleIntermediateVector& Out)
{
	Out = FDoubleIntermediateVector(In.X, In.Y);
}

void ConvertOperationalProperty(const FVector3d& In, FDoubleIntermediateVector& Out)
{
	Out = FDoubleIntermediateVector(In.X, In.Y, In.Z);
}

void ConvertOperationalProperty(const FVector4d& In, FDoubleIntermediateVector& Out)
{
	Out = FDoubleIntermediateVector(In.X, In.Y, In.Z, In.W);
}

void ConvertOperationalProperty(float In, double& Out)
{
	Out = static_cast<double>(In);
}

void ConvertOperationalProperty(double In, float& Out)
{
	Out = static_cast<float>(In);
}

void ConvertOperationalProperty(const FObjectComponent& In, UObject*& Out)
{
	Out = In.GetObject();
}

void ConvertOperationalProperty(UObject* In, FObjectComponent& Out)
{
	Out = FObjectComponent::Strong(In);
}

uint8 GetSkeletalMeshAnimationMode(const UObject* Object)
{
	const USkeletalMeshComponent* SkeletalMeshComponent = CastChecked<const USkeletalMeshComponent>(Object);
	return SkeletalMeshComponent->GetAnimationMode();
}

void SetSkeletalMeshAnimationMode(UObject* Object, uint8 InAnimationMode)
{
	USkeletalMeshComponent* SkeletalMeshComponent = CastChecked<USkeletalMeshComponent>(Object);
	constexpr bool bForceInitAnimScriptInstance = false; // Avoid reinits each frame if an anim node track is added with AnimBlueprint mode
	SkeletalMeshComponent->SetAnimationMode((EAnimationMode::Type)InAnimationMode, bForceInitAnimScriptInstance);
}

void FIntermediate3DTransform::ApplyTo(USceneComponent* SceneComponent) const
{
	ApplyTransformTo(SceneComponent, *this);
}

void FIntermediate3DTransform::ApplyTransformTo(USceneComponent* SceneComponent, const FIntermediate3DTransform& Transform)
{
	double DeltaTime = FApp::GetDeltaTime();
	if (DeltaTime <= 0)
	{
		SetComponentTransform(SceneComponent, Transform);
	}
	else
	{
		/* Cache initial absolute position */
		FVector PreviousPosition = SceneComponent->GetComponentLocation();

		SetComponentTransform(SceneComponent, Transform);

		/* Get current absolute position and set component velocity */
		FVector CurrentPosition = SceneComponent->GetComponentLocation();
		FVector ComponentVelocity = (CurrentPosition - PreviousPosition) / DeltaTime;
		SceneComponent->ComponentVelocity = ComponentVelocity;
	}
}

void FIntermediate3DTransform::ApplyTranslationAndRotationTo(USceneComponent* SceneComponent, const FIntermediate3DTransform& Transform)
{
	double DeltaTime = FApp::GetDeltaTime();
	if (DeltaTime <= 0)
	{
		SetComponentTranslationAndRotation(SceneComponent, Transform);
	}
	else
	{
		/* Cache initial absolute position */
		FVector PreviousPosition = SceneComponent->GetComponentLocation();

		SetComponentTranslationAndRotation(SceneComponent, Transform);

		/* Get current absolute position and set component velocity */
		FVector CurrentPosition = SceneComponent->GetComponentLocation();
		FVector ComponentVelocity = (CurrentPosition - PreviousPosition) / DeltaTime;
		SceneComponent->ComponentVelocity = ComponentVelocity;
	}
}

USceneComponent* FComponentAttachParamsDestination::ResolveAttachment(AActor* InParentActor) const
{
	if (SocketName != NAME_None)
	{
		if (ComponentName != NAME_None )
		{
			TInlineComponentArray<USceneComponent*> PotentialAttachComponents(InParentActor);
			for (USceneComponent* PotentialAttachComponent : PotentialAttachComponents)
			{
				if (PotentialAttachComponent->GetFName() == ComponentName && PotentialAttachComponent->DoesSocketExist(SocketName))
				{
					return PotentialAttachComponent;
				}
			}
		}
		else if (InParentActor->GetRootComponent()->DoesSocketExist(SocketName))
		{
			return InParentActor->GetRootComponent();
		}
	}
	else if (ComponentName != NAME_None )
	{
		TInlineComponentArray<USceneComponent*> PotentialAttachComponents(InParentActor);
		for (USceneComponent* PotentialAttachComponent : PotentialAttachComponents)
		{
			if (PotentialAttachComponent->GetFName() == ComponentName)
			{
				return PotentialAttachComponent;
			}
		}
	}

	if (InParentActor->GetDefaultAttachComponent())
	{
		return InParentActor->GetDefaultAttachComponent();
	}
	else
	{
		return InParentActor->GetRootComponent();
	}
}

void FComponentAttachParams::ApplyAttach(USceneComponent* ChildComponentToAttach, USceneComponent* NewAttachParent, const FName& SocketName) const
{
	if (ChildComponentToAttach->GetAttachParent() != NewAttachParent || ChildComponentToAttach->GetAttachSocketName() != SocketName)
	{
		// Attachment changes may try to mark a package as dirty but this prevents us from restoring the level to the pre-animated
		// state correctly which causes issues with validation.
		MovieSceneHelpers::FMovieSceneScopedPackageDirtyGuard DirtyFlagGuard(ChildComponentToAttach);

		FAttachmentTransformRules AttachmentRules(AttachmentLocationRule, AttachmentRotationRule, AttachmentScaleRule, false);
		ChildComponentToAttach->AttachToComponent(NewAttachParent, AttachmentRules, SocketName);
	}

	// Match the component velocity of the parent. If the attached child has any transformation, the velocity will be 
	// computed by the component transform system.
	if (ChildComponentToAttach->GetAttachParent())
	{
		ChildComponentToAttach->ComponentVelocity = ChildComponentToAttach->GetAttachParent()->GetComponentVelocity();
	}
}

void FComponentDetachParams::ApplyDetach(USceneComponent* ChildComponentToAttach, USceneComponent* NewAttachParent, const FName& SocketName) const
{
	// Detach if there was no pre-existing parent
	if (!NewAttachParent)
	{
		MovieSceneHelpers::FMovieSceneScopedPackageDirtyGuard DirtyFlagGuard(ChildComponentToAttach);

		FDetachmentTransformRules DetachmentRules(DetachmentLocationRule, DetachmentRotationRule, DetachmentScaleRule, false);
		ChildComponentToAttach->DetachFromComponent(DetachmentRules);
	}
	else
	{
		MovieSceneHelpers::FMovieSceneScopedPackageDirtyGuard DirtyFlagGuard(ChildComponentToAttach);

		ChildComponentToAttach->AttachToComponent(NewAttachParent, FAttachmentTransformRules::KeepRelativeTransform, SocketName);
	}
}

/* ---------------------------------------------------------------------------
 * Actor reference conversion functions
 * ---------------------------------------------------------------------------*/
void ConvertOperationalProperty(const FActorReferenceComponent& InComponentData, TObjectPtr<AActor>& Out)
{
	Out = InComponentData.WeakActor.Get();
}

void ConvertOperationalProperty(const FActorReferenceComponent& InComponentData, TSoftObjectPtr<AActor>& Out)
{
	Out = InComponentData.WeakActor.Get();
}

void ConvertOperationalProperty(TObjectPtr<AActor> const InActor, FActorReferenceComponent& Out)
{
	// This conversion is incomplete: we end up with an FActorReferenceComponent that has a resolved actor,
	// but its object binding ID is left invalid, because we don't know where that actor "came from" in terms
	// of object binding resolution. This is fine because we should only end up here for saving pre-animated
	// state, which means we only care about saving and restoring the actor pointer.
	Out.WeakActor = InActor.Get();
}

void ConvertOperationalProperty(TSoftObjectPtr<AActor> const InActor, FActorReferenceComponent& Out)
{
	// See comment above.
	Out.WeakActor = InActor.Get();
}


static bool GMovieSceneTracksComponentTypesDestroyed = false;
static TUniquePtr<FMovieSceneTracksComponentTypes> GMovieSceneTracksComponentTypes;

struct FFloatHandler : TPropertyComponentHandler<FFloatPropertyTraits, double>
{
};

struct FFloatParameterHandler : TPropertyComponentHandler<FFloatParameterTraits, double>
{
	virtual TSharedPtr<IInitialValueProcessor> MakeInitialValueProcessor(const FPropertyDefinition& Definition) override
	{
		return nullptr;
	}
};

struct FColorParameterHandler : TPropertyComponentHandler<FColorParameterTraits, double, double, double, double>
{
	virtual TSharedPtr<IInitialValueProcessor> MakeInitialValueProcessor(const FPropertyDefinition& Definition) override
	{
		return nullptr;
	}
};

struct FBoolHandler : TPropertyComponentHandler<FBoolPropertyTraits, bool>
{
	virtual void DispatchInitializePropertyMetaDataTasks(const FPropertyDefinition& Definition, FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents, UMovieSceneEntitySystemLinker* Linker) override
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

		FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.Read(BuiltInComponents->PropertyBinding)
		.Write(TrackComponents->Bool.MetaDataComponents.GetType<0>())
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.Iterate_PerEntity(&Linker->EntityManager, [](UObject* Object, const FMovieScenePropertyBinding& Binding, FBoolPropertyTraits::FBoolMetaData& OutMetaData)
		{
			OutMetaData.BitFieldSize = 0;
			OutMetaData.BitIndex     = 0;

			FTrackInstancePropertyBindings Bindings(Binding.PropertyName, Binding.PropertyPath.ToString());
			ensure(Bindings.HasValidBinding(*Object));
			FBoolProperty* BoundProperty = CastField<FBoolProperty>(Bindings.GetProperty(*Object));
			if (BoundProperty && !BoundProperty->IsNativeBool())
			{
				auto FieldMask = BoundProperty->GetFieldMask();
				static_assert(std::is_same_v<decltype(FieldMask), uint8>, "Unexpected size of field mask returned from FBoolProperty::FieldMask");

				OutMetaData.BitFieldSize = static_cast<uint8>(BoundProperty->GetElementSize());
				OutMetaData.BitIndex     = static_cast<uint8>(FMath::CountTrailingZeros(FieldMask));
			}
		});
	}
};

struct FComponentTransformHandler : TPropertyComponentHandler<FComponentTransformPropertyTraits, double, double, double, double, double, double, double, double, double>
{
	TSharedPtr<IPreAnimatedStorage> GetPreAnimatedStateStorage(const FPropertyDefinition& Definition, FPreAnimatedStateExtension* Container) override
	{
		return Container->GetOrCreateStorage<FPreAnimatedComponentTransformStorage>();
	}
	virtual void ScheduleSetterTasks(const FPropertyDefinition& Definition, TArrayView<const FPropertyCompositeDefinition> Composites, const FPropertyStats& Stats, IEntitySystemScheduler* TaskScheduler, UMovieSceneEntitySystemLinker* Linker) override
	{
		ScheduleSetterTasksImpl(Definition, Composites, Stats, TaskScheduler, Linker, FEntityComponentFilter().None({ FMovieSceneTracksComponentTypes::Get()->Tags.AnimMixerPoseProducer }));
	}
};

struct FObjectHandler : TPropertyComponentHandler<FObjectPropertyTraits, FObjectComponent>
{
	virtual void DispatchInitializePropertyMetaDataTasks(const FPropertyDefinition& Definition, FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents, UMovieSceneEntitySystemLinker* Linker) override
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

		FEntityTaskBuilder()
			.Read(BuiltInComponents->BoundObject)
			.Read(BuiltInComponents->PropertyBinding)
			.ReadOptional(BuiltInComponents->CustomPropertyIndex)
			.Write(TrackComponents->Object.MetaDataComponents.GetType<0>())
			.FilterAll({ BuiltInComponents->Tags.NeedsLink })
			.Iterate_PerEntity(&Linker->EntityManager, [TrackComponents](UObject* Object, const FMovieScenePropertyBinding& Binding, const FCustomPropertyIndex* OptionalCustomPropertyIndex, FObjectPropertyTraits::FObjectMetadata& OutMetaData)
			{
				if (OptionalCustomPropertyIndex)
				{
					if (const auto* CustomObjectMetaData = TrackComponents->Accessors.Object.MetaData.Find(OptionalCustomPropertyIndex->Value))
					{
						OutMetaData.ObjectClass = CustomObjectMetaData->AllowedClass.Get();
						OutMetaData.bAllowsClear = CustomObjectMetaData->bAllowsClear;
						return;
					}
				}

				FTrackInstancePropertyBindings Bindings(Binding.PropertyName, Binding.PropertyPath.ToString());
				ensure(Bindings.HasValidBinding(*Object));
				FObjectPropertyBase* BoundProperty = CastField<FObjectPropertyBase>(Bindings.GetProperty(*Object));
				if (BoundProperty)
				{
					OutMetaData.ObjectClass = BoundProperty->PropertyClass;
					OutMetaData.bAllowsClear = !BoundProperty->HasAnyPropertyFlags(CPF_NoClear);
				}
			});
	}
};

struct FActorReferenceHandler : TPropertyComponentHandler<FActorReferencePropertyTraits, FActorReferenceComponent>
{
	virtual void DispatchInitializePropertyMetaDataTasks(const FPropertyDefinition& Definition, FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents, UMovieSceneEntitySystemLinker* Linker) override
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

		FEntityTaskBuilder()
			.Read(BuiltInComponents->BoundObject)
			.Read(BuiltInComponents->PropertyBinding)
			.ReadOptional(BuiltInComponents->CustomPropertyIndex)
			.Write(TrackComponents->ActorReference.MetaDataComponents.GetType<0>())
			.FilterAll({ BuiltInComponents->Tags.NeedsLink })
			.Iterate_PerEntity(&Linker->EntityManager, [TrackComponents](UObject* Object, const FMovieScenePropertyBinding& Binding, const FCustomPropertyIndex* OptionalCustomPropertyIndex, FActorReferencePropertyTraits::FActorReferenceMetaData& OutMetaData)
			{
				if (OptionalCustomPropertyIndex)
				{
					if (const auto* CustomMetaData = TrackComponents->Accessors.ActorReference.MetaData.Find(OptionalCustomPropertyIndex->Value))
					{
						OutMetaData.Type = CustomMetaData->Type;
						return;
					}
				}

				FTrackInstancePropertyBindings Bindings(Binding.PropertyName, Binding.PropertyPath.ToString());
				ensure(Bindings.HasValidBinding(*Object));
				FProperty* BoundProperty = Bindings.GetProperty(*Object);
				if (FObjectProperty* BoundObjectProperty = CastField<FObjectProperty>(BoundProperty))
				{
					OutMetaData.Type = EActorReferenceType::ObjectPointer;
				}
				else if (FSoftObjectProperty* BoundSoftObjectProperty = CastField<FSoftObjectProperty>(BoundProperty))
				{
					OutMetaData.Type = EActorReferenceType::SoftObjectPointer;
				}
				else
				{
					ensureAlwaysMsgf(false, TEXT("Unsupported actor reference type for property '%s' on object '%s'"), *Bindings.GetPropertyPath(), *GetNameSafe(Object));
				}
			});
	}
};

FMovieSceneTracksComponentTypes::FMovieSceneTracksComponentTypes()
{
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FComponentRegistry* ComponentRegistry = UMovieSceneEntitySystemLinker::GetComponents();

	ComponentRegistry->NewPropertyType(Bool, TEXT("bool"));
	ComponentRegistry->NewPropertyType(Byte, TEXT("byte"));
	ComponentRegistry->NewPropertyType(Enum, TEXT("enum"));
	ComponentRegistry->NewPropertyType(Float, TEXT("float"));
	ComponentRegistry->NewPropertyType(Double, TEXT("double"));
	ComponentRegistry->NewPropertyType(Color, TEXT("color"));
	ComponentRegistry->NewPropertyType(Integer, TEXT("int32"));
	ComponentRegistry->NewPropertyType(FloatVector, TEXT("float vector"));
	ComponentRegistry->NewPropertyType(DoubleVector, TEXT("double vector"));
	ComponentRegistry->NewPropertyType(String, TEXT("FString"));
	ComponentRegistry->NewPropertyType(Text, TEXT("FText"));
	ComponentRegistry->NewPropertyType(Object, TEXT("Object"));
	ComponentRegistry->NewPropertyType(ActorReference, TEXT("Actor Reference"));

	ComponentRegistry->NewPropertyType(Transform, TEXT("FTransform"));
	ComponentRegistry->NewPropertyType(EulerTransform, TEXT("FEulerTransform"));
	ComponentRegistry->NewPropertyType(ComponentTransform, TEXT("Component Transform"));

	ComponentRegistry->NewPropertyType(Rotator, TEXT("FRotator"));

	ComponentRegistry->NewPropertyType(FloatParameter, TEXT("float parameter"));
	ComponentRegistry->NewPropertyType(ColorParameter, TEXT("color parameter"));

	ComponentRegistry->NewPropertyType(Parameters.Bool, TEXT("Bool parameter"));
	ComponentRegistry->NewPropertyType(Parameters.Byte, TEXT("Byte parameter"));
	ComponentRegistry->NewPropertyType(Parameters.Integer, TEXT("Integer parameter"));
	ComponentRegistry->NewPropertyType(Parameters.Scalar, TEXT("Scalar parameter"));
	ComponentRegistry->NewPropertyType(Parameters.Vector2, TEXT("Vector2 parameter"));
	ComponentRegistry->NewPropertyType(Parameters.Vector3, TEXT("Vector3 parameter"));
	ComponentRegistry->NewPropertyType(Parameters.Color, TEXT("Color parameter"));
	ComponentRegistry->NewPropertyType(Parameters.Transform, TEXT("Transform parameter"));

	Color.MetaDataComponents.Initialize(BuiltInComponents->VariantPropertyTypeIndex);
	Double.MetaDataComponents.Initialize(BuiltInComponents->VariantPropertyTypeIndex);
	Integer.MetaDataComponents.Initialize(BuiltInComponents->VariantPropertyTypeIndex);
	Transform.MetaDataComponents.Initialize(BuiltInComponents->VariantPropertyTypeIndex);
	FloatVector.MetaDataComponents.Initialize(BuiltInComponents->VariantPropertyTypeIndex);
	DoubleVector.MetaDataComponents.Initialize(BuiltInComponents->VariantPropertyTypeIndex);

	Bool.MetaDataComponents.Initialize(ComponentRegistry, TEXT("Bool Bitfield"));
	Object.MetaDataComponents.Initialize(ComponentRegistry, TEXT("Object Class"));
	ActorReference.MetaDataComponents.Initialize(ComponentRegistry, TEXT("Actor Reference Type"));

	ComponentRegistry->NewComponentType(&QuaternionRotationChannel[0], TEXT("Quaternion Rotation Channel 0"));
	ComponentRegistry->NewComponentType(&QuaternionRotationChannel[1], TEXT("Quaternion Rotation Channel 1"));
	ComponentRegistry->NewComponentType(&QuaternionRotationChannel[2], TEXT("Quaternion Rotation Channel 2"));

	ComponentRegistry->NewComponentType(&RotatorChannel[0], TEXT("Rotator Channel Y (Pitch)"));
	ComponentRegistry->NewComponentType(&RotatorChannel[1], TEXT("Rotator Channel Z (Yaw)"));
	ComponentRegistry->NewComponentType(&RotatorChannel[2], TEXT("Rotator Channel X (Roll)"));

	ComponentRegistry->NewComponentType(&ConstraintChannel, TEXT("Constraint Channel"));

	ComponentRegistry->NewComponentType(&ActorReferenceChannel, TEXT("Actor Reference Channel"));
	ComponentRegistry->NewComponentType(&ActorReferenceResult, TEXT("Actor Reference Result"));

	ComponentRegistry->NewComponentType(&AttachParent, TEXT("Attach Parent"));
	ComponentRegistry->NewComponentType(&AttachComponent, TEXT("Attachment Component"));
	ComponentRegistry->NewComponentType(&AttachParentBinding, TEXT("Attach Parent Binding"));
	ComponentRegistry->NewComponentType(&FloatPerlinNoiseChannel, TEXT("Float Perlin Noise Channel"));
	ComponentRegistry->NewComponentType(&DoublePerlinNoiseChannel, TEXT("Double Perlin Noise Channel"));

	ComponentRegistry->NewComponentType(&SkeletalAnimation, TEXT("Skeletal Animation"));

	ComponentRegistry->NewComponentType(&LevelVisibility, TEXT("Level Visibility"));
	ComponentRegistry->NewComponentType(&DataLayer, TEXT("Data Layer"));

	ComponentRegistry->NewComponentType(&ComponentMaterialInfo,		TEXT("Component Material Info"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&BoundMaterial,				TEXT("Bound Material"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&MPC,						TEXT("Material Parameter Collection"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);

	ComponentRegistry->NewComponentType(&BoolParameterName,      TEXT("Bool Parameter Name"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&ScalarParameterName,    TEXT("Scalar Parameter Name"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&Vector2DParameterName,  TEXT("Vector2D Parameter Name"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&VectorParameterName,    TEXT("Vector Parameter Name"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&ColorParameterName,     TEXT("Color Parameter Name"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&TransformParameterName, TEXT("Transform Parameter Name"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&GenericParameterName,   TEXT("Parameter Name"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);

	ComponentRegistry->NewComponentType(&ScalarMaterialParameterInfo, TEXT("Scalar Material Parameter Info"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&ColorMaterialParameterInfo, TEXT("Color Material Parameter Info"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);
	ComponentRegistry->NewComponentType(&VectorMaterialParameterInfo, TEXT("Vector Material Parameter Info"), EComponentTypeFlags::CopyToChildren | EComponentTypeFlags::CopyToOutput);

	ComponentRegistry->NewComponentType(&Fade,                   TEXT("Fade"), EComponentTypeFlags::CopyToChildren);

	ComponentRegistry->NewComponentType(&PropertyNotify,         TEXT("PropertyNotify"), EComponentTypeFlags::CopyToChildren);

	ComponentRegistry->NewComponentType(&Audio,                  TEXT("Audio"), EComponentTypeFlags::CopyToChildren);
	ComponentRegistry->NewComponentType(&AudioInputs,            TEXT("Audio Inputs"), EComponentTypeFlags::CopyToChildren);
	ComponentRegistry->NewComponentType(&AudioTriggerName,       TEXT("Audio Trigger Name"), EComponentTypeFlags::CopyToChildren);

	ComponentRegistry->NewComponentType(&CameraShake,            TEXT("Camera Shake"), EComponentTypeFlags::CopyToChildren);
	ComponentRegistry->NewComponentType(&CameraShakeInstance,    TEXT("Camera Shake Instance"), EComponentTypeFlags::Preserved);

	Tags.BoundMaterialChanged = ComponentRegistry->NewTag(TEXT("Bound Material Changed"));
	FBuiltInComponentTypes::Get()->RequiresInstantiationMask.Set(Tags.BoundMaterialChanged);

	Tags.Slomo = ComponentRegistry->NewTag(TEXT("Slomo"));
	ComponentRegistry->Factories.DefineChildComponent(Tags.Slomo, Tags.Slomo);

	Tags.Visibility = ComponentRegistry->NewTag(TEXT("Visibility"));
	ComponentRegistry->Factories.DefineChildComponent(Tags.Visibility, Tags.Visibility);

	// Used to indicate the ParameterName component for certain parameter types (scalar, vector2d, vector, color)
	// should be interpreted as an index for custom primitive data.
	Tags.CustomPrimitiveData = ComponentRegistry->NewTag(TEXT("Custom Primitive Data"));

	Tags.AnimMixerPoseProducer = ComponentRegistry->NewTag(TEXT("Anim Mixer Pose Producer"));

	// --------------------------------------------------------------------------------------------
	// Set up bool properties
	BuiltInComponents->PropertyRegistry.DefineProperty(Bool, TEXT("Apply bool Properties"))
	.AddSoleChannel(BuiltInComponents->BoolResult)
	.SetBlenderSystem<UMovieScenePiecewiseBoolBlenderSystem>()
	.SetDefaultTrackType(UMovieSceneBoolTrack::StaticClass())
	.SetCustomAccessors(&Accessors.Bool)
	.Commit(FBoolHandler());

	// --------------------------------------------------------------------------------------------
	// Set up FTransform properties
	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(Transform, TEXT("Apply Transform Properties"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FIntermediate3DTransform::T_X)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FIntermediate3DTransform::T_Y)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FIntermediate3DTransform::T_Z)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FIntermediate3DTransform::R_X)
	.AddComposite(BuiltInComponents->DoubleResult[4], &FIntermediate3DTransform::R_Y)
	.AddComposite(BuiltInComponents->DoubleResult[5], &FIntermediate3DTransform::R_Z)
	.AddComposite(BuiltInComponents->DoubleResult[6], &FIntermediate3DTransform::S_X)
	.AddComposite(BuiltInComponents->DoubleResult[7], &FIntermediate3DTransform::S_Y)
	.AddComposite(BuiltInComponents->DoubleResult[8], &FIntermediate3DTransform::S_Z)
	.SetCustomAccessors(&Accessors.Transform)
	.SetDefaultTrackType(UMovieSceneTransformTrack::StaticClass())
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up FRotator properties
	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(Rotator, TEXT("Apply FRotator Properties"))
		.AddComposite(BuiltInComponents->DoubleResult[0], &FRotator::Pitch)
		.AddComposite(BuiltInComponents->DoubleResult[1], &FRotator::Yaw)
		.AddComposite(BuiltInComponents->DoubleResult[2], &FRotator::Roll)
		// Use this blender since we want over rotation 720deg = 2 cycles instead of 720deg = 0, we could later add support for multiple interpolation
		.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
		.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up byte properties
	BuiltInComponents->PropertyRegistry.DefineProperty(Byte, TEXT("Apply Byte Properties"))
	.AddSoleChannel(BuiltInComponents->ByteResult)
	.SetBlenderSystem<UMovieScenePiecewiseByteBlenderSystem>()
	.SetCustomAccessors(&Accessors.Byte)
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up enum properties
	// No blender system specified, enums aren't supposed blend
	BuiltInComponents->PropertyRegistry.DefineProperty(Enum, TEXT("Apply Enum Properties"))
	.AddSoleChannel(BuiltInComponents->ByteResult)
	.SetCustomAccessors(&Accessors.Enum)
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up integer properties
	BuiltInComponents->PropertyRegistry.DefineProperty(Integer, TEXT("Apply Integer Properties"))
	.AddSoleChannel(BuiltInComponents->IntegerResult)
	.SetDefaultTrackType(UMovieSceneIntegerTrack::StaticClass())
	.SetBlenderSystem<UMovieScenePiecewiseIntegerBlenderSystem>()
	.SetCustomAccessors(&Accessors.Integer)
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up float properties
	BuiltInComponents->PropertyRegistry.DefineProperty(Float, TEXT("Apply float Properties"))
	.AddSoleChannel(BuiltInComponents->DoubleResult[0])
	.SetDefaultTrackType(UMovieSceneFloatTrack::StaticClass())
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.SetCustomAccessors(&Accessors.Float)
	.Commit(FFloatHandler());

	// --------------------------------------------------------------------------------------------
	// Set up double properties
	BuiltInComponents->PropertyRegistry.DefineProperty(Double, TEXT("Apply Double Properties"))
	.AddSoleChannel(BuiltInComponents->DoubleResult[0])
	.SetDefaultTrackType(UMovieSceneDoubleTrack::StaticClass())
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.SetCustomAccessors(&Accessors.Double)
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up color properties
	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(Color, TEXT("Apply Color Properties"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FIntermediateColor::R)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FIntermediateColor::G)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FIntermediateColor::B)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FIntermediateColor::A)
	.SetDefaultTrackType(UMovieSceneColorTrack::StaticClass())
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.SetCustomAccessors(&Accessors.Color)
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up string properties
	BuiltInComponents->PropertyRegistry.DefineProperty(String, TEXT("Apply String Properties"))
	.AddSoleChannel(BuiltInComponents->StringResult)
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up text properties
	BuiltInComponents->PropertyRegistry.DefineProperty(Text, TEXT("Apply Text Properties"))
	.AddSoleChannel(BuiltInComponents->TextResult)
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up Object properties
	BuiltInComponents->PropertyRegistry.DefineProperty(Object, TEXT("Apply Object Properties"))
	.AddSoleChannel(BuiltInComponents->ObjectResult)
	.SetCustomAccessors(&Accessors.Object)
	.Commit(FObjectHandler());

	// --------------------------------------------------------------------------------------------
	// Set up Actor Reference properties
	BuiltInComponents->PropertyRegistry.DefineProperty(ActorReference, TEXT("Apply Actor Reference Properties"))
	.AddSoleChannel(ActorReferenceResult)
	.SetCustomAccessors(&Accessors.ActorReference)
	.Commit(FActorReferenceHandler());

	// --------------------------------------------------------------------------------------------
	// Set up float parameters
	BuiltInComponents->PropertyRegistry.DefineProperty(FloatParameter, TEXT("Apply Float Parameters"))
	.AddSoleChannel(BuiltInComponents->DoubleResult[0])
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.Commit(FFloatParameterHandler());

	BuiltInComponents->PropertyRegistry.DefineProperty(Parameters.Scalar, TEXT("Scalar Parameters"))
	.AddSoleChannel(BuiltInComponents->DoubleResult[0])
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up color parameters
	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(ColorParameter, TEXT("Apply Color Parameters"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FIntermediateColor::R)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FIntermediateColor::G)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FIntermediateColor::B)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FIntermediateColor::A)
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.Commit(FColorParameterHandler());

	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(Parameters.Color, TEXT("Color Parameters"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FIntermediateColor::R)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FIntermediateColor::G)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FIntermediateColor::B)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FIntermediateColor::A)
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.Commit();

	Accessors.Byte.Add(
		USkeletalMeshComponent::StaticClass(), USkeletalMeshComponent::GetAnimationModePropertyNameChecked(),
		GetSkeletalMeshAnimationMode, SetSkeletalMeshAnimationMode);
	Accessors.Enum.Add(
		USkeletalMeshComponent::StaticClass(), USkeletalMeshComponent::GetAnimationModePropertyNameChecked(),
		GetSkeletalMeshAnimationMode, SetSkeletalMeshAnimationMode);

	// --------------------------------------------------------------------------------------------
	// Set up vector properties
	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(FloatVector, TEXT("Apply FloatVector Properties"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FFloatIntermediateVector::X)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FFloatIntermediateVector::Y)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FFloatIntermediateVector::Z)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FFloatIntermediateVector::W)
	.SetDefaultTrackType(UMovieSceneFloatVectorTrack::StaticClass())
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.SetCustomAccessors(&Accessors.FloatVector)
	.Commit();

	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(Parameters.Vector2, TEXT("Vector2 Parameters"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FFloatIntermediateVector::X)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FFloatIntermediateVector::Y)
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.Commit();

	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(Parameters.Vector3, TEXT("Vector3 Parameters"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FFloatIntermediateVector::X)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FFloatIntermediateVector::Y)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FFloatIntermediateVector::Z)
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.Commit();

	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(DoubleVector, TEXT("Apply DoubleVector Properties"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FDoubleIntermediateVector::X)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FDoubleIntermediateVector::Y)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FDoubleIntermediateVector::Z)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FDoubleIntermediateVector::W)
	.SetDefaultTrackType(UMovieSceneDoubleVectorTrack::StaticClass())
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.SetCustomAccessors(&Accessors.DoubleVector)
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up FEulerTransform properties
	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(EulerTransform, TEXT("Apply FEulerTransform Properties"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FIntermediate3DTransform::T_X)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FIntermediate3DTransform::T_Y)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FIntermediate3DTransform::T_Z)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FIntermediate3DTransform::R_X)
	.AddComposite(BuiltInComponents->DoubleResult[4], &FIntermediate3DTransform::R_Y)
	.AddComposite(BuiltInComponents->DoubleResult[5], &FIntermediate3DTransform::R_Z)
	.AddComposite(BuiltInComponents->DoubleResult[6], &FIntermediate3DTransform::S_X)
	.AddComposite(BuiltInComponents->DoubleResult[7], &FIntermediate3DTransform::S_Y)
	.AddComposite(BuiltInComponents->DoubleResult[8], &FIntermediate3DTransform::S_Z)
	.SetDefaultTrackType(UMovieSceneEulerTransformTrack::StaticClass())
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.Commit();

	BuiltInComponents->PropertyRegistry.DefineCompositeProperty(Parameters.Transform, TEXT("Transform Parameters"))
	.AddComposite(BuiltInComponents->DoubleResult[0], &FIntermediate3DTransform::T_X)
	.AddComposite(BuiltInComponents->DoubleResult[1], &FIntermediate3DTransform::T_Y)
	.AddComposite(BuiltInComponents->DoubleResult[2], &FIntermediate3DTransform::T_Z)
	.AddComposite(BuiltInComponents->DoubleResult[3], &FIntermediate3DTransform::R_X)
	.AddComposite(BuiltInComponents->DoubleResult[4], &FIntermediate3DTransform::R_Y)
	.AddComposite(BuiltInComponents->DoubleResult[5], &FIntermediate3DTransform::R_Z)
	.AddComposite(BuiltInComponents->DoubleResult[6], &FIntermediate3DTransform::S_X)
	.AddComposite(BuiltInComponents->DoubleResult[7], &FIntermediate3DTransform::S_Y)
	.AddComposite(BuiltInComponents->DoubleResult[8], &FIntermediate3DTransform::S_Z)
	.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
	.Commit();

	// --------------------------------------------------------------------------------------------
	// Set up component transforms
	{
		BuiltInComponents->PropertyRegistry.DefineCompositeProperty(ComponentTransform, TEXT("Call USceneComponent::SetRelativeTransform"))
		.AddComposite(BuiltInComponents->DoubleResult[0], &FIntermediate3DTransform::T_X)
		.AddComposite(BuiltInComponents->DoubleResult[1], &FIntermediate3DTransform::T_Y)
		.AddComposite(BuiltInComponents->DoubleResult[2], &FIntermediate3DTransform::T_Z)
		.AddComposite(BuiltInComponents->DoubleResult[3], &FIntermediate3DTransform::R_X)
		.AddComposite(BuiltInComponents->DoubleResult[4], &FIntermediate3DTransform::R_Y)
		.AddComposite(BuiltInComponents->DoubleResult[5], &FIntermediate3DTransform::R_Z)
		.AddComposite(BuiltInComponents->DoubleResult[6], &FIntermediate3DTransform::S_X)
		.AddComposite(BuiltInComponents->DoubleResult[7], &FIntermediate3DTransform::S_Y)
		.AddComposite(BuiltInComponents->DoubleResult[8], &FIntermediate3DTransform::S_Z)
		.SetBlenderSystem<UMovieScenePiecewiseDoubleBlenderSystem>()
		.SetCustomAccessors(&Accessors.ComponentTransform)
		.Commit(FComponentTransformHandler());
	}

	// --------------------------------------------------------------------------------------------
	// Set up quaternion rotation components
	for (int32 Index = 0; Index < UE_ARRAY_COUNT(QuaternionRotationChannel); ++Index)
	{
		ComponentRegistry->Factories.DuplicateChildComponent(QuaternionRotationChannel[Index]);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(QuaternionRotationChannel[Index], BuiltInComponents->DoubleResult[Index + 3]);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(QuaternionRotationChannel[Index], BuiltInComponents->EvalTime);
	}

	// --------------------------------------------------------------------------------------------
	// Set up rotator components
	for (int32 Index = 0; Index < UE_ARRAY_COUNT(RotatorChannel); ++Index)
	{
		ComponentRegistry->Factories.DuplicateChildComponent(RotatorChannel[Index]);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(RotatorChannel[Index], BuiltInComponents->DoubleResult[Index]);
		ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(RotatorChannel[Index], BuiltInComponents->EvalTime);
	}

	// -------------------------------------------------------------------------------------------
	// Set up constraint components
	ComponentRegistry->Factories.DuplicateChildComponent(ConstraintChannel);
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(ConstraintChannel, BuiltInComponents->EvalTime);

	// --------------------------------------------------------------------------------------------
	// Set up attachment components
	ComponentRegistry->Factories.DefineChildComponent(AttachParentBinding, AttachParent);

	ComponentRegistry->Factories.DuplicateChildComponent(AttachParentBinding);
	ComponentRegistry->Factories.DuplicateChildComponent(AttachComponent);

	// --------------------------------------------------------------------------------------------
	// Set up PerlinNoise components
	ComponentRegistry->Factories.DuplicateChildComponent(FloatPerlinNoiseChannel);
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(FloatPerlinNoiseChannel, BuiltInComponents->EvalSeconds);

	ComponentRegistry->Factories.DuplicateChildComponent(DoublePerlinNoiseChannel);
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(DoublePerlinNoiseChannel, BuiltInComponents->EvalSeconds);

	// --------------------------------------------------------------------------------------------
	// Set up SkeletalAnimation components
	ComponentRegistry->Factories.DuplicateChildComponent(SkeletalAnimation);
	ComponentRegistry->Factories.DefineChildComponent(Tags.AnimMixerPoseProducer, Tags.AnimMixerPoseProducer);

	// --------------------------------------------------------------------------------------------
	// Set up custom primitive data components
	ComponentRegistry->Factories.DefineChildComponent(Tags.CustomPrimitiveData, Tags.CustomPrimitiveData);

	// --------------------------------------------------------------------------------------------
	// Set up camera shake components
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(CameraShake, CameraShakeInstance);

	// --------------------------------------------------------------------------------------------
	// Set up actor reference components
	ComponentRegistry->Factories.DuplicateChildComponent(ActorReferenceChannel);
	ComponentRegistry->Factories.DuplicateChildComponent(ActorReferenceResult);
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(ActorReferenceChannel, ActorReferenceResult);
	ComponentRegistry->Factories.DefineMutuallyInclusiveComponent(ActorReferenceChannel, BuiltInComponents->EvalTime);

	InitializeMovieSceneTracksAccessors(this);
}

FMovieSceneTracksComponentTypes::~FMovieSceneTracksComponentTypes()
{
}

void FMovieSceneTracksComponentTypes::Destroy()
{
	GMovieSceneTracksComponentTypes.Reset();
	GMovieSceneTracksComponentTypesDestroyed = true;
}

FMovieSceneTracksComponentTypes* FMovieSceneTracksComponentTypes::Get()
{
	if (!GMovieSceneTracksComponentTypes.IsValid())
	{
		check(!GMovieSceneTracksComponentTypesDestroyed);
		GMovieSceneTracksComponentTypes.Reset(new FMovieSceneTracksComponentTypes);
	}
	return GMovieSceneTracksComponentTypes.Get();
}

} // namespace MovieScene
} // namespace UE

FPerlinNoiseParams::FPerlinNoiseParams()
	: Frequency(4.0f)
	, Amplitude(1.0f)
	, Offset(0)
{
}

FPerlinNoiseParams::FPerlinNoiseParams(float InFrequency, double InAmplitude)
	: Frequency(InFrequency)
	, Amplitude(InAmplitude)
	, Offset(0)
{
}

void FPerlinNoiseParams::RandomizeOffset(float InMaxOffset)
{
	Offset = FMath::FRand() * InMaxOffset;
}

