// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTracksCustomAccessors.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Components/DirectionalLightComponent.h"
#include "Components/ExponentialHeightFogComponent.h"
#include "Components/LightComponent.h"
#include "Components/LocalFogVolumeComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/SkyAtmosphereComponent.h"
#include "Components/SkyLightComponent.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/EngineTypes.h"
#include "Engine/SkeletalMesh.h"

namespace UE::MovieScene
{

FIntermediate3DTransform GetComponentTransform(const UObject* Object)
{
	const USceneComponent* SceneComponent = CastChecked<const USceneComponent>(Object);
	FIntermediate3DTransform Result(SceneComponent->GetRelativeLocation(), SceneComponent->GetRelativeRotation(), SceneComponent->GetRelativeScale3D());
	return Result;
}

void SetComponentTranslationAndRotation(USceneComponent* SceneComponent, const FIntermediate3DTransform& Transform)
{
	// If this is a simulating component, teleport since sequencer takes over. 
	// Teleport will not have no velocity, but it's computed later by sequencer so that it will be correct for physics.
	// @todo: We would really rather not 
	AActor* Actor = SceneComponent->GetOwner();
	USceneComponent* RootComponent = Actor ? Actor->GetRootComponent() : nullptr;
	bool bIsSimulatingPhysics = RootComponent ? RootComponent->IsSimulatingPhysics() : false;

	FVector Translation = Transform.GetTranslation();
	FRotator Rotation = Transform.GetRotation();
	SceneComponent->SetRelativeLocationAndRotation(Translation, Rotation, false, nullptr, bIsSimulatingPhysics ? ETeleportType::ResetPhysics : ETeleportType::None);

	// Force the location and rotation values to avoid Rot->Quat->Rot conversions
	SceneComponent->SetRelativeLocation_Direct(Translation);
	SceneComponent->SetRelativeRotation_Direct(Rotation);
}

void SetComponentTransform(USceneComponent* SceneComponent, const FIntermediate3DTransform& Transform)
{
	// If this is a simulating component, teleport since sequencer takes over. 
	// Teleport will not have no velocity, but it's computed later by sequencer so that it will be correct for physics.
	// @todo: We would really rather not 
	AActor* Actor = SceneComponent->GetOwner();
	USceneComponent* RootComponent = Actor ? Actor->GetRootComponent() : nullptr;
	const bool bIsSimulatingPhysics = RootComponent ? RootComponent->IsSimulatingPhysics() : false;

	FVector Translation = Transform.GetTranslation();
	FRotator Rotation = Transform.GetRotation();
	FTransform NewTransform(Rotation, Translation, Transform.GetScale());
	if (!NewTransform.IsValid())
	{
#if !NO_LOGGING
		if (UE_LOG_ACTIVE(LogMovieScene, Warning))
		{
			UE_LOGF(LogMovieScene, Warning, "SetComponentTransform: Trying to set new transform with bad data: %ls", *NewTransform.ToString());
		}
#endif
		return;
	}

	const bool bFlipped = FMath::Sign(NewTransform.GetDeterminant()) != FMath::Sign(SceneComponent->GetComponentTransform().GetDeterminant());

	SceneComponent->SetRelativeTransform(NewTransform, false, nullptr, bIsSimulatingPhysics ? ETeleportType::ResetPhysics : ETeleportType::None);

	// Force the location and rotation values to avoid Rot->Quat->Rot conversions
	SceneComponent->SetRelativeLocation_Direct(Translation);
	SceneComponent->SetRelativeRotation_Direct(Rotation);

	// The cached mesh draw command includes the culling order. If the scale sign changes, the render state needs 
	// to be marked dirty so that the culling order is refreshed (the renderer does not currently do this automatically)
	if (bFlipped)
	{
		SceneComponent->MarkRenderStateDirty();
	}
}

void SetComponentTransformAndVelocity(UObject* Object, const FIntermediate3DTransform& InTransform)
{
	UE::MovieScene::FIntermediate3DTransform::ApplyTransformTo(CastChecked<USceneComponent>(Object), InTransform);
}

FIntermediateColor GetLightComponentLightColor(const UObject* Object)
{
	const ULightComponent* LightComponent = CastChecked<const ULightComponent>(Object);
	return FIntermediateColor(LightComponent->GetLightColor());
}

void SetLightComponentLightColor(UObject* Object, const FIntermediateColor& InColor)
{
	// This is a little esoteric - ULightComponentBase::LightColor is the UPROPERTY that generates the meta-data
	// for this custom callback, but it is an FColor, even though the public get/set functions expose it as an
	// FLinearColor. FIntermediateColor is always blended and dealt with in linear space, so it's fine to 
	// simply reinterpret the color

	const bool bConvertBackToSRgb = true;
	ULightComponent* LightComponent = CastChecked<ULightComponent>(Object);
	LightComponent->SetLightColor(InColor.GetLinearColor(), bConvertBackToSRgb);
}

float GetLightComponentIntensity(const UObject* Object)
{
	const ULightComponent* LightComponent = CastChecked<const ULightComponent>(Object);
	return LightComponent->Intensity;
}

void SetLightComponentIntensity(UObject* Object, float InIntensity)
{
	ULightComponent* LightComponent = CastChecked<ULightComponent>(Object);
	LightComponent->SetIntensity(InIntensity);
}

float GetLightComponentVolumetricScatteringIntensity(const UObject* Object)
{
	const ULightComponent* LightComponent = CastChecked<const ULightComponent>(Object);
	return LightComponent->VolumetricScatteringIntensity;
}

void SetLightComponentVolumetricScatteringIntensity(UObject* Object, float InVolumetricScatteringIntensity)
{
	ULightComponent* LightComponent = CastChecked<ULightComponent>(Object);
	LightComponent->SetVolumetricScatteringIntensity(InVolumetricScatteringIntensity);
}

float GetLightComponentIndirectLightingIntensity(const UObject* Object)
{
	const ULightComponent* LightComponent = CastChecked<const ULightComponent>(Object);
	return LightComponent->IndirectLightingIntensity;
}

void SetLightComponentIndirectLightingIntensity(UObject* Object, float InIndirectLightingIntensity)
{
	ULightComponent* LightComponent = CastChecked<ULightComponent>(Object);
	LightComponent->SetIndirectLightingIntensity(InIndirectLightingIntensity);
}

float GetDirectionalLightComponentLightSourceAngle(const UObject* Object)
{
	const UDirectionalLightComponent* DirectionalLightComponent = CastChecked<UDirectionalLightComponent>(Object);
	return DirectionalLightComponent->LightSourceAngle;
}

void SetDirectionalLightComponentLightSourceAngle(UObject* Object, float InLightSourceAngle)
{
	UDirectionalLightComponent* DirectionalLightComponent = CastChecked<UDirectionalLightComponent>(Object);
	DirectionalLightComponent->SetLightSourceAngle(InLightSourceAngle);
}

FIntermediateColor GetSkyLightComponentLightColor(const UObject* Object)
{
	const USkyLightComponent* SkyLightComponent = CastChecked<const USkyLightComponent>(Object);
	return FIntermediateColor(SkyLightComponent->GetLightColor());
}

void SetSkyLightComponentLightColor(UObject* Object, const FIntermediateColor& InColor)
{
	// This is a little esoteric - ULightComponentBase::LightColor is the UPROPERTY that generates the meta-data
	// for this custom callback, but it is an FColor, even though the public get/set functions expose it as an
	// FLinearColor. FIntermediateColor is always blended and dealt with in linear space, so it's fine to 
	// simply reinterpret the color
	USkyLightComponent* SkyLightComponent = CastChecked<USkyLightComponent>(Object);
	SkyLightComponent->SetLightColor(InColor.GetLinearColor());
}

float GetSkyLightComponentIntensity(const UObject* Object)
{
	const USkyLightComponent* SkyLightComponent = CastChecked<const USkyLightComponent>(Object);
	return SkyLightComponent->Intensity;
}

void SetSkyLightComponentIntensity(UObject* Object, float InIntensity)
{
	USkyLightComponent* SkyLightComponent = CastChecked<USkyLightComponent>(Object);
	SkyLightComponent->SetIntensity(InIntensity);
}

bool GetSkyLightComponentRealtimeCapture(const UObject* Object, FBoolPropertyTraits::FBoolMetaData InMetadata)
{
	const USkyLightComponent* SkyLightComponent = CastChecked<const USkyLightComponent>(Object);
	return SkyLightComponent->bRealTimeCapture;
}

void SetSkyLightComponentRealtimeCapture(UObject* Object, FBoolPropertyTraits::FBoolMetaData InMetadata, bool bInRealtimeCapture)
{
	USkyLightComponent* SkyLightComponent = CastChecked<USkyLightComponent>(Object);
	SkyLightComponent->SetRealTimeCapture(bInRealtimeCapture);
}

float GetSkyAtmosphereComponentMieScatteringScale(const UObject* Object)
{
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return SkyAtmosphereComponent->MieScatteringScale;
}

void SetSkyAtmosphereComponentMieScatteringScale(UObject* Object, float InMieScatteringScale)
{
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetMieScatteringScale(InMieScatteringScale);
}

float GetSkyAtmosphereComponentMieExponentialDistribution(const UObject* Object)
{
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return SkyAtmosphereComponent->MieExponentialDistribution;
}

void SetSkyAtmosphereComponentMieExponentialDistribution(UObject* Object, float InMieExponentialDistribution)
{
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetMieExponentialDistribution(InMieExponentialDistribution);
}

float GetSkyAtmosphereComponentOtherAbsorptionScale(const UObject* Object)
{
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return SkyAtmosphereComponent->OtherAbsorptionScale;
}

void SetSkyAtmosphereComponentOtherAbsorptionScale(UObject* Object, float InOtherAbsorptionScale)
{
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetOtherAbsorptionScale(InOtherAbsorptionScale);
}

FIntermediateColor GetSkyAtmosphereComponentOtherAbsorption(const UObject* Object)
{
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return FIntermediateColor(SkyAtmosphereComponent->OtherAbsorption);
}

void SetSkyAtmosphereComponentOtherAbsorption(UObject* Object, const FIntermediateColor& InColor)
{
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetOtherAbsorption(InColor.GetLinearColor());
}

float GetSkyAtmosphereComponentRayleighScatteringScale(const UObject* Object)
{
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return SkyAtmosphereComponent->RayleighScatteringScale;
}

void SetSkyAtmosphereComponentRayleighScatteringScale(UObject* Object, float InRayleighScatteringScale)
{
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetRayleighScatteringScale(InRayleighScatteringScale);
}

float GetSkyAtmosphereComponentRayleighExponentialDistribution(const UObject* Object)
{
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return SkyAtmosphereComponent->RayleighExponentialDistribution;
}

void SetSkyAtmosphereComponentRayleighExponentialDistribution(UObject* Object, float InRayleighExponentialDistribution)
{
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetRayleighExponentialDistribution(InRayleighExponentialDistribution);
}

float GetSkyAtmosphereComponentAerialPerspectiveViewDistanceScale(const UObject* Object)
{
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return SkyAtmosphereComponent->AerialPespectiveViewDistanceScale;
}

void SetSkyAtmosphereComponentAerialPerspectiveViewDistanceScale(UObject* Object, float InAerialPerspectiveViewDistanceScale)
{
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetAerialPespectiveViewDistanceScale(InAerialPerspectiveViewDistanceScale);
}

FIntermediateColor GetSkyAtmosphereComponentMieAbsorption(const UObject* Object)
{
	// USkyAtmosphereComponent::MieAbsorption is a FLinearColor. FIntermediateColor is always blended and dealt with
	// in linear space so it should be safe to reinterpret it as linear.
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return FIntermediateColor(SkyAtmosphereComponent->MieAbsorption);
}

void SetSkyAtmosphereComponentMieAbsorption(UObject* Object, const FIntermediateColor& InColor)
{
	// USkyAtmosphereComponent::MieAbsorption is a FLinearColor. FIntermediateColor is always blended and dealt with
	// in linear space so it should be safe to reinterpret it as linear.
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetMieAbsorption(InColor.GetLinearColor());
}

FIntermediateColor GetSkyAtmosphereComponentMieScattering(const UObject* Object)
{
	// USkyAtmosphereComponent::MieScattering is a FLinearColor. FIntermediateColor is always blended and dealt with
	// in linear space so it should be safe to reinterpret it as linear.
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return FIntermediateColor(SkyAtmosphereComponent->MieScattering);
}

void SetSkyAtmosphereComponentMieScattering(UObject* Object, const FIntermediateColor& InColor)
{
	// USkyAtmosphereComponent::MieScattering is a FLinearColor. FIntermediateColor is always blended and dealt with
	// in linear space so it should be safe to reinterpret it as linear.
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetMieScattering(InColor.GetLinearColor());
}

FIntermediateColor GetSkyAtmosphereComponentRayleighScattering(const UObject* Object)
{
	// USkyAtmosphereComponent::RayleighScattering is a FLinearColor. FIntermediateColor is always blended and dealt with
	// in linear space so it should be safe to reinterpret it as linear.
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return FIntermediateColor(SkyAtmosphereComponent->RayleighScattering);
}

void SetSkyAtmosphereComponentRayleighScattering(UObject* Object, const FIntermediateColor& InColor)
{
	// USkyAtmosphereComponent::RayleighScattering is a FLinearColor. FIntermediateColor is always blended and dealt with
	// in linear space so it should be safe to reinterpret it as linear.
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetRayleighScattering(InColor.GetLinearColor());
}

FIntermediateColor GetSkyAtmosphereComponentSkyLuminanceFactor(const UObject* Object)
{
	// USkyAtmosphereComponent::SkyLuminanceFactor is a FLinearColor. FIntermediateColor is always blended and dealt with
	// in linear space so it should be safe to reinterpret it as linear.
	const USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<const USkyAtmosphereComponent>(Object);
	return FIntermediateColor(SkyAtmosphereComponent->SkyLuminanceFactor);
}

void SetSkyAtmosphereComponentSkyLuminanceFactor(UObject* Object, const FIntermediateColor& InColor)
{
	// USkyAtmosphereComponent::SkyLuminanceFactor is a FLinearColor. FIntermediateColor is always blended and dealt with
	// in linear space so it should be safe to reinterpret it as linear.
	USkyAtmosphereComponent* SkyAtmosphereComponent = CastChecked<USkyAtmosphereComponent>(Object);
	SkyAtmosphereComponent->SetSkyLuminanceFactor(InColor.GetLinearColor());
}

float GetMassScale(const UObject* Object)
{
	const UPrimitiveComponent* PrimitiveComponent = CastChecked<const UPrimitiveComponent>(Object);
	return PrimitiveComponent->GetMassScale();
}

void SetMassScale(UObject* Object, float InMassScale)
{
	UPrimitiveComponent* PrimitiveComponent = CastChecked<UPrimitiveComponent>(Object);
	PrimitiveComponent->SetMassScale(NAME_None, InMassScale);
}

bool GetLightingChannel0(const UObject* Object, FBoolPropertyTraits::FBoolMetaData InMetadata)
{
	if (Object->IsA<ULightComponent>())
	{
		const ULightComponent* LightComponent = CastChecked<const ULightComponent>(Object);
		return LightComponent->LightingChannels.bChannel0;
	}
	else if (Object->IsA<UPrimitiveComponent>())
	{
		const UPrimitiveComponent* PrimitiveComponent = CastChecked<const UPrimitiveComponent>(Object);
		return PrimitiveComponent->LightingChannels.bChannel0;
	}
	ensureMsgf(false, TEXT("GetLightingChannel0 with unknown object %s"), *GetNameSafe(Object));
	return false;
}

void SetLightingChannel0(UObject* Object, FBoolPropertyTraits::FBoolMetaData InMetadata, bool bInLightingChannel0)
{
	if (Object->IsA<ULightComponent>())
	{
		ULightComponent* LightComponent = CastChecked<ULightComponent>(Object);
		LightComponent->SetLightingChannels(bInLightingChannel0, LightComponent->LightingChannels.bChannel1, LightComponent->LightingChannels.bChannel2);
		return;
	}
	else if (Object->IsA<UPrimitiveComponent>())
	{
		UPrimitiveComponent* PrimitiveComponent = CastChecked<UPrimitiveComponent>(Object);
		PrimitiveComponent->SetLightingChannels(bInLightingChannel0, PrimitiveComponent->LightingChannels.bChannel1, PrimitiveComponent->LightingChannels.bChannel2);
		return;
	}
	ensureMsgf(false, TEXT("SetLightingChannel0 with unknown object %s"), *GetNameSafe(Object));
}

bool GetLightingChannel1(const UObject* Object, FBoolPropertyTraits::FBoolMetaData InMetadata)
{
	if (Object->IsA<ULightComponent>())
	{
		const ULightComponent* LightComponent = CastChecked<const ULightComponent>(Object);
		return LightComponent->LightingChannels.bChannel1;
	}
	else if (Object->IsA<UPrimitiveComponent>())
	{
		const UPrimitiveComponent* PrimitiveComponent = CastChecked<const UPrimitiveComponent>(Object);
		return PrimitiveComponent->LightingChannels.bChannel1;
	}
	ensureMsgf(false, TEXT("GetLightingChannel1 with unknown object %s"), *GetNameSafe(Object));
	return false;
}

void SetLightingChannel1(UObject* Object, FBoolPropertyTraits::FBoolMetaData InMetadata, bool bInLightingChannel1)
{
	if (Object->IsA<ULightComponent>())
	{
		ULightComponent* LightComponent = CastChecked<ULightComponent>(Object);
		LightComponent->SetLightingChannels(LightComponent->LightingChannels.bChannel0, bInLightingChannel1, LightComponent->LightingChannels.bChannel2);
		return;
	}
	else if (Object->IsA<UPrimitiveComponent>())
	{
		UPrimitiveComponent* PrimitiveComponent = CastChecked<UPrimitiveComponent>(Object);
		PrimitiveComponent->SetLightingChannels(PrimitiveComponent->LightingChannels.bChannel0, bInLightingChannel1, PrimitiveComponent->LightingChannels.bChannel2);
		return;
	}
	ensureMsgf(false, TEXT("SetLightingChannel1 with unknown object %s"), *GetNameSafe(Object));
}

bool GetLightingChannel2(const UObject* Object, FBoolPropertyTraits::FBoolMetaData InMetadata)
{
	if (Object->IsA<ULightComponent>())
	{
		const ULightComponent* LightComponent = CastChecked<const ULightComponent>(Object);
		return LightComponent->LightingChannels.bChannel2;
	}
	else if (Object->IsA<UPrimitiveComponent>())
	{
		const UPrimitiveComponent* PrimitiveComponent = CastChecked<const UPrimitiveComponent>(Object);
		return PrimitiveComponent->LightingChannels.bChannel2;
	}
	ensureMsgf(false, TEXT("GetLightingChannel2 with unknown object %s"), *GetNameSafe(Object));
	return false;
}

void SetLightingChannel2(UObject* Object, FBoolPropertyTraits::FBoolMetaData InMetadata, bool bInLightingChannel2)
{
	if (Object->IsA<ULightComponent>())
	{
		ULightComponent* LightComponent = CastChecked<ULightComponent>(Object);
		LightComponent->SetLightingChannels(LightComponent->LightingChannels.bChannel0, LightComponent->LightingChannels.bChannel1, bInLightingChannel2);
		return;
	}
	else if (Object->IsA<UPrimitiveComponent>())
	{
		UPrimitiveComponent* PrimitiveComponent = CastChecked<UPrimitiveComponent>(Object);
		PrimitiveComponent->SetLightingChannels(PrimitiveComponent->LightingChannels.bChannel0, PrimitiveComponent->LightingChannels.bChannel1, bInLightingChannel2);
		return;
	}
	ensureMsgf(false, TEXT("SetLightingChannel2 with unknown object %s"), *GetNameSafe(Object));
}

float GetExponentialHeightFogComponentFogDensity(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->FogDensity; 
}

void SetExponentialHeightFogComponentFogDensity(UObject* Object, float InFogDensity)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetFogDensity(InFogDensity);
}

float GetExponentialHeightFogComponentFogHeightFalloff(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->FogHeightFalloff;
}

void SetExponentialHeightFogComponentFogHeightFalloff(UObject* Object, float InFogHeightFalloff)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetFogHeightFalloff(InFogHeightFalloff);
}

float GetExponentialHeightFogComponentFogMaxOpacity(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->FogMaxOpacity;
}

void SetExponentialHeightFogComponentFogMaxOpacity(UObject* Object, float InFogMaxOpacity)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetFogMaxOpacity(InFogMaxOpacity);
}

float GetExponentialHeightFogComponentDirectionalInscatteringExponent(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->DirectionalInscatteringExponent;
}

void SetExponentialHeightFogComponentDirectionalInscatteringExponent(UObject* Object, float InDirectionalInscatteringExponent)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetDirectionalInscatteringExponent(InDirectionalInscatteringExponent);
}

float GetExponentialHeightFogComponentStartDistance(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->StartDistance;
}

void SetExponentialHeightFogComponentStartDistance(UObject* Object, float InStartDistance)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetStartDistance(InStartDistance);
}

float GetExponentialHeightFogComponentVolumetricFogExtinctionScale(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->VolumetricFogExtinctionScale;
}

void SetExponentialHeightFogComponentVolumetricFogExtinctionScale(UObject* Object, float InVolumetricFogExtinctionScale)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetVolumetricFogExtinctionScale(InVolumetricFogExtinctionScale);
}

float GetExponentialHeightFogComponentVolumetricFogStartDistance(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->VolumetricFogStartDistance;
}

void SetExponentialHeightFogComponentVolumetricFogStartDistance(UObject* Object, float InVolumetricFogStartDistance)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetVolumetricFogStartDistance(InVolumetricFogStartDistance);
}

float GetExponentialHeightFogComponentVolumetricFogNearFadeInDistance(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->VolumetricFogNearFadeInDistance;
}

void SetExponentialHeightFogComponentVolumetricFogNearFadeInDistance(UObject* Object, float InVolumetricFogNearFadeInDistance)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetVolumetricFogNearFadeInDistance(InVolumetricFogNearFadeInDistance);
}
	
FIntermediateColor GetExponentialHeightFogComponentDirectionalInscatteringColor(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return FIntermediateColor(ExponentialHeightFogComponent->DirectionalInscatteringLuminance);
}

void SetExponentialHeightFogComponentDirectionalInscatteringColor(UObject* Object, const FIntermediateColor& InColor)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetDirectionalInscatteringColor(InColor.GetLinearColor());
}

FIntermediateColor GetExponentialHeightFogComponentFogInscatteringColor(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return FIntermediateColor(ExponentialHeightFogComponent->FogInscatteringLuminance);
}

void SetExponentialHeightFogComponentFogInscatteringColor(UObject* Object, const FIntermediateColor& InColor)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetFogInscatteringColor(InColor.GetLinearColor());
}

FIntermediateColor GetExponentialHeightFogVolumetricFogAlbedo(const UObject* Object)
{
	// UExponentialHeightFog::VolumetricFogAlbedo is an FColor. FIntermediateColor is always blended and dealt with
	// in linear space, so reinterpret the color.
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return FIntermediateColor(ExponentialHeightFogComponent->VolumetricFogAlbedo);
}

void SetExponentialHeightFogVolumetricFogAlbedo(UObject* Object, const FIntermediateColor& InColor)
{
	// UExponentialHeightFog::VolumetricFogAlbedo is an FColor. FIntermediateColor is always blended and dealt with
	// in linear space, so reinterpret the color back to FColor when setting via the public interface.
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetVolumetricFogAlbedo(InColor.GetColor());
}

float GetSecondFogDataFogDensity(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->SecondFogData.FogDensity;
}

void SetSecondFogDataFogDensity(UObject* Object, float InFogDensity)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetSecondFogDensity(InFogDensity);
}

float GetSecondFogDataFogHeightFalloff(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->SecondFogData.FogHeightFalloff;
}

void SetSecondFogDataFogHeightFalloff(UObject* Object, float InFogHeightFalloff)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetSecondFogHeightFalloff(InFogHeightFalloff);
}

float GetSecondFogDataFogHeightOffset(const UObject* Object)
{
	const UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<const UExponentialHeightFogComponent>(Object);
	return ExponentialHeightFogComponent->SecondFogData.FogHeightOffset;
}

void SetSecondFogDataFogHeightOffset(UObject* Object, float InFogHeightOffset)
{
	UExponentialHeightFogComponent* ExponentialHeightFogComponent = CastChecked<UExponentialHeightFogComponent>(Object);
	ExponentialHeightFogComponent->SetSecondFogHeightOffset(InFogHeightOffset);
}

float GetLocalFogVolumeComponentRadialFogExtinction(const UObject* Object)
{
	const ULocalFogVolumeComponent* LocalFogVolumeComponent = CastChecked<ULocalFogVolumeComponent>(Object);
	return LocalFogVolumeComponent->RadialFogExtinction;
}

void SetLocalFogVolumeComponentRadialFogExtinction(UObject* Object, float InRadialFogExtinction)
{
	ULocalFogVolumeComponent* LocalFogVolumeComponent = CastChecked<ULocalFogVolumeComponent>(Object);
	LocalFogVolumeComponent->SetRadialFogExtinction(InRadialFogExtinction);
}

FIntermediateColor GetLocalFogVolumeComponentFogAlbedo(const UObject* Object)
{
	const ULocalFogVolumeComponent* LocalFogVolumeComponent = CastChecked<ULocalFogVolumeComponent>(Object);
	return FIntermediateColor(LocalFogVolumeComponent->FogAlbedo);
}

void SetLocalFogVolumeComponentFogAlbedo(UObject* Object, const FIntermediateColor& InColor)
{
	ULocalFogVolumeComponent* LocalFogVolumeComponent = CastChecked<ULocalFogVolumeComponent>(Object);
	LocalFogVolumeComponent->SetFogAlbedo(InColor.GetLinearColor());
}

FObjectComponent GetSkeletalMeshAsset(const UObject* Object, FObjectPropertyTraits::FObjectMetadata ObjectMetadata)
{
	const USkeletalMeshComponent* SkeletalMeshComponent = CastChecked<const USkeletalMeshComponent>(Object);
	return FObjectComponent::Strong(SkeletalMeshComponent->GetSkeletalMeshAsset());
}

void SetSkeletalMeshAsset(UObject* Object, FObjectPropertyTraits::FObjectMetadata ObjectMetadata, const FObjectComponent& InSkeletalMeshAsset)
{
	USkeletalMeshComponent* SkeletalMeshComponent = CastChecked<USkeletalMeshComponent>(Object);
	SkeletalMeshComponent->SetSkeletalMeshAsset(Cast<USkeletalMesh>(InSkeletalMeshAsset.GetObject()));
}

FObjectComponent GetSkeletalMesh_DEPRECATED(const UObject* Object, FObjectPropertyTraits::FObjectMetadata ObjectMetadata)
{
	const USkinnedMeshComponent* SkinnedMeshComponent = CastChecked<const USkinnedMeshComponent>(Object);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return FObjectComponent::Strong(SkinnedMeshComponent->GetSkeletalMesh_DEPRECATED());
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void SetSkeletalMesh_DEPRECATED(UObject* Object, FObjectPropertyTraits::FObjectMetadata ObjectMetadata, const FObjectComponent& InSkeletalMeshAsset)
{
	USkinnedMeshComponent* SkinnedMeshComponent = CastChecked<USkinnedMeshComponent>(Object);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	SkinnedMeshComponent->SetSkeletalMesh_DEPRECATED(Cast<USkeletalMesh>(InSkeletalMeshAsset.GetObject()));
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void InitializeMovieSceneTracksAccessors(FMovieSceneTracksComponentTypes* TracksComponents)
{
	// We have some custom accessors for well-known types.

	// LightComponent
	TracksComponents->Accessors.Color.Add(
			ULightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(ULightComponent, LightColor), 
			GetLightComponentLightColor, SetLightComponentLightColor);

	TracksComponents->Accessors.Float.Add(
			ULightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(ULightComponent, Intensity),
			GetLightComponentIntensity, SetLightComponentIntensity);

	TracksComponents->Accessors.Float.Add(
			ULightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(ULightComponent, VolumetricScatteringIntensity),
			GetLightComponentVolumetricScatteringIntensity, SetLightComponentVolumetricScatteringIntensity);

	TracksComponents->Accessors.Float.Add(
			ULightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(ULightComponent, IndirectLightingIntensity),
			GetLightComponentIndirectLightingIntensity, SetLightComponentIndirectLightingIntensity);

	const FString LightingChannel0Path = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightingChannels), GET_MEMBER_NAME_STRING_CHECKED(FLightingChannels, bChannel0));
	TracksComponents->Accessors.Bool.Add(
		ULightComponent::StaticClass(), *LightingChannel0Path,
		GetLightingChannel0, SetLightingChannel0);

	const FString LightingChannel1Path = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightingChannels), GET_MEMBER_NAME_STRING_CHECKED(FLightingChannels, bChannel1));
	TracksComponents->Accessors.Bool.Add(
		ULightComponent::StaticClass(), *LightingChannel1Path,
		GetLightingChannel1, SetLightingChannel1);

	const FString LightingChannel2Path = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(ULightComponent, LightingChannels), GET_MEMBER_NAME_STRING_CHECKED(FLightingChannels, bChannel2));
	TracksComponents->Accessors.Bool.Add(
		ULightComponent::StaticClass(), *LightingChannel2Path,
		GetLightingChannel2, SetLightingChannel2);

	// DirectionalLightComponent
	TracksComponents->Accessors.Float.Add(
			UDirectionalLightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UDirectionalLightComponent, LightSourceAngle),
			GetDirectionalLightComponentLightSourceAngle, SetDirectionalLightComponentLightSourceAngle);

	// SkyLightComponent
	TracksComponents->Accessors.Color.Add(
			USkyLightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyLightComponent, LightColor), 
			GetSkyLightComponentLightColor, SetSkyLightComponentLightColor);
	TracksComponents->Accessors.Float.Add(
			USkyLightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyLightComponent, Intensity),
			GetSkyLightComponentIntensity, SetSkyLightComponentIntensity);
	TracksComponents->Accessors.Bool.Add(
			USkyLightComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyLightComponent, bRealTimeCapture),
			GetSkyLightComponentRealtimeCapture, SetSkyLightComponentRealtimeCapture);

	// SkyAtmosphereComponent
	TracksComponents->Accessors.Float.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, MieScatteringScale),
			GetSkyAtmosphereComponentMieScatteringScale, SetSkyAtmosphereComponentMieScatteringScale);
	TracksComponents->Accessors.Float.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, MieExponentialDistribution),
			GetSkyAtmosphereComponentMieExponentialDistribution, SetSkyAtmosphereComponentMieExponentialDistribution);
	TracksComponents->Accessors.Float.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, OtherAbsorptionScale),
			GetSkyAtmosphereComponentOtherAbsorptionScale, SetSkyAtmosphereComponentOtherAbsorptionScale);
	TracksComponents->Accessors.Float.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, RayleighExponentialDistribution),
			GetSkyAtmosphereComponentRayleighExponentialDistribution, SetSkyAtmosphereComponentRayleighExponentialDistribution);
	TracksComponents->Accessors.Float.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, RayleighScatteringScale),
			GetSkyAtmosphereComponentRayleighScatteringScale, SetSkyAtmosphereComponentRayleighScatteringScale);
	TracksComponents->Accessors.Float.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, AerialPespectiveViewDistanceScale),
			GetSkyAtmosphereComponentAerialPerspectiveViewDistanceScale, SetSkyAtmosphereComponentAerialPerspectiveViewDistanceScale);
	TracksComponents->Accessors.Color.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, MieAbsorption),
			GetSkyAtmosphereComponentMieAbsorption, SetSkyAtmosphereComponentMieAbsorption);
	TracksComponents->Accessors.Color.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, MieScattering),
			GetSkyAtmosphereComponentMieScattering, SetSkyAtmosphereComponentMieScattering);
	TracksComponents->Accessors.Color.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, OtherAbsorption),
			GetSkyAtmosphereComponentOtherAbsorption, SetSkyAtmosphereComponentOtherAbsorption);
	TracksComponents->Accessors.Color.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, RayleighScattering),
			GetSkyAtmosphereComponentRayleighScattering, SetSkyAtmosphereComponentRayleighScattering);
	TracksComponents->Accessors.Color.Add(
			USkyAtmosphereComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(USkyAtmosphereComponent, SkyLuminanceFactor),
			GetSkyAtmosphereComponentSkyLuminanceFactor, SetSkyAtmosphereComponentSkyLuminanceFactor);

	// PrimitiveComponent
	const FString MassScalePath = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, BodyInstance), GET_MEMBER_NAME_STRING_CHECKED(FBodyInstance, MassScale));
	TracksComponents->Accessors.Float.Add(
		UPrimitiveComponent::StaticClass(), *MassScalePath,
		GetMassScale, SetMassScale);

	const FString PrimitiveLightingChannel0Path = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, LightingChannels), GET_MEMBER_NAME_STRING_CHECKED(FLightingChannels, bChannel0));
	TracksComponents->Accessors.Bool.Add(
		UPrimitiveComponent::StaticClass(), *PrimitiveLightingChannel0Path,
		GetLightingChannel0, SetLightingChannel0);

	const FString PrimitiveLightingChannel1Path = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, LightingChannels), GET_MEMBER_NAME_STRING_CHECKED(FLightingChannels, bChannel1));
	TracksComponents->Accessors.Bool.Add(
		UPrimitiveComponent::StaticClass(), *PrimitiveLightingChannel1Path,
		GetLightingChannel1, SetLightingChannel1);

	const FString PrimitiveLightingChannel2Path = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UPrimitiveComponent, LightingChannels), GET_MEMBER_NAME_STRING_CHECKED(FLightingChannels, bChannel2));
	TracksComponents->Accessors.Bool.Add(
		UPrimitiveComponent::StaticClass(), *PrimitiveLightingChannel2Path,
		GetLightingChannel2, SetLightingChannel2);

	// ExponentialHeightFogComponent
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, FogDensity),
			GetExponentialHeightFogComponentFogDensity, SetExponentialHeightFogComponentFogDensity);
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, FogHeightFalloff),
			GetExponentialHeightFogComponentFogHeightFalloff, SetExponentialHeightFogComponentFogHeightFalloff);
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, DirectionalInscatteringExponent),
			GetExponentialHeightFogComponentDirectionalInscatteringExponent, SetExponentialHeightFogComponentDirectionalInscatteringExponent);
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, StartDistance),
			GetExponentialHeightFogComponentStartDistance, SetExponentialHeightFogComponentStartDistance);
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, VolumetricFogStartDistance),
			GetExponentialHeightFogComponentVolumetricFogStartDistance, SetExponentialHeightFogComponentVolumetricFogStartDistance);
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, VolumetricFogNearFadeInDistance),
			GetExponentialHeightFogComponentVolumetricFogNearFadeInDistance, SetExponentialHeightFogComponentVolumetricFogNearFadeInDistance);
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, VolumetricFogExtinctionScale),
			GetExponentialHeightFogComponentVolumetricFogExtinctionScale, SetExponentialHeightFogComponentVolumetricFogExtinctionScale);
	TracksComponents->Accessors.Color.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, DirectionalInscatteringLuminance),
			GetExponentialHeightFogComponentDirectionalInscatteringColor, SetExponentialHeightFogComponentDirectionalInscatteringColor);
	TracksComponents->Accessors.Color.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, FogInscatteringLuminance),
			GetExponentialHeightFogComponentFogInscatteringColor, SetExponentialHeightFogComponentFogInscatteringColor);
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, FogMaxOpacity),
			GetExponentialHeightFogComponentFogMaxOpacity, SetExponentialHeightFogComponentFogMaxOpacity);
	TracksComponents->Accessors.Color.Add(
			UExponentialHeightFogComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(UExponentialHeightFogComponent, VolumetricFogAlbedo),
			GetExponentialHeightFogVolumetricFogAlbedo, SetExponentialHeightFogVolumetricFogAlbedo);
	const FString SecondFogDataFogDensityPath = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, SecondFogData), GET_MEMBER_NAME_STRING_CHECKED(FExponentialHeightFogData, FogDensity));
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), *SecondFogDataFogDensityPath,
			GetSecondFogDataFogDensity, SetSecondFogDataFogDensity);
	const FString SecondFogDataFogHeightFalloffPath = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, SecondFogData), GET_MEMBER_NAME_STRING_CHECKED(FExponentialHeightFogData, FogHeightFalloff));
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), *SecondFogDataFogHeightFalloffPath,
			GetSecondFogDataFogHeightFalloff, SetSecondFogDataFogHeightFalloff);
	const FString SecondFogDataFogHeightOffsetPath = FString::Printf(TEXT("%s.%s"), GET_MEMBER_NAME_STRING_CHECKED(UExponentialHeightFogComponent, SecondFogData), GET_MEMBER_NAME_STRING_CHECKED(FExponentialHeightFogData, FogHeightOffset));
	TracksComponents->Accessors.Float.Add(
			UExponentialHeightFogComponent::StaticClass(), *SecondFogDataFogHeightOffsetPath,
			GetSecondFogDataFogHeightOffset, SetSecondFogDataFogHeightOffset);

	// LocalFogVolumeComponent
	TracksComponents->Accessors.Float.Add(
			ULocalFogVolumeComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(ULocalFogVolumeComponent, RadialFogExtinction),
			GetLocalFogVolumeComponentRadialFogExtinction, SetLocalFogVolumeComponentRadialFogExtinction);
	TracksComponents->Accessors.Color.Add(
			ULocalFogVolumeComponent::StaticClass(), GET_MEMBER_NAME_CHECKED(ULocalFogVolumeComponent, FogAlbedo),
			GetLocalFogVolumeComponentFogAlbedo, SetLocalFogVolumeComponentFogAlbedo);

	// SceneComponent
	TracksComponents->Accessors.ComponentTransform.Add(USceneComponent::StaticClass(), "Transform", &GetComponentTransform, &SetComponentTransformAndVelocity);

	// SkeletalMeshComponent
	TracksComponents->Accessors.Object.Add(USkeletalMeshComponent::StaticClass(), "SkeletalMeshAsset", &GetSkeletalMeshAsset, &SetSkeletalMeshAsset, { USkeletalMeshComponent::StaticClass(), true });

	//SkinnedMeshComponent
	TracksComponents->Accessors.Object.Add(USkinnedMeshComponent::StaticClass(), "SkeletalMesh", &GetSkeletalMesh_DEPRECATED, &SetSkeletalMesh_DEPRECATED, { USkinnedMeshComponent::StaticClass() , true});

	
}

} // namespace UE::MovieScene