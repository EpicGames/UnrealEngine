// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

class UObject;
class USceneComponent;

namespace UE::MovieScene
{

struct FMovieSceneTracksComponentTypes;
struct FIntermediate3DTransform;

void InitializeMovieSceneTracksAccessors(FMovieSceneTracksComponentTypes* TracksComponents);

MOVIESCENETRACKS_API FIntermediate3DTransform GetComponentTransform(const UObject* Object);
MOVIESCENETRACKS_API void SetComponentTranslationAndRotation(USceneComponent* SceneComponent, const FIntermediate3DTransform& Transform);
MOVIESCENETRACKS_API void SetComponentTransform(USceneComponent* SceneComponent, const FIntermediate3DTransform& Transform);
MOVIESCENETRACKS_API void SetComponentTransformAndVelocity(UObject* Object, const FIntermediate3DTransform& InTransform);


} // namespace UE::MovieScene