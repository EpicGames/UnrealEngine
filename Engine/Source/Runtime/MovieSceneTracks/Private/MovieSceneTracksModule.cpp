// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Modules/ModuleManager.h"
#include "IMovieSceneModule.h"
#include "IMovieSceneTracksModule.h"
#include "Components/SceneComponent.h"
#include "Systems/MovieScene3DTransformPropertySystem.h"

#include "EntitySystem/MovieSceneEntityManager.h"
#include "Systems/MovieSceneMaterialSystem.h"
#include "MovieSceneTracksComponentTypes.h"


DEFINE_STAT(MovieSceneEval_ReinitializeBoundMaterials);

/**
 * Implements the MovieSceneTracks module.
 */
class FMovieSceneTracksModule
	: public IMovieSceneTracksModule
{

	virtual void StartupModule() override
	{
		IMovieSceneModule& MovieSceneModule = IMovieSceneModule::Get();
		
		MovieSceneModule.RegisterEvaluationGroupParameters(
			IMovieSceneTracksModule::GetEvaluationGroupName(EBuiltInEvaluationGroup::PreEvaluation),
			FMovieSceneEvaluationGroupParameters(0x8FFF));

		MovieSceneModule.RegisterEvaluationGroupParameters(
			IMovieSceneTracksModule::GetEvaluationGroupName(EBuiltInEvaluationGroup::SpawnObjects),
			FMovieSceneEvaluationGroupParameters(0x0FFF));

		MovieSceneModule.RegisterEvaluationGroupParameters(
			IMovieSceneTracksModule::GetEvaluationGroupName(EBuiltInEvaluationGroup::PostEvaluation),
			FMovieSceneEvaluationGroupParameters(0x0008));

		UE::MovieScene::FMovieSceneTracksComponentTypes::Get();
	}

	virtual void ShutdownModule() override
	{
	}
};

FName IMovieSceneTracksModule::GetEvaluationGroupName(EBuiltInEvaluationGroup::Type InEvalGroup)
{
	static FName Names[] = {
		"PreEvaluation",
		"SpawnObjects",
		"PostEvaluation",
	};
	check(InEvalGroup >= 0 && InEvalGroup < UE_ARRAY_COUNT(Names));
	return Names[InEvalGroup];
}

IMPLEMENT_MODULE(FMovieSceneTracksModule, MovieSceneTracks);
