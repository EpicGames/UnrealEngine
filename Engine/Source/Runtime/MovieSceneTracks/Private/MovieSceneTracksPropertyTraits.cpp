// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTracksPropertyTraits.h"
#include "MovieSceneTracksPropertyTypes.h"
#include "Channels/MovieSceneUnpackedChannelValues.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneStringChannel.h"
#include "Channels/MovieSceneTextChannel.h"


namespace UE::MovieScene
{

void UnpackChannelsFromOperational(uint8 Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneByteChannel>(Value, 0), NAME_None));
}

void UnpackChannelsFromOperational(int64 Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneIntegerChannel>(Value, 0), NAME_None));
}

void UnpackChannelsFromOperational(double Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneDoubleChannel>(Value, 0), NAME_None));
}

void UnpackChannelsFromOperational(float Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneFloatChannel>(Value, 0), NAME_None));
}

void UnpackChannelsFromOperational(const FString& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneStringChannel>(Value, 0), NAME_None));
}

void UnpackChannelsFromOperational(const FText& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneTextChannel>(Value, 0), NAME_None));
}

void UnpackChannelsFromOperational(const FRotator& Rotator, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 0, Rotator, Pitch));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 1, Rotator, Yaw));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 2, Rotator, Roll));
}

void UnpackChannelsFromOperational(const FFloatIntermediateVector& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneFloatChannel, 0, Value, X));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneFloatChannel, 1, Value, Y));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneFloatChannel, 2, Value, Z));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneFloatChannel, 3, Value, W));
}

void UnpackChannelsFromOperational(const FDoubleIntermediateVector& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 0, Value, X));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 1, Value, Y));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 2, Value, Z));
	OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneDoubleChannel, 3, Value, W));
}

void UnpackChannelsFromOperational(const FIntermediateColor& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	const FStructProperty* StructProperty = CastField<FStructProperty>(&Property);
	if (StructProperty && StructProperty->Struct == FSlateColor::StaticStruct())
	{
		OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneFloatChannel>(Value.R, 0), FName("SpecifiedColor.R")));
		OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneFloatChannel>(Value.G, 1), FName("SpecifiedColor.G")));
		OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneFloatChannel>(Value.B, 2), FName("SpecifiedColor.B")));
		OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneFloatChannel>(Value.A, 3), FName("SpecifiedColor.A")));
	}
	else
	{
		OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneFloatChannel, 0, Value, R));
		OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneFloatChannel, 1, Value, G));
		OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneFloatChannel, 2, Value, B));
		OutUnpackedValues.Add(UE_MOVIESCENE_UNPACKED_MEMBER(FMovieSceneFloatChannel, 3, Value, A));
	}
}

void UnpackChannelsFromOperational(const FIntermediate3DTransform& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
{
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneDoubleChannel>(Value.T_X, 0), FName("Location.X")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneDoubleChannel>(Value.T_Y, 1), FName("Location.Y")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneDoubleChannel>(Value.T_Z, 2), FName("Location.Z")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneDoubleChannel>(Value.R_X, 3), FName("Rotation.X")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneDoubleChannel>(Value.R_Y, 4), FName("Rotation.Y")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneDoubleChannel>(Value.R_Z, 5), FName("Rotation.Z")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneDoubleChannel>(Value.S_X, 6), FName("Scale.X")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneDoubleChannel>(Value.S_Y, 7), FName("Scale.Y")));
	OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneDoubleChannel>(Value.S_Z, 8), FName("Scale.Z")));
}

} // namespace UE::MovieScene
