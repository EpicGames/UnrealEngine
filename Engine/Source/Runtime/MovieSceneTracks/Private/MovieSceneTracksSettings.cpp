// Copyright Epic Games, Inc. All Rights Reserved.

#include "MovieSceneTracksSettings.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTracksSettings)

UMovieSceneTracksSettings::UMovieSceneTracksSettings(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bPreviewCameraCutsInSimulate = true;
}

void UMovieSceneTracksSettings::SetPreviewCameraCutsInSimulate(bool bInPreviewCameraCutsInSimulate)
{
	if (bInPreviewCameraCutsInSimulate != bPreviewCameraCutsInSimulate)
	{
		bPreviewCameraCutsInSimulate = bInPreviewCameraCutsInSimulate;
		SaveConfig();
	}
}

