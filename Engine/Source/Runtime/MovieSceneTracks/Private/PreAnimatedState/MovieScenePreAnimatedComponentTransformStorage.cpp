// Copyright Epic Games, Inc. All Rights Reserved.

#include "PreAnimatedState/MovieScenePreAnimatedComponentTransformStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

#include "Components/SceneComponent.h"

namespace UE
{
namespace MovieScene
{

TAutoRegisterPreAnimatedStorageID<FPreAnimatedComponentTransformStorage> FPreAnimatedComponentTransformStorage::StorageID;

int32 GResetComponentVelocity = 1;
static FAutoConsoleVariableRef CVarResetComponentVelocity(
	TEXT("MovieScene.ResetComponentVelocity"),
	GResetComponentVelocity,
	TEXT("If 1 set component velocity to 0 on restore state, otherwise don't do anything (previous behavior)."),
	ECVF_Default);

struct FTemporaryMobilityScope
{
	// Ideally we would not be temporarily changing mobility here, but there are some very specific
	// edge cases where mobility can be legitimately restored whilst pre-animated transforms are still
	// maintained. One example is where an attach track has previously been run and since restored - 
	// thus detatching and resetting the transform. If nothing else animates the mobility, this will also
	// be reset, but the object's global transform may have been captured.

	FTemporaryMobilityScope(UObject* Object)
		: SceneComponent(Cast<USceneComponent>(Object))
		, PreviousMobility(SceneComponent ? SceneComponent->Mobility.GetValue() : EComponentMobility::Movable)
	{
		if (PreviousMobility != EComponentMobility::Movable)
		{
			SceneComponent->SetMobility(EComponentMobility::Movable);
		}
	}
	~FTemporaryMobilityScope()
	{
		if (PreviousMobility != EComponentMobility::Movable)
		{
			SceneComponent->SetMobility(PreviousMobility);
		}

		// Forcibly reset the component velocity which is set in FIntermediate3DTransform::ApplyTransformTo
		if (CVarResetComponentVelocity->GetInt() > 0)
		{
			SceneComponent->ComponentVelocity = FVector(0.f);
		}
	}

	USceneComponent* SceneComponent;
	const EComponentMobility::Type PreviousMobility;
};


void FComponentTransformPreAnimatedTraits::SetObjectPropertyValue(UObject* InObject, const FCustomPropertyAccessor& BaseCustomAccessor, const FIntermediate3DTransform& CachedTransform)
{
	FTemporaryMobilityScope TemporaryMobilityScope(InObject);

	const TCustomPropertyAccessor<TraitsType>& CustomAccessor = static_cast<const TCustomPropertyAccessor<TraitsType>&>(BaseCustomAccessor);
	(*CustomAccessor.Functions.Setter)(InObject, CachedTransform);
}

void FComponentTransformPreAnimatedTraits::SetObjectPropertyValue(UObject* InObject, uint16 PropertyOffset, const FIntermediate3DTransform& CachedTransform)
{
	FTemporaryMobilityScope TemporaryMobilityScope(InObject);

	StorageType* PropertyAddress = reinterpret_cast<StorageType*>( reinterpret_cast<uint8*>(InObject) + PropertyOffset );
	*PropertyAddress = CachedTransform;
}

void FComponentTransformPreAnimatedTraits::SetObjectPropertyValue(UObject* InObject, FTrackInstancePropertyBindings* PropertyBindings, const FIntermediate3DTransform& CachedTransform)
{
	FTemporaryMobilityScope TemporaryMobilityScope(InObject);

	PropertyBindings->CallFunction<StorageType>(*InObject, CachedTransform);
}

FPreAnimatedComponentTransformStorage::FPreAnimatedComponentTransformStorage()
	: TPreAnimatedPropertyStorage<FComponentTransformPreAnimatedTraits>(FBuiltInComponentTypes::Get()->PropertyRegistry.GetDefinition(FMovieSceneTracksComponentTypes::Get()->ComponentTransform.CompositeID))
{}

void FPreAnimatedComponentTransformStorage::CachePreAnimatedTransforms(const FCachePreAnimatedValueParams& Params, TArrayView<UObject* const> BoundObjects, TOptional<TFunctionRef<bool(int32)>> Predicate)
{

	for (int32 Index = 0; Index < BoundObjects.Num(); ++Index)
	{
		UObject* BoundObject = BoundObjects[Index];
		if (BoundObject && (!Predicate || (*Predicate)(Index)))
		{
			CachePreAnimatedTransform(Params, BoundObject);
		}
	}
}

void FPreAnimatedComponentTransformStorage::CachePreAnimatedTransform(const FCachePreAnimatedValueParams& Params, UObject* BoundObject)
{
	check(BoundObject);

	static FMovieScenePropertyBinding PropertyBinding("Transform", TEXT("Transform"));

	TOptional<FResolvedProperty> Property = FPropertyRegistry::ResolveProperty(BoundObject, PropertyBinding, CustomAccessors);
	if (!Property)
	{
		return;
	}

	TTuple<FObjectKey, FName> Key{ BoundObject, PropertyBinding.PropertyPath };

	FPreAnimatedStorageGroupHandle GroupHandle  = this->Traits.MakeGroup(BoundObject);
	FPreAnimatedStorageIndex       StorageIndex = this->GetOrCreateStorageIndex(Key);

	FPreAnimatedStateEntry Entry{ GroupHandle, FPreAnimatedStateCachedValueHandle{ StorageID, StorageIndex } };

	ParentExtension->EnsureMetaData(Entry);

	EPreAnimatedStorageRequirement StorageRequirement = ParentExtension->GetStorageRequirement(Entry);
	if (!this->IsStorageRequirementSatisfied(StorageIndex, StorageRequirement))
	{
		StorageType NewValue;

		if (const uint16* Fast = Property->TryGet<uint16>())
		{
			NewValue.Binding.template Set<uint16>(*Fast);
			FComponentTransformPropertyTraits::GetObjectPropertyValue(BoundObject, *Fast, NewValue.Data);
		}
		else if (const FCustomPropertyIndex* CustomIndex = Property->TryGet<FCustomPropertyIndex>())
		{
			const FCustomPropertyAccessor& Accessor = CustomAccessors[CustomIndex->Value];

			NewValue.Binding.template Set<const FCustomPropertyAccessor*>(&Accessor);
			FComponentTransformPropertyTraits::GetObjectPropertyValue(BoundObject, Accessor, NewValue.Data);
		}
		else
		{
			const TSharedPtr<FTrackInstancePropertyBindings>& Bindings = Property->Get<TSharedPtr<FTrackInstancePropertyBindings>>();

			NewValue.Binding.template Set<TSharedPtr<FTrackInstancePropertyBindings>>(Bindings);
			FComponentTransformPropertyTraits::GetObjectPropertyValue(BoundObject, Bindings.Get(), NewValue.Data);
		}

		this->AssignPreAnimatedValue(StorageIndex, StorageRequirement, MoveTemp(NewValue));
	}

	if (Params.bForcePersist)
	{
		this->ForciblyPersistStorage(StorageIndex);
	}
}

} // namespace MovieScene
} // namespace UE