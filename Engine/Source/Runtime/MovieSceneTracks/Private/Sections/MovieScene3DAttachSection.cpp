// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieScene3DAttachSection.h"
#include "UObject/SequencerObjectVersion.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DAttachSection)


UMovieScene3DAttachSection::UMovieScene3DAttachSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	AttachSocketName = NAME_None;
	AttachComponentName = NAME_None;
	AttachmentLocationRule = EAttachmentRule::KeepRelative;
	AttachmentRotationRule = EAttachmentRule::KeepRelative;
	AttachmentScaleRule = EAttachmentRule::KeepRelative;
	DetachmentLocationRule = EDetachmentRule::KeepRelative;
	DetachmentRotationRule = EDetachmentRule::KeepRelative;
	DetachmentScaleRule = EDetachmentRule::KeepRelative;

	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);
}


void UMovieScene3DAttachSection::SetAttachTargetID( const FMovieSceneObjectBindingID& InAttachBindingID )
{
	if (TryModify())
	{
		ConstraintBindingID = InAttachBindingID;
	}
}

void UMovieScene3DAttachSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!ConstraintBindingID.GetGuid().IsValid() || !Params.GetObjectBindingID().IsValid())
	{
		return;
	}

	FBuiltInComponentTypes*          BuiltInComponentTypes = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents       = FMovieSceneTracksComponentTypes::Get();

	FAttachmentComponent AttachComponent = {
		FComponentAttachParamsDestination { AttachSocketName, AttachComponentName },
		FComponentAttachParams{ AttachmentLocationRule, AttachmentRotationRule, AttachmentScaleRule },
		FComponentDetachParams{ DetachmentLocationRule, DetachmentRotationRule, DetachmentScaleRule }
	};

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(BuiltInComponentTypes->GenericObjectBinding, Params.GetObjectBindingID())
		.Add(BuiltInComponentTypes->BoundObjectResolver, MovieSceneHelpers::ResolveSceneComponentBoundObject)
		.Add(TrackComponents->AttachParentBinding, ConstraintBindingID)
		.Add(TrackComponents->AttachComponent, AttachComponent)
	);
}

