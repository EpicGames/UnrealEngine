// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieScene3DConstraintSection.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequenceID.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DConstraintSection)

UMovieScene3DConstraintSection::UMovieScene3DConstraintSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{ 
	bSupportsInfiniteRange = true;
}

void UMovieScene3DConstraintSection::OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	UE::MovieScene::FFixedObjectBindingID FixedBindingID = ConstraintBindingID.ResolveToFixed(LocalSequenceID, SharedPlaybackState);

	if (OldFixedToNewFixedMap.Contains(FixedBindingID))
	{
		Modify();

		const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy();
		SetConstraintBindingID(OldFixedToNewFixedMap[FixedBindingID].ConvertToRelative(LocalSequenceID, Hierarchy));
	}
}

void UMovieScene3DConstraintSection::GetReferencedBindings(TArray<FGuid>& OutBindings)
{
	OutBindings.Add(ConstraintBindingID.GetGuid());
}

void UMovieScene3DConstraintSection::PostLoad()
{
	Super::PostLoad();

	if (ConstraintId_DEPRECATED.IsValid())
	{
		if (!ConstraintBindingID.IsValid())
		{
			ConstraintBindingID = UE::MovieScene::FRelativeObjectBindingID(ConstraintId_DEPRECATED);
		}
		ConstraintId_DEPRECATED.Invalidate();
	}
}

