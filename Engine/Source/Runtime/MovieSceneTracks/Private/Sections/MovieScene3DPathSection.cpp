// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieScene3DPathSection.h"
#include "Components/SplineComponent.h"
#include "Channels/MovieSceneChannelProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DPathSection)


UMovieScene3DPathSection::UMovieScene3DPathSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
	, FrontAxisEnum(MovieScene3DPathSection_Axis::Y)
	, UpAxisEnum(MovieScene3DPathSection_Axis::Z)
	, bFollow (true)
	, bReverse (false)
	, bForceUpright (false)
{
#if WITH_EDITOR

	static const FMovieSceneChannelMetaData MetaData("Timing", NSLOCTEXT("MovieScene3DPathSection", "TimingArea", "Timing"));
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(TimingCurve, MetaData, TMovieSceneExternalValue<float>());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(TimingCurve);

#endif
}

void UMovieScene3DPathSection::InitialPlacement(const TArray<UMovieSceneSection*>& Sections, FFrameNumber InStartTime, int32 Duration, bool bAllowMultipleRows)
{
	Super::InitialPlacement(Sections, InStartTime, Duration, bAllowMultipleRows);

	TMovieSceneChannelData<FMovieSceneFloatValue> KeyData = TimingCurve.GetData();
	KeyData.UpdateOrAddKey(InStartTime, FMovieSceneFloatValue(0.f));
	if (Duration > 0)
	{
		KeyData.UpdateOrAddKey(InStartTime + Duration, FMovieSceneFloatValue(1.f));
	}
}

void UMovieScene3DPathSection::Eval( USceneComponent* SceneComponent, FFrameTime Position, USplineComponent* SplineComponent, FVector& OutTranslation, FRotator& OutRotation ) const
{
	float Timing = 0.f;
	TimingCurve.Evaluate( Position, Timing );

	if (Timing < 0.f)
	{
		Timing = 0.f;
	}
	else if (Timing > 1.f)
	{
		Timing = 1.f;
	}

	if (bReverse)
	{
		Timing = 1.f - Timing;
	}

	const bool UseConstantVelocity = true;
	OutTranslation = SplineComponent->GetWorldLocationAtTime(Timing, UseConstantVelocity);
	OutRotation = SplineComponent->GetWorldRotationAtTime(Timing, UseConstantVelocity);
	
	FMatrix NewRotationMatrix = FRotationMatrix(OutRotation);

	FVector UpAxis(0, 0, 0);
	if (UpAxisEnum == MovieScene3DPathSection_Axis::X)
	{
		UpAxis = FVector(1, 0, 0);
	}
	else if (UpAxisEnum == MovieScene3DPathSection_Axis::NEG_X)
	{
		UpAxis = FVector(-1, 0, 0);
	}
	else if (UpAxisEnum == MovieScene3DPathSection_Axis::Y)
	{
		UpAxis = FVector(0, 1, 0);
	}
	else if (UpAxisEnum == MovieScene3DPathSection_Axis::NEG_Y)
	{
		UpAxis = FVector(0, -1, 0);
	}
	else if (UpAxisEnum == MovieScene3DPathSection_Axis::Z)
	{
		UpAxis = FVector(0, 0, 1);
	}
	else if (UpAxisEnum == MovieScene3DPathSection_Axis::NEG_Z)
	{
		UpAxis = FVector(0, 0, -1);
	}

	FVector FrontAxis(0, 0, 0);
	if (FrontAxisEnum == MovieScene3DPathSection_Axis::X)
	{
		FrontAxis = FVector(1, 0, 0);
	}
	else if (FrontAxisEnum == MovieScene3DPathSection_Axis::NEG_X)
	{
		FrontAxis = FVector(-1, 0, 0);
	}
	else if (FrontAxisEnum == MovieScene3DPathSection_Axis::Y)
	{
		FrontAxis = FVector(0, 1, 0);
	}
	else if (FrontAxisEnum == MovieScene3DPathSection_Axis::NEG_Y)
	{
		FrontAxis = FVector(0, -1, 0);
	}
	else if (FrontAxisEnum == MovieScene3DPathSection_Axis::Z)
	{
		FrontAxis = FVector(0, 0, 1);
	}
	else if (FrontAxisEnum == MovieScene3DPathSection_Axis::NEG_Z)
	{
		FrontAxis = FVector(0, 0, -1);
	}

	// Negate the front axis because the spline rotation comes in reversed
	FrontAxis *= FVector(-1, -1, -1);

	FMatrix AxisRotator = FRotationMatrix::MakeFromXZ(FrontAxis, UpAxis);
	NewRotationMatrix = AxisRotator * NewRotationMatrix;
	OutRotation = NewRotationMatrix.Rotator();

	if (bForceUpright)
	{
		OutRotation.Pitch = 0.f;
		OutRotation.Roll = 0.f;
	}

	if (!bFollow)
	{
		OutRotation = SceneComponent->GetRelativeTransform().GetRotation().Rotator();
	}
}


void UMovieScene3DPathSection::SetPathBindingID( const FMovieSceneObjectBindingID& InPathBindingID )
{
	if (TryModify())
	{
		ConstraintBindingID = InPathBindingID;
	}
}

