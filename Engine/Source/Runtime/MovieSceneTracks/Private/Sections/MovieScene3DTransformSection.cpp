// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieScene3DTransformSection.h"
#include "Channels/IMovieSceneChannelOverrideProvider.h"
#include "UObject/StructOnScope.h"
#include "UObject/SequencerObjectVersion.h"
#include "Algo/AnyOf.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneSectionChannelOverrideRegistry.h"
#include "GameFramework/Actor.h"
#include "EulerTransform.h"
#include "Systems/MovieScene3DTransformPropertySystem.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneEulerTransformTrack.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformableHandle.h"
#include "UObject/ObjectSaveContext.h"
#include "Misc/AxisDisplayInfo.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DTransformSection)

#if WITH_EDITOR

namespace
{
FIntVector4 ReverseSwizzleFunc(const FIntVector4& InSwizzle)
{
	FIntVector4 ReverseSwizzle;
	for (int32 i = 0; i < 4; ++i)
	{
		int32 Index = 0;
		for (int32 j = 0; j < 4; ++j)
		{
			if (InSwizzle[j] == i)
			{
				Index = j;
				break;
			}
		}
		ReverseSwizzle[i] = Index;
	}

	return ReverseSwizzle;
};
} // empty namespace

struct F3DTransformChannelEditorData
{
	F3DTransformChannelEditorData(EMovieSceneTransformChannel Mask,int SortOrderStart)
	{
		static const TSet<FName> PropertyMetaDataKeys = { "UIMin", "UIMax", "SliderExponent", "LinearDeltaSensitivity", "Delta", "ClampMin", "ClampMax", "ForceUnits", "WheelStep" };

		auto MakeSubPropertyMap = [](FName FTransformName, FName FEulerTransformName)->TMap<FName, FName>
		{
			return
			{
				{ TBaseStructure<FTransform>::Get()->GetFName(), FTransformName },
				{ TBaseStructure<FEulerTransform>::Get()->GetFName(), FEulerTransformName }
			};
		};

		const FProperty* RelativeLocationProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeLocationPropertyName());
		const FProperty* RelativeRotationProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeRotationPropertyName());
		const FProperty* RelativeScale3DProperty = USceneComponent::StaticClass()->FindPropertyByName(USceneComponent::GetRelativeScale3DPropertyName());

		const EAxisList::Type XAxis = EAxisList::Forward;
		const EAxisList::Type YAxis = EAxisList::Left;
		const EAxisList::Type ZAxis = EAxisList::Up;

		const FIntVector4 Swizzle = AxisDisplayInfo::GetTransformAxisSwizzle();
		const FIntVector4 ReverseSwizzle = ReverseSwizzleFunc(Swizzle);
		const int32 TranslationOrderOffset = 0;
		const int32 RotationOrderOffset = TranslationOrderOffset + 3;
		const int32 ScaleOrderOffset = RotationOrderOffset + 3;

		FText LocationGroup = NSLOCTEXT("MovieSceneTransformSection", "Location", "Location");
		FText RotationGroup = NSLOCTEXT("MovieSceneTransformSection", "Rotation", "Rotation");
		FText ScaleGroup    = NSLOCTEXT("MovieSceneTransformSection", "Scale",    "Scale");
		{
			MetaData[0].SetIdentifiers("Location.X", AxisDisplayInfo::GetAxisDisplayName(XAxis), LocationGroup);
			MetaData[0].SubPropertyPathMap = MakeSubPropertyMap(TEXT("Translation.X"), TEXT("Location.X"));
			MetaData[0].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::TranslationX);
			MetaData[0].Color = AxisDisplayInfo::GetAxisColor(XAxis);
			MetaData[0].SortOrder = SortOrderStart + TranslationOrderOffset + ReverseSwizzle[0];
			MetaData[0].bCanCollapseToTrack = false;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[0].PropertyMetaData.Add(PropertyMetaDataKey, RelativeLocationProperty->GetMetaData(PropertyMetaDataKey));
			}

			MetaData[1].SetIdentifiers("Location.Y", AxisDisplayInfo::GetAxisDisplayName(YAxis), LocationGroup);
			MetaData[1].SubPropertyPathMap = MakeSubPropertyMap(TEXT("Translation.Y"), TEXT("Location.Y"));
			MetaData[1].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::TranslationY);
			MetaData[1].Color = AxisDisplayInfo::GetAxisColor(YAxis);
			MetaData[1].SortOrder = SortOrderStart + TranslationOrderOffset + ReverseSwizzle[1];
			MetaData[1].bCanCollapseToTrack = false;
			MetaData[1].bInvertValue = AxisDisplayInfo::GetAxisDisplayCoordinateSystem() == EAxisList::LeftUpForward;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[1].PropertyMetaData.Add(PropertyMetaDataKey, RelativeLocationProperty->GetMetaData(PropertyMetaDataKey));
			}

			MetaData[2].SetIdentifiers("Location.Z", AxisDisplayInfo::GetAxisDisplayName(ZAxis), LocationGroup);
			MetaData[2].SubPropertyPathMap = MakeSubPropertyMap(TEXT("Translation.Z"), TEXT("Location.Z"));
			MetaData[2].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::TranslationZ);
			MetaData[2].Color = AxisDisplayInfo::GetAxisColor(ZAxis);
			MetaData[2].SortOrder = SortOrderStart + TranslationOrderOffset + ReverseSwizzle[2];
			MetaData[2].bCanCollapseToTrack = false;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[2].PropertyMetaData.Add(PropertyMetaDataKey, RelativeLocationProperty->GetMetaData(PropertyMetaDataKey));
			}
		}
		{
			MetaData[3].SetIdentifiers("Rotation.X", NSLOCTEXT("MovieSceneTransformSection", "RotationX", "Roll"), RotationGroup);
			MetaData[3].SubPropertyPathMap = MakeSubPropertyMap(TEXT("Rotation.X"), TEXT("Rotation.Roll"));
			MetaData[3].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::RotationX);
			MetaData[3].Color = AxisDisplayInfo::GetAxisColor(XAxis);
			MetaData[3].SortOrder = SortOrderStart + RotationOrderOffset + 0;
			MetaData[3].bCanCollapseToTrack = false;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[3].PropertyMetaData.Add(PropertyMetaDataKey, RelativeRotationProperty->GetMetaData(PropertyMetaDataKey));
			}

			MetaData[4].SetIdentifiers("Rotation.Y", NSLOCTEXT("MovieSceneTransformSection", "RotationY", "Pitch"), RotationGroup);
			MetaData[4].SubPropertyPathMap = MakeSubPropertyMap(TEXT("Rotation.Y"), TEXT("Rotation.Pitch"));
			MetaData[4].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::RotationY);
			MetaData[4].Color = AxisDisplayInfo::GetAxisColor(YAxis);
			MetaData[4].SortOrder = SortOrderStart + RotationOrderOffset + 1;
			MetaData[4].bCanCollapseToTrack = false;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[4].PropertyMetaData.Add(PropertyMetaDataKey, RelativeRotationProperty->GetMetaData(PropertyMetaDataKey));
			}

			MetaData[5].SetIdentifiers("Rotation.Z", NSLOCTEXT("MovieSceneTransformSection", "RotationZ", "Yaw"), RotationGroup);
			MetaData[5].SubPropertyPathMap = MakeSubPropertyMap(TEXT("Rotation.Z"), TEXT("Rotation.Yaw"));
			MetaData[5].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::RotationZ);
			MetaData[5].Color = AxisDisplayInfo::GetAxisColor(ZAxis);
			MetaData[5].SortOrder = SortOrderStart + RotationOrderOffset + 2;
			MetaData[5].bCanCollapseToTrack = false;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[5].PropertyMetaData.Add(PropertyMetaDataKey, RelativeRotationProperty->GetMetaData(PropertyMetaDataKey));
			}
		}
		{
			MetaData[6].SetIdentifiers("Scale.X", AxisDisplayInfo::GetAxisDisplayName(XAxis), ScaleGroup);
			MetaData[6].SubPropertyPathMap = MakeSubPropertyMap(TEXT("Scale3D.X"), TEXT("Scale.X"));
			MetaData[6].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::ScaleX);
			MetaData[6].Color = AxisDisplayInfo::GetAxisColor(XAxis);
			MetaData[6].SortOrder = SortOrderStart + ScaleOrderOffset + ReverseSwizzle[0];
			MetaData[6].bCanCollapseToTrack = false;
			MetaData[6].bCanPreserveRatio = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[6].PropertyMetaData.Add(PropertyMetaDataKey, RelativeScale3DProperty->GetMetaData(PropertyMetaDataKey));
			}

			MetaData[7].SetIdentifiers("Scale.Y", AxisDisplayInfo::GetAxisDisplayName(YAxis), ScaleGroup);
			MetaData[7].SubPropertyPathMap = MakeSubPropertyMap(TEXT("Scale3D.Y"), TEXT("Scale.Y"));
			MetaData[7].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::ScaleY);
			MetaData[7].Color = AxisDisplayInfo::GetAxisColor(YAxis);
			MetaData[7].SortOrder = SortOrderStart + ScaleOrderOffset + ReverseSwizzle[1];
			MetaData[7].bCanCollapseToTrack = false;
			MetaData[7].bCanPreserveRatio = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[7].PropertyMetaData.Add(PropertyMetaDataKey, RelativeScale3DProperty->GetMetaData(PropertyMetaDataKey));
			}

			MetaData[8].SetIdentifiers("Scale.Z", AxisDisplayInfo::GetAxisDisplayName(ZAxis), ScaleGroup);
			MetaData[8].SubPropertyPathMap = MakeSubPropertyMap(TEXT("Scale3D.Z"), TEXT("Scale.Z"));
			MetaData[8].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::ScaleZ);
			MetaData[8].Color = AxisDisplayInfo::GetAxisColor(ZAxis);
			MetaData[8].SortOrder = SortOrderStart + ScaleOrderOffset + ReverseSwizzle[2];
			MetaData[8].bCanCollapseToTrack = false;
			MetaData[8].bCanPreserveRatio = true;
			for (const FName& PropertyMetaDataKey : PropertyMetaDataKeys)
			{
				MetaData[8].PropertyMetaData.Add(PropertyMetaDataKey, RelativeScale3DProperty->GetMetaData(PropertyMetaDataKey));
			}
		}
		{
			MetaData[9].SetIdentifiers("Weight", NSLOCTEXT("MovieSceneTransformSection", "Weight", "Weight"));
			MetaData[9].bEnabled = EnumHasAllFlags(Mask, EMovieSceneTransformChannel::Weight);
		}

		ExternalValues[0].OnGetExternalValue = ExtractTranslationX;
		ExternalValues[1].OnGetExternalValue = ExtractTranslationY;
		ExternalValues[2].OnGetExternalValue = ExtractTranslationZ;
		ExternalValues[3].OnGetExternalValue = ExtractRotationX;
		ExternalValues[4].OnGetExternalValue = ExtractRotationY;
		ExternalValues[5].OnGetExternalValue = ExtractRotationZ;
		ExternalValues[6].OnGetExternalValue = ExtractScaleX;
		ExternalValues[7].OnGetExternalValue = ExtractScaleY;
		ExternalValues[8].OnGetExternalValue = ExtractScaleZ;
	}

	static TOptional<FVector> GetTranslation(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		const UStruct* TransformStruct = Bindings ? Bindings->GetPropertyStruct(InObject) : nullptr;

		if (TransformStruct)
		{
			if (TransformStruct == TBaseStructure<FTransform>::Get())
			{
				if (TOptional<FTransform> Transform = Bindings->GetOptionalValue<FTransform>(InObject))
				{
					return Transform->GetTranslation();
				}
			}
			else if (TransformStruct == TBaseStructure<FEulerTransform>::Get())
			{
				if (TOptional<FEulerTransform> EulerTransform = Bindings->GetOptionalValue<FEulerTransform>(InObject))
				{
					return EulerTransform->Location;
				}
			}
		}
		else if (USceneComponent* SceneComponent = Cast<USceneComponent>(&InObject))
		{
			return SceneComponent->GetRelativeTransform().GetTranslation();		
		}
		else if (AActor* Actor = Cast<AActor>(&InObject))
		{
			if (USceneComponent* RootComponent = Actor->GetRootComponent())
			{
				return RootComponent->GetRelativeTransform().GetTranslation();
			}
		}

		return TOptional<FVector>();
	}

	static TOptional<FRotator> GetRotator(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		const UStruct* TransformStruct = Bindings ? Bindings->GetPropertyStruct(InObject) : nullptr;

		if (TransformStruct)
		{
			if (TransformStruct == TBaseStructure<FTransform>::Get())
			{
				if (TOptional<FTransform> Transform = Bindings->GetOptionalValue<FTransform>(InObject))
				{
					return Transform->GetRotation().Rotator();
				}
			}
			else if (TransformStruct == TBaseStructure<FEulerTransform>::Get())
			{
				if (TOptional<FEulerTransform> EulerTransform = Bindings->GetOptionalValue<FEulerTransform>(InObject))
				{
					return EulerTransform->Rotation;
				}
			}
		}
		else if (USceneComponent* SceneComponent = Cast<USceneComponent>(&InObject))
		{
			return SceneComponent->GetRelativeRotation();
		}
		else if (AActor* Actor = Cast<AActor>(&InObject))
		{
			if (USceneComponent* RootComponent = Actor->GetRootComponent())
			{
				return RootComponent->GetRelativeRotation();
			}
		}

		return TOptional<FRotator>();
	}

	static TOptional<FVector> GetScale(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		const UStruct* TransformStruct = Bindings ? Bindings->GetPropertyStruct(InObject) : nullptr;

		if (TransformStruct)
		{
			if (TransformStruct == TBaseStructure<FTransform>::Get())
			{
				if (TOptional<FTransform> Transform = Bindings->GetOptionalValue<FTransform>(InObject))
				{
					return Transform->GetScale3D();
				}
			}
			else if (TransformStruct == TBaseStructure<FEulerTransform>::Get())
			{
				if (TOptional<FEulerTransform> EulerTransform = Bindings->GetOptionalValue<FEulerTransform>(InObject))
				{
					return EulerTransform->Scale;
				}
			}
		}
		else if (USceneComponent* SceneComponent = Cast<USceneComponent>(&InObject))
		{
			return SceneComponent->GetRelativeTransform().GetScale3D();
		}
		else if (AActor* Actor = Cast<AActor>(&InObject))
		{
			if (USceneComponent* RootComponent = Actor->GetRootComponent())
			{
				return RootComponent->GetRelativeTransform().GetScale3D();
			}
		}

		return TOptional<FVector>();
	}

	static TOptional<double> ExtractTranslationX(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		TOptional<FVector> Translation = GetTranslation(InObject, Bindings);
		return Translation.IsSet() ? Translation->X : TOptional<double>();
	}
	static TOptional<double> ExtractTranslationY(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		TOptional<FVector> Translation = GetTranslation(InObject, Bindings);
		return Translation.IsSet() ? Translation->Y : TOptional<double>();
	}
	static TOptional<double> ExtractTranslationZ(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		TOptional<FVector> Translation = GetTranslation(InObject, Bindings);
		return Translation.IsSet() ? Translation->Z : TOptional<double>();
	}

	static TOptional<double> ExtractRotationX(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		TOptional<FRotator> Rotator = GetRotator(InObject, Bindings);
		return Rotator.IsSet() ? Rotator->Roll : TOptional<double>();
	}
	static TOptional<double> ExtractRotationY(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		TOptional<FRotator> Rotator = GetRotator(InObject, Bindings);
		return Rotator.IsSet() ? Rotator->Pitch : TOptional<double>();
	}
	static TOptional<double> ExtractRotationZ(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		TOptional<FRotator> Rotator = GetRotator(InObject, Bindings);
		return Rotator.IsSet() ? Rotator->Yaw : TOptional<double>();
	}

	static TOptional<double> ExtractScaleX(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		TOptional<FVector> Scale = GetScale(InObject, Bindings);
		return Scale.IsSet() ? Scale->X : TOptional<double>();
	}
	static TOptional<double> ExtractScaleY(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		TOptional<FVector> Scale = GetScale(InObject, Bindings);
		return Scale.IsSet() ? Scale->Y : TOptional<double>();
	}
	static TOptional<double> ExtractScaleZ(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		TOptional<FVector> Scale = GetScale(InObject, Bindings);
		return Scale.IsSet() ? Scale->Z : TOptional<double>();
	}

	FMovieSceneChannelMetaData      MetaData[10];
	TMovieSceneExternalValue<double> ExternalValues[9];
	TMovieSceneExternalValue<float> WeightExternalValue;
};

#endif // WITH_EDITOR



/* FMovieScene3DLocationKeyStruct interface
 *****************************************************************************/

void FMovieScene3DLocationKeyStruct::PropagateChanges(const FPropertyChangedEvent& ChangeEvent)
{
	KeyStructInterop.Apply(Time);
}


/* FMovieScene3DRotationKeyStruct interface
 *****************************************************************************/

void FMovieScene3DRotationKeyStruct::PropagateChanges(const FPropertyChangedEvent& ChangeEvent)
{
	KeyStructInterop.Apply(Time);
}


/* FMovieScene3DScaleKeyStruct interface
 *****************************************************************************/

void FMovieScene3DScaleKeyStruct::PropagateChanges(const FPropertyChangedEvent& ChangeEvent)
{
	KeyStructInterop.Apply(Time);
}


/* FMovieScene3DTransformKeyStruct interface
 *****************************************************************************/

void FMovieScene3DTransformKeyStruct::PropagateChanges(const FPropertyChangedEvent& ChangeEvent)
{
	KeyStructInterop.Apply(Time);
}

namespace UE::MovieScene
{
	static constexpr uint32 ConstraintTypeMask = 1 << 31;
}


/* UMovieScene3DTransformSection interface
 *****************************************************************************/

UE::MovieScene::FChannelOverrideNames UMovieScene3DTransformSection::ChannelOverrideNames(10, {
		TEXT("Location.X"), TEXT("Location.Y"), TEXT("Location.Z"),
		TEXT("Rotation.X"), TEXT("Rotation.Y"), TEXT("Rotation.Z"),
		TEXT("Scale.X"), TEXT("Scale.Y"), TEXT("Scale.Z"),
		TEXT("Weight")
		});

UMovieScene3DTransformSection::UMovieScene3DTransformSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseQuaternionInterpolation(false)
#if WITH_EDITORONLY_DATA
	, Show3DTrajectory(EShow3DTrajectory::EST_OnlyWhenSelected)
#endif
{
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);

	TransformMask = EMovieSceneTransformChannel::AllTransform;
	BlendType = EMovieSceneBlendType::Absolute;
	bSupportsInfiniteRange = true;

	Translation[0].SetDefault(0.f);
	Translation[1].SetDefault(0.f);
	Translation[2].SetDefault(0.f);

	Rotation[0].SetDefault(0.f);
	Rotation[1].SetDefault(0.f);
	Rotation[2].SetDefault(0.f);

	Scale[0].SetDefault(1.f);
	Scale[1].SetDefault(1.f);
	Scale[2].SetDefault(1.f);
}

void UMovieScene3DTransformSection::OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	if (Constraints)
	{
		for (FConstraintAndActiveChannel& ConstraintChannel : Constraints->ConstraintsChannels)
		{
			if (UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(ConstraintChannel.GetConstraint()))
			{
				//Don't do child's we do that in the system, needed for duplication
				if (TransformConstraint->ParentTRSHandle)
				{
					TransformConstraint->ParentTRSHandle->OnBindingIDsUpdated(OldFixedToNewFixedMap, LocalSequenceID, SharedPlaybackState);
				}
			}
		}
	}
}

void UMovieScene3DTransformSection::GetReferencedBindings(TArray<FGuid>& OutBindings)
{
	if (Constraints)
	{
		for (FConstraintAndActiveChannel& ConstraintChannel : Constraints->ConstraintsChannels)
		{
			if (UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(ConstraintChannel.GetConstraint().Get()))
			{
				if (TransformConstraint->ChildTRSHandle && TransformConstraint->ChildTRSHandle->ConstraintBindingID.IsValid())
				{
					OutBindings.Add(TransformConstraint->ChildTRSHandle->ConstraintBindingID.GetGuid());
				}
				if (TransformConstraint->ParentTRSHandle && TransformConstraint->ParentTRSHandle->ConstraintBindingID.IsValid())
				{
					OutBindings.Add(TransformConstraint->ParentTRSHandle->ConstraintBindingID.GetGuid());
				}
			}
		}
	}
}

void UMovieScene3DTransformSection::PreSave(FObjectPreSaveContext SaveContext)
{
	Super::PreSave(SaveContext);
}

template<typename BaseBuilderType>
void UMovieScene3DTransformSection::BuildEntity(BaseBuilderType& InBaseBuilder, UMovieSceneEntitySystemLinker* Linker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{

	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponentTypes = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents       = FMovieSceneTracksComponentTypes::Get();
	UMovieScenePropertyTrack*        Track                 = GetTypedOuter<UMovieScenePropertyTrack>();

	check(Track);

	const bool bIsComponentTransform = Track->IsA<UMovieScene3DTransformTrack>();
	const bool bIsEulerTransform = Track->IsA<UMovieSceneEulerTransformTrack>();

	FComponentTypeID PropertyTag = TrackComponents->Transform.PropertyTag;
	if (bIsComponentTransform)
	{
		PropertyTag = TrackComponents->ComponentTransform.PropertyTag;
	}
	else if (bIsEulerTransform)
	{
		PropertyTag = TrackComponents->EulerTransform.PropertyTag;
	}

	EMovieSceneTransformChannel Channels = TransformMask.GetChannels();

	const bool ActiveChannelsMask[] = {
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::TranslationX) && Translation[0].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::TranslationY) && Translation[1].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::TranslationZ) && Translation[2].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::RotationX) && Rotation[0].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::RotationY) && Rotation[1].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::RotationZ) && Rotation[2].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::ScaleX) && Scale[0].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::ScaleY) && Scale[1].HasAnyData(),
		EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::ScaleZ) && Scale[2].HasAnyData(),
	};

	if (!Algo::AnyOf(ActiveChannelsMask))
	{
		return;
	}

	TComponentTypeID<FSourceDoubleChannel> RotationChannel[3];
	if (!bUseQuaternionInterpolation)
	{
		RotationChannel[0] = BuiltInComponentTypes->DoubleChannel[3];
		RotationChannel[1] = BuiltInComponentTypes->DoubleChannel[4];
		RotationChannel[2] = BuiltInComponentTypes->DoubleChannel[5];
	}
	else
	{
		RotationChannel[0] = TrackComponents->QuaternionRotationChannel[0];
		RotationChannel[1] = TrackComponents->QuaternionRotationChannel[1];
		RotationChannel[2] = TrackComponents->QuaternionRotationChannel[2];
	}

	// Let the override registry handle overriden channels
	const FName ChannelOverrideName = ChannelOverrideNames.GetChannelName(Params.EntityID);

	// If we are building the entity of an overriden channel, we pass the partial builder we have at this point 
	// (the one with the bindings) and add the tag to it.
	// Then call into the override repository so it can add a second builder with the override stuff.
	if (ChannelOverrideName != NAME_None)
	{
		if (!ensureMsgf(OverrideRegistry && OverrideRegistry->ContainsChannel(ChannelOverrideName),
				TEXT("We received an entity ID that corresponds to an override channel, but no such override channel exists in the registry.")))
		{
			return;
		}

		const int32 ChannelOverrideIndex = (Params.EntityID - ChannelOverrideNames.IndexOffset);
		if (ChannelOverrideIndex == 9)
		{
			if (EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::Weight))
			{
				OutImportedEntity->AddBuilder(InBaseBuilder.AddTag(PropertyTag));

				FChannelOverrideEntityImportParams OverrideParams{ ChannelOverrideName, BuiltInComponentTypes->WeightResult };
				OverrideRegistry->ImportEntityImpl(OverrideParams, Params, OutImportedEntity);
			}
		}
		else
		{
			if (ActiveChannelsMask[ChannelOverrideIndex])
			{
				OutImportedEntity->AddBuilder(InBaseBuilder.AddTag(PropertyTag));

				FChannelOverrideEntityImportParams OverrideParams{ ChannelOverrideName, BuiltInComponentTypes->DoubleResult[ChannelOverrideIndex] };
				OverrideRegistry->ImportEntityImpl(OverrideParams, Params, OutImportedEntity);
			}
		}
		return;
	}	
	
	// Here proceed with the all the normal channels
	OutImportedEntity->AddBuilder(
		InBaseBuilder
		.AddConditional(BuiltInComponentTypes->DoubleChannel[0], &Translation[0], ActiveChannelsMask[0] && !IsChannelOverriden(OverrideRegistry, TEXT("Location.X")))
		.AddConditional(BuiltInComponentTypes->DoubleChannel[1], &Translation[1], ActiveChannelsMask[1] && !IsChannelOverriden(OverrideRegistry, TEXT("Location.Y")))
		.AddConditional(BuiltInComponentTypes->DoubleChannel[2], &Translation[2], ActiveChannelsMask[2] && !IsChannelOverriden(OverrideRegistry, TEXT("Location.Z")))
		.AddConditional(RotationChannel[0],                      &Rotation[0],    ActiveChannelsMask[3] && !IsChannelOverriden(OverrideRegistry, TEXT("Rotation.X")))
		.AddConditional(RotationChannel[1],                      &Rotation[1],    ActiveChannelsMask[4] && !IsChannelOverriden(OverrideRegistry, TEXT("Rotation.Y")))
		.AddConditional(RotationChannel[2],                      &Rotation[2],    ActiveChannelsMask[5] && !IsChannelOverriden(OverrideRegistry, TEXT("Rotation.Z")))
		.AddConditional(BuiltInComponentTypes->DoubleChannel[6], &Scale[0],       ActiveChannelsMask[6] && !IsChannelOverriden(OverrideRegistry, TEXT("Scale.X")))
		.AddConditional(BuiltInComponentTypes->DoubleChannel[7], &Scale[1],       ActiveChannelsMask[7] && !IsChannelOverriden(OverrideRegistry, TEXT("Scale.Y")))
		.AddConditional(BuiltInComponentTypes->DoubleChannel[8], &Scale[2],       ActiveChannelsMask[8] && !IsChannelOverriden(OverrideRegistry, TEXT("Scale.Z")))
		.AddConditional(BuiltInComponentTypes->WeightChannel,    &ManualWeight,   EnumHasAnyFlags(Channels, EMovieSceneTransformChannel::Weight) && ManualWeight.HasAnyData() && !IsChannelOverriden(OverrideRegistry, TEXT("Weight")))
		.AddTag(PropertyTag)
	);
}

void UMovieScene3DTransformSection::PopulateConstraintEntities(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	check(Constraints);

	using namespace UE::MovieScene;

	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	// Add explicitly typed entities for each constraint
	// Encode the top bit of the entity ID to mean it is a constraint
	// We can check this in ImportEntityImpl to import the correct constraint
	for (int32 ConstraintIndex = 0; ConstraintIndex < Constraints->ConstraintsChannels.Num(); ++ConstraintIndex)
	{
		// Entity IDs for constraints are their index within the array, masked with the top bit set
		const uint32 EntityID = ConstraintIndex | ConstraintTypeMask;
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, this, EntityID, MetaDataIndex);
	}
}


bool UMovieScene3DTransformSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	if (Constraints)
	{
		PopulateConstraintEntities(EffectiveRange, InMetaData, OutFieldBuilder);
	}
	if (OverrideRegistry)
	{
		// Add evaluation field entries for each channel that runs with a different logic.
		OverrideRegistry->PopulateEvaluationFieldImpl(EffectiveRange, InMetaData, OutFieldBuilder, *this);
	}

	// Return false even if we have an override registry, so that we also add an evaluation field entry for the default
	// stuff.
	return false;
}

void UMovieScene3DTransformSection::ImportConstraintEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponentTypes = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents       = FMovieSceneTracksComponentTypes::Get();

	// Constraints must always operate on a USceneComponent. Putting one on a generic FTransform property has no effect (or is not possible).
	FGuid ObjectBindingID = Params.GetObjectBindingID();
	if (ObjectBindingID.IsValid())
	{
		// Mask out the top bit to get the constraint index we encoded into the EntityID.
		const int32 ConstraintIndex = static_cast<int32>(Params.EntityID & ~ConstraintTypeMask);

		checkf(Constraints->ConstraintsChannels.IsValidIndex(ConstraintIndex), TEXT("Encoded constraint (%d) index is not valid within array size %d. Data must have been manipulated without re-compilaition."), ConstraintIndex, Constraints->ConstraintsChannels.Num());
		//add if constraint or spawn copy is valid
		if (Constraints->ConstraintsChannels[ConstraintIndex].GetConstraint().Get())
		{
			FGuid ConstraintID = Constraints->ConstraintsChannels[ConstraintIndex].GetConstraint()->ConstraintID;
			//ID's should be the same!

			FConstraintComponentData ComponentData;
			ComponentData.ConstraintID = ConstraintID;
			ComponentData.Section = this;
			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(BuiltInComponentTypes->BoundObjectResolver, MovieSceneHelpers::ResolveSceneComponentBoundObject)
				.Add(BuiltInComponentTypes->GenericObjectBinding, ObjectBindingID)
				.Add(TrackComponents->ConstraintChannel, ComponentData)
			);
		}
	}
}

void UMovieScene3DTransformSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if ((Params.EntityID & ConstraintTypeMask) != 0  && Constraints) 
	{
		ImportConstraintEntity(EntityLinker, Params, OutImportedEntity);
		return;
	}

	FBuiltInComponentTypes*   BuiltInComponentTypes = FBuiltInComponentTypes::Get();
	UMovieScenePropertyTrack* Track                 = GetTypedOuter<UMovieScenePropertyTrack>();

	check(Track);

	FGuid ObjectBindingID = Params.GetObjectBindingID();

	auto BaseBuilder = FEntityBuilder()
		.Add(BuiltInComponentTypes->PropertyBinding, Track->GetPropertyBinding())
		// 3D Transform tracks use a scene component binding by default. Every other transform property track must be bound directly to the object.
		.AddConditional(BuiltInComponentTypes->BoundObjectResolver, MovieSceneHelpers::ResolveSceneComponentBoundObject, Track->IsA<UMovieScene3DTransformTrack>())
		.AddConditional(BuiltInComponentTypes->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid());

	BuildEntity(BaseBuilder, EntityLinker, Params, OutImportedEntity);
}

void UMovieScene3DTransformSection::InterrogateEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();

	auto BaseBuilder = FEntityBuilder().AddDefaulted(BuiltInComponentTypes->EvalTime);
	BuildEntity(BaseBuilder, EntityLinker, Params, OutImportedEntity);
}

FMovieSceneTransformMask UMovieScene3DTransformSection::GetMask() const
{
	return TransformMask;
}

void UMovieScene3DTransformSection::SetMask(FMovieSceneTransformMask NewMask)
{
	TransformMask = NewMask;
	ChannelProxy = nullptr;
}

FMovieSceneTransformMask UMovieScene3DTransformSection::GetMaskByName(const FName& InName) const
{
	if (InName.ToString() == NSLOCTEXT("MovieSceneTransformSection", "Location", "Location").ToString())
	{
		return EMovieSceneTransformChannel::Translation;
	}
	else if (InName == TEXT("Location.X"))
	{
		return EMovieSceneTransformChannel::TranslationX;
	}
	else if (InName == TEXT("Location.Y"))
	{
		return EMovieSceneTransformChannel::TranslationY;
	}
	else if (InName == TEXT("Location.Z"))
	{
		return EMovieSceneTransformChannel::TranslationZ;
	}
	else if (InName.ToString() == NSLOCTEXT("MovieSceneTransformSection", "Rotation", "Rotation").ToString())
	{
		return EMovieSceneTransformChannel::Rotation;
	}
	else if (InName == TEXT("Rotation.X"))
	{
		return EMovieSceneTransformChannel::RotationX;
	}
	else if (InName == TEXT("Rotation.Y"))
	{
		return EMovieSceneTransformChannel::RotationY;
	}
	else if (InName == TEXT("Rotation.Z"))
	{
		return EMovieSceneTransformChannel::RotationZ;
	}
	else if (InName.ToString() == NSLOCTEXT("MovieSceneTransformSection", "Scale", "Scale").ToString())
	{
		return EMovieSceneTransformChannel::Scale;
	}
	else if (InName == TEXT("Scale.X"))
	{
		return EMovieSceneTransformChannel::ScaleX;
	}
	else if (InName == TEXT("Scale.Y"))
	{
		return EMovieSceneTransformChannel::ScaleY;
	}
	else if (InName == TEXT("Scale.Z"))
	{
		return EMovieSceneTransformChannel::ScaleZ;
	}
	//Constraints aren't masked since they need to be deleted etc via their own API's
	FString ConstraintString = NSLOCTEXT("MovieSceneTransformSection", "Constraint", "Constraint").ToString();
	if (InName.ToString().Contains(ConstraintString))
	{
		return EMovieSceneTransformChannel::None;
	}
	return EMovieSceneTransformChannel::All;
}

EMovieSceneChannelProxyType UMovieScene3DTransformSection::CacheChannelProxy()
{
	using namespace UE::MovieScene;

	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR

	auto GetValidConstraint = [](FConstraintAndActiveChannel& ConstraintChannel)
	{
		return ConstraintChannel.GetConstraint().Get();
	};
#endif
		
	//Constraints go on top
	int32 NumConstraints = 0;
	if(Constraints)
	{ 
		for (FConstraintAndActiveChannel& ConstraintChannel : Constraints->ConstraintsChannels)
		{
#if WITH_EDITOR
			TWeakObjectPtr<UTickableConstraint> WeakConstraint = GetValidConstraint(ConstraintChannel);
			if (WeakConstraint.IsValid())
			{
				FText ConstraintGroup = NSLOCTEXT("MovieSceneTransformSection", "Constraints", "Constraints");
				const FName& ConstraintName = WeakConstraint->GetFName();
				ConstraintChannel.ActiveChannel.ExtraLabel = [WeakConstraint]
				{
					if (WeakConstraint.IsValid())
					{
						constexpr TCHAR ParentChildDelimiter[] = TEXT("|");
						FString ParentStr; WeakConstraint->GetLabel().Split(ParentChildDelimiter, &ParentStr, nullptr);
						if (!ParentStr.IsEmpty())
						{
							return ParentStr;
						}
					}
					static const FString DummyStr;
					return DummyStr;
				};
				const FText DisplayText = FText::FromString(WeakConstraint->GetTypeLabel());
				FMovieSceneChannelMetaData MetaData(ConstraintName, DisplayText, ConstraintGroup, true /*bInEnabled*/);
				MetaData.SortOrder = NumConstraints++;
				MetaData.bCanCollapseToTrack = false;
				Channels.Add(ConstraintChannel.ActiveChannel, MetaData, TMovieSceneExternalValue<bool>());
			}
#else
			Channels.Add(ConstraintChannel.ActiveChannel);
#endif
			
		}
	}

#if WITH_EDITOR

	F3DTransformChannelEditorData EditorData(TransformMask.GetChannels(), NumConstraints);

	AddChannelProxy(Channels, OverrideRegistry, TEXT("Location.X"), Translation[0], EditorData.MetaData[0], EditorData.ExternalValues[0]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Location.Y"), Translation[1], EditorData.MetaData[1], EditorData.ExternalValues[1]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Location.Z"), Translation[2], EditorData.MetaData[2], EditorData.ExternalValues[2]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Rotation.X"), Rotation[0],    EditorData.MetaData[3], EditorData.ExternalValues[3]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Rotation.Y"), Rotation[1],    EditorData.MetaData[4], EditorData.ExternalValues[4]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Rotation.Z"), Rotation[2],    EditorData.MetaData[5], EditorData.ExternalValues[5]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Scale.X"),    Scale[0],       EditorData.MetaData[6], EditorData.ExternalValues[6]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Scale.Y"),    Scale[1],       EditorData.MetaData[7], EditorData.ExternalValues[7]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Scale.Z"),    Scale[2],       EditorData.MetaData[8], EditorData.ExternalValues[8]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Weight"),     ManualWeight,   EditorData.MetaData[9], EditorData.WeightExternalValue);

#else

	AddChannelProxy(Channels, OverrideRegistry, TEXT("Location.X"), Translation[0]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Location.Y"), Translation[1]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Location.Z"), Translation[2]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Rotation.X"), Rotation[0]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Rotation.Y"), Rotation[1]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Rotation.Z"), Rotation[2]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Scale.X"),    Scale[0]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Scale.Y"),    Scale[1]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Scale.Z"),    Scale[2]);
	AddChannelProxy(Channels, OverrideRegistry, TEXT("Weight"),     ManualWeight);

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
	return EMovieSceneChannelProxyType::Dynamic;
}

/* UMovieSceneSection interface
 *****************************************************************************/


TSharedPtr<FStructOnScope> UMovieScene3DTransformSection::GetKeyStruct(TArrayView<const FKeyHandle> KeyHandles)
{
	if (!ChannelProxy)
	{
		GetChannelProxy();
	}

	FMovieSceneChannelHandle LocationXChannel, LocationYChannel, LocationZChannel;
	FMovieSceneChannelHandle RotationXChannel, RotationYChannel, RotationZChannel;
	FMovieSceneChannelHandle ScaleXChannel, ScaleYChannel, ScaleZChannel;

#if WITH_EDITOR
	LocationXChannel = ChannelProxy->GetChannelByName<FMovieSceneDoubleChannel>("Location.X");
	LocationYChannel = ChannelProxy->GetChannelByName<FMovieSceneDoubleChannel>("Location.Y");
	LocationZChannel = ChannelProxy->GetChannelByName<FMovieSceneDoubleChannel>("Location.Z");

	RotationXChannel = ChannelProxy->GetChannelByName<FMovieSceneDoubleChannel>("Rotation.X");
	RotationYChannel = ChannelProxy->GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Y");
	RotationZChannel = ChannelProxy->GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Z");

	ScaleXChannel = ChannelProxy->GetChannelByName<FMovieSceneDoubleChannel>("Scale.X");
	ScaleYChannel = ChannelProxy->GetChannelByName<FMovieSceneDoubleChannel>("Scale.Y");
	ScaleZChannel = ChannelProxy->GetChannelByName<FMovieSceneDoubleChannel>("Scale.Z");
#endif

	int32 AnyLocationKeys = 0;
	int32 AnyRotationKeys = 0;
	int32 AnyScaleKeys = 0;

	TOptional<TTuple<FKeyHandle, FFrameNumber>> LocationKeys[3];
	if (LocationXChannel.Get())
	{
		LocationKeys[0] = FMovieSceneChannelValueHelper::FindFirstKey(static_cast<FMovieSceneDoubleChannel*>(LocationXChannel.Get()), KeyHandles);
		AnyLocationKeys++;
	}
	if (LocationYChannel.Get())
	{
		LocationKeys[1] = FMovieSceneChannelValueHelper::FindFirstKey(static_cast<FMovieSceneDoubleChannel*>(LocationYChannel.Get()), KeyHandles);
		AnyLocationKeys++;
	}
	if (LocationZChannel.Get())
	{
		LocationKeys[2] = FMovieSceneChannelValueHelper::FindFirstKey(static_cast<FMovieSceneDoubleChannel*>(LocationZChannel.Get()), KeyHandles);
		AnyLocationKeys++;
	}

	TOptional<TTuple<FKeyHandle, FFrameNumber>> RotationKeys[3];
	if (RotationXChannel.Get())
	{
		RotationKeys[0] = FMovieSceneChannelValueHelper::FindFirstKey(static_cast<FMovieSceneDoubleChannel*>(RotationXChannel.Get()), KeyHandles);
		AnyRotationKeys++;
	}
	if (RotationYChannel.Get())
	{
		RotationKeys[1] = FMovieSceneChannelValueHelper::FindFirstKey(static_cast<FMovieSceneDoubleChannel*>(RotationYChannel.Get()), KeyHandles);
		AnyRotationKeys++;
	}
	if (RotationZChannel.Get())
	{
		RotationKeys[2] = FMovieSceneChannelValueHelper::FindFirstKey(static_cast<FMovieSceneDoubleChannel*>(RotationZChannel.Get()), KeyHandles);
		AnyRotationKeys++;
	}

	TOptional<TTuple<FKeyHandle, FFrameNumber>> ScaleKeys[3];
	if (ScaleXChannel.Get())
	{
		ScaleKeys[0] = FMovieSceneChannelValueHelper::FindFirstKey(static_cast<FMovieSceneDoubleChannel*>(ScaleXChannel.Get()), KeyHandles);
		AnyScaleKeys++;
	}
	if (ScaleYChannel.Get())
	{
		ScaleKeys[1] = FMovieSceneChannelValueHelper::FindFirstKey(static_cast<FMovieSceneDoubleChannel*>(ScaleYChannel.Get()), KeyHandles);
		AnyScaleKeys++;
	}
	if (ScaleZChannel.Get())
	{
		ScaleKeys[2] = FMovieSceneChannelValueHelper::FindFirstKey(static_cast<FMovieSceneDoubleChannel*>(ScaleZChannel.Get()), KeyHandles);
		AnyScaleKeys++;
	}

	// do we have multiple keys on multiple parts of the transform?
	if (AnyLocationKeys + AnyRotationKeys + AnyScaleKeys > 1)
	{
		TSharedRef<FStructOnScope> KeyStruct = MakeShareable(new FStructOnScope(FMovieScene3DTransformKeyStruct::StaticStruct()));
		auto Struct = (FMovieScene3DTransformKeyStruct*)KeyStruct->GetStructMemory();

		if (LocationKeys[0].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(LocationXChannel.GetChannelIndex()), &Struct->Location.X,     LocationKeys[0]));
		}
		
		if (LocationKeys[1].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(LocationYChannel.GetChannelIndex()), &Struct->Location.Y,     LocationKeys[1]));
		}
		
		if (LocationKeys[2].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(LocationZChannel.GetChannelIndex()), &Struct->Location.Z,     LocationKeys[2]));
		}

		if (RotationKeys[0].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(RotationXChannel.GetChannelIndex()), &Struct->Rotation.Roll,  RotationKeys[0]));
		}

		if (RotationKeys[1].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(RotationYChannel.GetChannelIndex()), &Struct->Rotation.Pitch, RotationKeys[1]));
		}
		
		if (RotationKeys[2].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(RotationZChannel.GetChannelIndex()), &Struct->Rotation.Yaw,   RotationKeys[2]));
		}

		if (ScaleKeys[0].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(ScaleXChannel.GetChannelIndex()), &Struct->Scale.X,        ScaleKeys[0]));
		}
		
		if (ScaleKeys[1].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(ScaleYChannel.GetChannelIndex()), &Struct->Scale.Y,        ScaleKeys[1]));
		}
		
		if (ScaleKeys[2].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(ScaleZChannel.GetChannelIndex()), &Struct->Scale.Z,        ScaleKeys[2]));
		}

		Struct->KeyStructInterop.SetStartingValues();
		Struct->Time = Struct->KeyStructInterop.GetUnifiedKeyTime().Get(0);
		return KeyStruct;
	}

	if (AnyLocationKeys)
	{
		TSharedRef<FStructOnScope> KeyStruct = MakeShareable(new FStructOnScope(FMovieScene3DLocationKeyStruct::StaticStruct()));
		auto Struct = (FMovieScene3DLocationKeyStruct*)KeyStruct->GetStructMemory();

		if (LocationKeys[0].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(LocationXChannel.GetChannelIndex()), &Struct->Location.X,     LocationKeys[0]));
		}
		
		if (LocationKeys[1].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(LocationYChannel.GetChannelIndex()), &Struct->Location.Y,     LocationKeys[1]));
		}
		
		if (LocationKeys[2].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(LocationZChannel.GetChannelIndex()), &Struct->Location.Z,     LocationKeys[2]));
		}

		Struct->KeyStructInterop.SetStartingValues();
		Struct->Time = Struct->KeyStructInterop.GetUnifiedKeyTime().Get(0);
		return KeyStruct;
	}

	if (AnyRotationKeys)
	{
		TSharedRef<FStructOnScope> KeyStruct = MakeShareable(new FStructOnScope(FMovieScene3DRotationKeyStruct::StaticStruct()));
		auto Struct = (FMovieScene3DRotationKeyStruct*)KeyStruct->GetStructMemory();

		if (RotationKeys[0].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(RotationXChannel.GetChannelIndex()), &Struct->Rotation.Roll,  RotationKeys[0]));
		}
		
		if (RotationKeys[1].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(RotationYChannel.GetChannelIndex()), &Struct->Rotation.Pitch, RotationKeys[1]));
		}
		
		if (RotationKeys[2].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(RotationZChannel.GetChannelIndex()), &Struct->Rotation.Yaw,   RotationKeys[2]));
		}

		Struct->KeyStructInterop.SetStartingValues();
		Struct->Time = Struct->KeyStructInterop.GetUnifiedKeyTime().Get(0);
		return KeyStruct;
	}

	if (AnyScaleKeys)
	{
		TSharedRef<FStructOnScope> KeyStruct = MakeShareable(new FStructOnScope(FMovieScene3DScaleKeyStruct::StaticStruct()));
		auto Struct = (FMovieScene3DScaleKeyStruct*)KeyStruct->GetStructMemory();

		if (ScaleKeys[0].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(ScaleXChannel.GetChannelIndex()), &Struct->Scale.X,        ScaleKeys[0]));
		}
		
		if (ScaleKeys[1].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(ScaleYChannel.GetChannelIndex()), &Struct->Scale.Y,        ScaleKeys[1]));
		}
		
		if (ScaleKeys[2].IsSet())
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(ScaleZChannel.GetChannelIndex()), &Struct->Scale.Z,        ScaleKeys[2]));
		}
		Struct->KeyStructInterop.SetStartingValues();
		Struct->Time = Struct->KeyStructInterop.GetUnifiedKeyTime().Get(0);
		return KeyStruct;
	}

	return nullptr;
}

bool UMovieScene3DTransformSection::GetUseQuaternionInterpolation() const
{
	return bUseQuaternionInterpolation;
}

void UMovieScene3DTransformSection::SetUseQuaternionInterpolation(bool bInUseQuaternionInterpolation)
{
	bUseQuaternionInterpolation = bInUseQuaternionInterpolation;
}

bool UMovieScene3DTransformSection::ShowCurveForChannel(const void *ChannelPtr) const
{
	if (GetUseQuaternionInterpolation())
	{
		return ChannelPtr != &Rotation[0] && ChannelPtr != &Rotation[1] && ChannelPtr != &Rotation[2];
	}
	return true;
}

void UMovieScene3DTransformSection::SetBlendType(EMovieSceneBlendType InBlendType)
{
	Super::SetBlendType(InBlendType);
	if (GetSupportedBlendTypes().Contains(InBlendType))
	{
		if (InBlendType == EMovieSceneBlendType::Absolute)
		{
			Scale[0].SetDefault(1.f);
			Scale[1].SetDefault(1.f);
			Scale[2].SetDefault(1.f);
		}
		else if (InBlendType == EMovieSceneBlendType::Additive || InBlendType == EMovieSceneBlendType::Relative)
		{
			Scale[0].SetDefault(0.f);
			Scale[1].SetDefault(0.f);
			Scale[2].SetDefault(0.f);
		}
	}
}

bool UMovieScene3DTransformSection::HasConstraintChannel(const FGuid& InGuid) const
{
	if (Constraints)
	{
		return Constraints->ConstraintsChannels.ContainsByPredicate([InGuid](const FConstraintAndActiveChannel& InChannel)
			{
				return InChannel.GetConstraint().Get() ? InChannel.GetConstraint()->ConstraintID == InGuid : false;
			});
	}
	return false;
}

FConstraintAndActiveChannel* UMovieScene3DTransformSection::GetConstraintChannel(const FGuid& InConstraintID)
{
	if (Constraints)
	{
		const int32 Index = Constraints->ConstraintsChannels.IndexOfByPredicate([InConstraintID](const FConstraintAndActiveChannel& InChannel)
			{
				return InChannel.GetConstraint().Get() ? InChannel.GetConstraint()->ConstraintID == InConstraintID : false;
			});
		return (Index != INDEX_NONE) ? &(Constraints->ConstraintsChannels[Index]) : nullptr;
	}
	return nullptr;
}

TArray<FConstraintAndActiveChannel>& UMovieScene3DTransformSection::GetConstraintsChannels() 
{
	static TArray< FConstraintAndActiveChannel> EmptyChannels;
	if (Constraints)
	{
		return Constraints->ConstraintsChannels;
	}
	return EmptyChannels;
}

const TArray<FConstraintAndActiveChannel>& UMovieScene3DTransformSection::GetConstraintsChannels() const
{
	static TArray< FConstraintAndActiveChannel> EmptyChannels;
	if (Constraints)
	{
		return Constraints->ConstraintsChannels;
	}
	return EmptyChannels;
}

void UMovieScene3DTransformSection::PostLoad()
{
	Super::PostLoad();
}	

#if WITH_EDITOR
bool UMovieScene3DTransformSection::Modify(bool bAlwaysMarkDirty)
{
	using namespace UE::MovieScene;
	if (Constraints)
	{
		Constraints->SetFlags(RF_Transactional);
		Constraints->Modify(bAlwaysMarkDirty);
	}
	bool bModified = Super::Modify(bAlwaysMarkDirty);
	
	return bModified;
}

void UMovieScene3DTransformSection::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);
}
#endif
void UMovieScene3DTransformSection::AddConstraintChannel(UTickableConstraint* InConstraint)
{
	Modify();
	if (!Constraints)
	{
		Constraints = NewObject<UMovieScene3DTransformSectionConstraints>(this, NAME_None, RF_Public| RF_Transactional);
	}
	if (InConstraint && !HasConstraintChannel(InConstraint->ConstraintID))
	{
		Constraints->SetFlags(RF_Transactional);
		Constraints->Modify();
		const int32 NewIndex = Constraints->ConstraintsChannels.Add(FConstraintAndActiveChannel(InConstraint));

		FMovieSceneConstraintChannel* ExistingChannel = &Constraints->ConstraintsChannels[NewIndex].ActiveChannel;

		//make copy that we can spawn if it doesn't exist
		//the rename changes the outer to this section (from any actor manager)
		InConstraint->Rename(nullptr, this, REN_DontCreateRedirectors | REN_AllowPackageLinkerMismatch);
	
		CacheChannelProxy();

		if (OnConstraintChannelAdded.IsBound())
		{
			OnConstraintChannelAdded.Broadcast(this, ExistingChannel);
		}
	}
}

void UMovieScene3DTransformSection::ReplaceConstraint(const FName InConstraintName, UTickableConstraint* InConstraint) 
{
	if (Constraints)
	{
		const int32 Index = Constraints->ConstraintsChannels.IndexOfByPredicate([InConstraintName](const FConstraintAndActiveChannel& InChannel)
			{
				return InChannel.GetConstraint().Get() ? InChannel.GetConstraint()->GetFName() == InConstraintName : false;
			});
		if (Index != INDEX_NONE)
		{
			Modify();
			Constraints->ConstraintsChannels[Index].GetConstraint() = InConstraint;
			CacheChannelProxy();
		}
	}
}

void UMovieScene3DTransformSection::RemoveConstraintChannel(const UTickableConstraint* InConstraint)
{
	if (bDoNotRemoveChannel == true)
	{
		return;
	}
	if (Constraints)
	{
		const int32 Index = Constraints->ConstraintsChannels.IndexOfByPredicate([InConstraint](const FConstraintAndActiveChannel& InChannel)
			{
				return InChannel.GetConstraint().Get() ? InChannel.GetConstraint().Get() == InConstraint : false;
			});

		if (Constraints->ConstraintsChannels.IsValidIndex(Index))
		{
			Modify();
			Constraints->ConstraintsChannels.RemoveAt(Index);
			CacheChannelProxy();
		}
	}
}

UMovieSceneSectionChannelOverrideRegistry* UMovieScene3DTransformSection::GetChannelOverrideRegistry(bool bCreateIfMissing)
{
	if (bCreateIfMissing && OverrideRegistry == nullptr)
	{
		OverrideRegistry = NewObject<UMovieSceneSectionChannelOverrideRegistry>(this, NAME_None, RF_Transactional);
	}
	return OverrideRegistry;
}

UE::MovieScene::FChannelOverrideProviderTraitsHandle UMovieScene3DTransformSection::GetChannelOverrideProviderTraits() const
{
	struct F3DTransformChannelOverrideProviderTraits : UE::MovieScene::FChannelOverrideProviderTraits
	{
		FName GetDefaultChannelTypeName(FName ChannelName) const override
		{
			if (ChannelName == TEXT("Weight"))
			{
				return FMovieSceneFloatChannel::StaticStruct()->GetFName();
			}
			else
			{
				return FMovieSceneDoubleChannel::StaticStruct()->GetFName();
			}
		}

		int32 GetChannelOverrideEntityID(FName ChannelName) const override
		{
			return UMovieScene3DTransformSection::ChannelOverrideNames.GetIndex(ChannelName);
		}

		FName GetChannelOverrideName(int32 EntityID) const override
		{
			return UMovieScene3DTransformSection::ChannelOverrideNames.GetChannelName(EntityID);
		}
	};

	F3DTransformChannelOverrideProviderTraits Traits;
	return UE::MovieScene::FChannelOverrideProviderTraitsHandle(Traits);
}

void UMovieScene3DTransformSection::OnConstraintsChanged()
{
	ChannelProxy = nullptr;
}

void UMovieScene3DTransformSection::OnChannelOverridesChanged()
{
	ChannelProxy = nullptr;
}


#if WITH_EDITOR

void UMovieScene3DTransformSection::PostPaste()
{
	Super::PostPaste();
	
	if (OverrideRegistry)
	{
		OverrideRegistry->OnPostPaste();
	}
	if (Constraints)
	{
		Constraints->ClearFlags(RF_Transient);
	}
}

void UMovieScene3DTransformSectionConstraints::PostEditUndo()
{
	Super::PostEditUndo();
	if (UMovieScene3DTransformSection* Section = GetTypedOuter<UMovieScene3DTransformSection>())
	{
		if (IMovieSceneConstrainedSection* ConstrainedSection = Cast<IMovieSceneConstrainedSection>(Section))
		{
			ConstrainedSection->OnConstraintsChanged();
		}
	}
}

#endif

