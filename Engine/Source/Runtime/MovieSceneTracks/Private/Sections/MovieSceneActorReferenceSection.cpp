// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneActorReferenceSection.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSequenceID.h"
#include "Tracks/MovieSceneObjectPropertyTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneActorReferenceSection)

UMovieSceneActorReferenceSection::UMovieSceneActorReferenceSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);

#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(ActorReferenceData, FMovieSceneChannelMetaData());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(ActorReferenceData);

#endif
}

void UMovieSceneActorReferenceSection::PostLoad()
{
	Super::PostLoad();

	if (ActorGuidStrings_DEPRECATED.Num())
	{
		TArray<FGuid> Guids;

		for (const FString& ActorGuidString : ActorGuidStrings_DEPRECATED)
		{
			FGuid& ActorGuid = Guids[Guids.Emplace()];
			FGuid::Parse( ActorGuidString,  ActorGuid);
		}

		if (Guids.IsValidIndex(ActorGuidIndexCurve_DEPRECATED.GetDefaultValue()))
		{
			FMovieSceneObjectBindingID DefaultValue = UE::MovieScene::FRelativeObjectBindingID(Guids[ActorGuidIndexCurve_DEPRECATED.GetDefaultValue()]);
			ActorReferenceData.SetDefault(DefaultValue);
		}

		for (auto It = ActorGuidIndexCurve_DEPRECATED.GetKeyIterator(); It; ++It)
		{
			if (ensure(Guids.IsValidIndex(It->Value)))
			{
				FMovieSceneObjectBindingID BindingID = UE::MovieScene::FRelativeObjectBindingID(Guids[It->Value]);
				ActorReferenceData.UpgradeLegacyTime(this, It->Time, BindingID);
			}
		}
	}
}

void UMovieSceneActorReferenceSection::OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	UE::MovieScene::FFixedObjectBindingID DefaultFixedBindingID = ActorReferenceData.GetDefault().Object.ResolveToFixed(LocalSequenceID, SharedPlaybackState);

	if (OldFixedToNewFixedMap.Contains(DefaultFixedBindingID))
	{
		Modify();

		const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy();
		FMovieSceneActorReferenceKey NewDefaultValue = ActorReferenceData.GetDefault();
		NewDefaultValue.Object = OldFixedToNewFixedMap[DefaultFixedBindingID].ConvertToRelative(LocalSequenceID, Hierarchy);

		ActorReferenceData.SetDefault(NewDefaultValue);
	}

	for (FMovieSceneActorReferenceKey& Key : ActorReferenceData.GetData().GetValues())
	{
		UE::MovieScene::FFixedObjectBindingID KeyFixedBindingID = Key.Object.ResolveToFixed(LocalSequenceID, SharedPlaybackState);

		if (OldFixedToNewFixedMap.Contains(KeyFixedBindingID))
		{
			Modify();

			const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy();
			Key.Object = OldFixedToNewFixedMap[KeyFixedBindingID].ConvertToRelative(LocalSequenceID, Hierarchy);
		}
	}
}

void UMovieSceneActorReferenceSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->ActorReference)
		.Add(TracksComponents->ActorReferenceChannel, &ActorReferenceData)
		.Commit(this, Params, OutImportedEntity);
}

