// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneAudioSection.h"
#include "Decorations/MovieSceneLanguagePreviewDecoration.h"
#include "Decorations/MovieSceneSectionAnchorsDecoration.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Async/Async.h"
#include "Sound/SoundBase.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNode.h"
#include "Sound/SoundNodeBranch.h"
#include "Sound/SoundNodeRandom.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Sound/SoundWave.h"
#include "UObject/SequencerObjectVersion.h"
#include "UObject/UE5ReleaseStreamObjectVersion.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "GameFramework/Actor.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "Misc/FrameRate.h"
#include "Misc/GeneratedTypeName.h"
#include "Misc/PackageName.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAudioSection)

static bool bUseRepeatingForAudioSectionsCVar = true;

void UMovieSceneAudioSection::OnUseRepeatingCVarChanged(IConsoleVariable*)
{
	check(IsInGameThread());

	ForEachObjectOfClass(UMovieSceneAudioSection::StaticClass(), [](UObject* Object)
	{
		CastChecked<UMovieSceneAudioSection>(Object)->UpdateCompletionMode();
	});
}

FAutoConsoleVariableRef CVarUseRepeatingForAudioSections(
	TEXT("Sequencer.Audio.UseRepeating"),
	bUseRepeatingForAudioSectionsCVar,
	TEXT("When enabled, audio sections use 'Repeating' instead of legacy 'Looping'."),
	FConsoleVariableDelegate::CreateStatic(&UMovieSceneAudioSection::OnUseRepeatingCVarChanged),
	ECVF_Default);

// Enables the new bPlayUntilFinished UI/runtime path, hides the legacy "When Finished" (CompletionMode) dropdown, and switches UpdateCompletionMode() to always RestoreState.
static bool bUsePlayUntilFinishedForAudioSectionsCVar = false;

void UMovieSceneAudioSection::OnUsePlayUntilFinishedCVarChanged(IConsoleVariable*)
{
	check(IsInGameThread());

	ForEachObjectOfClass(UMovieSceneAudioSection::StaticClass(), [](UObject* Object)
	{
		UMovieSceneAudioSection* Section = CastChecked<UMovieSceneAudioSection>(Object);
		Section->Modify();
		Section->UpdateCompletionMode();
		Section->MarkAsChanged();
	});
}

FAutoConsoleVariableRef CVarUsePlayUntilFinishedForAudioSections(
	TEXT("Sequencer.Audio.UsePlayUntilFinished"),
	bUsePlayUntilFinishedForAudioSectionsCVar,
	TEXT("When enabled, audio sections expose 'Play Until Finished' and hide the 'When Finished' (CompletionMode) UI."),
	FConsoleVariableDelegate::CreateStatic(&UMovieSceneAudioSection::OnUsePlayUntilFinishedCVarChanged),
	ECVF_Default);

static float DefaultStreamingPreRollSecondsCVar = 1.0f;
FAutoConsoleVariableRef CVarDefaultStreamingPreRollSeconds(
	TEXT("Sequencer.Audio.DefaultStreamingPreRollSeconds"),
	DefaultStreamingPreRollSecondsCVar,
	TEXT("Seconds of automatic preroll added to audio sections containing streaming sounds when PreRollFrames is not manually configured. Set to 0 to disable.\n"),
	ECVF_Default);

#if WITH_EDITOR

struct FAudioChannelEditorData
{
	FAudioChannelEditorData()
	{
		Data[0].SetIdentifiers("Volume", NSLOCTEXT("MovieSceneAudioSection", "SoundVolumeText", "Volume"));
		Data[1].SetIdentifiers("Pitch", NSLOCTEXT("MovieSceneAudioSection", "PitchText", "Pitch"));
		Data[2].SetIdentifiers("AttachActor", NSLOCTEXT("MovieSceneAudioSection", "AttachActorText", "Attach"));
	}

	FMovieSceneChannelMetaData Data[3];
};

#endif // WITH_EDITOR

namespace
{
	float AudioDeprecatedMagicNumber = TNumericLimits<float>::Lowest();

	FFrameNumber GetStartOffsetAtTrimTime(FQualifiedFrameTime TrimTime, FFrameNumber StartOffset, FFrameNumber StartFrame)
	{
		return StartOffset + TrimTime.Time.FrameNumber - StartFrame;
	}
}

namespace UE::MovieScene
{

// Built-in float channels used by audio tracks
enum class EBuiltinFloatChannel : uint8 { VolumeChannel, PitchChannel, Count };

// User float inputs claim the float-channel slots left over after the built-in volume and pitch slots.
constexpr int32 NumUniqueFloatChannelsPerEntity = FMovieSceneAudioInputData::NumFloatChannels - static_cast<int32>(EBuiltinFloatChannel::Count);
static_assert(NumUniqueFloatChannelsPerEntity > 0, "FMovieSceneAudioInputData::NumFloatChannels must leave room for at least one user float input after the built-in volume and pitch channels.");

/* 
 * Entity IDs are an encoded type and index, with the upper byte being the type (scalar inputs vs audio trigger),
 * and the lower 24 bits as the entity index
 */
enum class EAudioSectionEntityType : uint8 { MainEntity, InputsEntity, TriggerEntity };

uint32 EncodeEntityID(int32 InIndex, EAudioSectionEntityType InEntityType)
{
	check(InIndex >= 0 && InIndex < int32(0x00FFFFFF));
	return static_cast<uint32>(InIndex) | ((uint8)InEntityType << 24);
}
void DecodeEntityID(uint32 InEntityID, int32& OutIndex, EAudioSectionEntityType& OutEntityType)
{
	OutIndex = static_cast<int32>(InEntityID & 0x00FFFFFF);
	OutEntityType = (EAudioSectionEntityType)(InEntityID >> 24);
}

}

UMovieSceneAudioSection::UMovieSceneAudioSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	Sound = nullptr;
	StartOffset_DEPRECATED = AudioDeprecatedMagicNumber;
	AudioStartTime_DEPRECATED = AudioDeprecatedMagicNumber;
	AudioDilationFactor_DEPRECATED = AudioDeprecatedMagicNumber;
	AudioVolume_DEPRECATED = AudioDeprecatedMagicNumber;
	bLooping = true; // for legacy compatibility, this must default to true, updated only for new content or edits via SetSoundInternal()
	bRepeating = false;
	bSuppressSubtitles = false;
	bOverrideAttenuation = false;
	BlendType = EMovieSceneBlendType::Absolute;

	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ?
			EMovieSceneCompletionMode::RestoreState :
			EMovieSceneCompletionMode::ProjectDefault);
	CachedCompletionMode = EvalOptions.CompletionMode;

	SoundVolume.SetDefault(1.f);
	PitchMultiplier.SetDefault(1.f);

	UpdateCompletionMode();
}

namespace MovieSceneAudioSectionPrivate
{
	// Returns true when the cue's node tree has a deterministic playback duration. False if any
	// Random/Branch node is present: USoundNode::GetDuration() returns the MAX of all branches,
	// so the runtime branch may finish earlier than PUF's extension expects.
	static bool IsDeterministic(const USoundNode* Node)
	{
		if (!Node)
		{
			return true;
		}

		if (Node->IsA<USoundNodeRandom>() || Node->IsA<USoundNodeBranch>())
		{
			return false;
		}

		for (const USoundNode* Child : Node->ChildNodes)
		{
			if (!IsDeterministic(Child))
			{
				return false;
			}
		}

		return true;
	}

	static bool IsPlayUntilFinishedCompatible(const USoundBase* Sound)
	{
		if (!Sound || !Sound->IsOneShot())
		{
			return false;
		}

		if (Sound->IsA<USoundCue>())
		{
			const USoundCue* Cue = CastChecked<USoundCue>(Sound);

			return IsDeterministic(Cue->FirstNode);
		}

		// Sound is a OneShot
		return true;
	}

	template<typename ChannelType, typename ValueType>
	void AddInputChannels(UMovieSceneAudioSection* InSection, FMovieSceneChannelProxyData& InChannelProxyData)
	{
		InSection->ForEachInput([&InChannelProxyData](FName InName, const ChannelType& InChannel)
		{
#if WITH_EDITOR
			FMovieSceneChannelMetaData Data;
			FText TextName = FText::FromName(InName);	
			Data.SetIdentifiers(FName(InName.ToString() + GetGeneratedTypeName<ChannelType>()), TextName, TextName);
			InChannelProxyData.Add(const_cast<ChannelType&>(InChannel), Data, TMovieSceneExternalValue<ValueType>::Make());
#else //WITH_EDITOR
			InChannelProxyData.Add(const_cast<ChannelType&>(InChannel));
#endif //WITH_EDITOR

		});
	}
}

EMovieSceneChannelProxyType  UMovieSceneAudioSection::CacheChannelProxy()
{
	// Set up the channel proxy
	FMovieSceneChannelProxyData Channels;

	UMovieSceneAudioTrack* AudioTrack = Cast<UMovieSceneAudioTrack>(GetOuter());
	UMovieScene* MovieScene = AudioTrack ? Cast<UMovieScene>(AudioTrack->GetOuter()) : nullptr;
	const bool bHasAttachData = MovieScene && MovieScene->ContainsTrack(*AudioTrack);

#if WITH_EDITOR

	FAudioChannelEditorData EditorData;
	Channels.Add(SoundVolume,     EditorData.Data[0], TMovieSceneExternalValue<float>());
	Channels.Add(PitchMultiplier, EditorData.Data[1], TMovieSceneExternalValue<float>());

	if (bHasAttachData)
	{
		Channels.Add(AttachActorData, EditorData.Data[2]);
	}

#else

	Channels.Add(SoundVolume);
	Channels.Add(PitchMultiplier);
	if (bHasAttachData)
	{
		Channels.Add(AttachActorData);
	}

#endif

	using namespace MovieSceneAudioSectionPrivate;
	SetupSoundInputParameters(Sound);
	AddInputChannels<FMovieSceneFloatChannel, float>(this, Channels);
	AddInputChannels<FMovieSceneBoolChannel, bool>(this, Channels);
	AddInputChannels<FMovieSceneIntegerChannel, int32>(this, Channels);
	AddInputChannels<FMovieSceneStringChannel, FString>(this, Channels);
	AddInputChannels<FMovieSceneAudioTriggerChannel, bool>(this, Channels);

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));

	return EMovieSceneChannelProxyType::Dynamic;
}

USoundBase* UMovieSceneAudioSection::GetPlaybackSound() const
{
	return UMovieSceneLanguagePreviewDecoration::FindLocalizedAsset(Sound, this);
}

void UMovieSceneAudioSection::PopulateInitialAnchors(TMap<FGuid, FMovieSceneScalingAnchor>& OutAnchors)
{
	UMovieSceneSectionAnchorsDecoration* Decoration = FindDecoration<UMovieSceneSectionAnchorsDecoration>();
	if (Decoration && HasStartFrame() && HasEndFrame())
	{
		FFrameNumber StartTime = GetInclusiveStartFrame();
		OutAnchors.Add(Decoration->StartAnchor, FMovieSceneScalingAnchor{ StartTime, (GetExclusiveEndFrame() - StartTime).Value });
	}
}

void UMovieSceneAudioSection::PopulateAnchors(TMap<FGuid, FMovieSceneScalingAnchor>& OutAnchors)
{
	UMovieSceneSectionAnchorsDecoration* Decoration = FindDecoration<UMovieSceneSectionAnchorsDecoration>();
	if (Decoration && HasStartFrame() && HasEndFrame())
	{
		USoundBase* ReferenceSound = GetSound();
		USoundBase* PlaybackSound  = GetPlaybackSound();

		if (PlaybackSound && ReferenceSound && ReferenceSound != PlaybackSound)
		{
			const UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
			const float ReferenceDuration = MovieSceneHelpers::GetSoundDuration(ReferenceSound);
			const float PlaybackDuration  = MovieSceneHelpers::GetSoundDuration(PlaybackSound);

			FFrameNumber StartTime = GetInclusiveStartFrame();
			FFrameNumber EndTime   = GetExclusiveEndFrame();

			int32 SectionLength = FMath::CeilToInt32((EndTime.Value - StartTime.Value) * double(PlaybackDuration / ReferenceDuration));

			OutAnchors.Add(Decoration->StartAnchor, FMovieSceneScalingAnchor{ StartTime, SectionLength });
		}
	}
}

UObject* UMovieSceneAudioSection::GetSourceObject() const
{
	return GetSound();
}

void UMovieSceneAudioSection::SetupSoundInputParameters(USoundBase* InSoundBase)
{
	// Populate with defaults.

	// Don't init resources when running cook, as this can trigger 
	// registration of a MetaSound and its dependent graphs.
	// Those will instead be registered when the MetaSound itself is cooked (FMetasoundAssetBase::CookMetaSound)
	// in a way that does not deal with runtime data like this function does
	// Getting the default parameters and the rest of the function are 
	// dependent on that runtime data and don't need to be cooked
	if (InSoundBase && !IsRunningCookCommandlet())
	{
		InSoundBase->InitResources();

		TArray<FAudioParameter> DefaultParams;
		InSoundBase->GetAllDefaultParameters(DefaultParams);

		TSet<FName> OrphanedFloatInputs;
		Inputs_Float.GetKeys(OrphanedFloatInputs);
		TSet<FName> OrphanedTriggerInputs;
		Inputs_Trigger.GetKeys(OrphanedTriggerInputs);
		TSet<FName> OrphanedBoolInputs;
		Inputs_Bool.GetKeys(OrphanedBoolInputs);
		TSet<FName> OrphanedIntInputs;
		Inputs_Int.GetKeys(OrphanedIntInputs);
		TSet<FName> OrphanedStringInputs;
		Inputs_String.GetKeys(OrphanedStringInputs);

		for (const FAudioParameter& Param : DefaultParams)
		{
			switch (Param.ParamType)
			{
			case EAudioParameterType::Float:
			{
				Inputs_Float.FindOrAdd(Param.ParamName, FMovieSceneFloatChannel{}).SetDefault(Param.FloatParam);
				OrphanedFloatInputs.Remove(Param.ParamName);
				break;
			}
			case EAudioParameterType::Trigger:
			{
				Inputs_Trigger.FindOrAdd(Param.ParamName, FMovieSceneAudioTriggerChannel{});
				OrphanedTriggerInputs.Remove(Param.ParamName);
				break;
			}
			case EAudioParameterType::Boolean:
			{
				Inputs_Bool.FindOrAdd(Param.ParamName, FMovieSceneBoolChannel{}).SetDefault(Param.BoolParam);
				OrphanedBoolInputs.Remove(Param.ParamName);
				break;
			}
			case EAudioParameterType::Integer:
			{
				Inputs_Int.FindOrAdd(Param.ParamName, FMovieSceneIntegerChannel{}).SetDefault(Param.IntParam);
				OrphanedIntInputs.Remove(Param.ParamName);
				break;
			}
			case EAudioParameterType::String:
			{
				Inputs_String.FindOrAdd(Param.ParamName, FMovieSceneStringChannel{}).SetDefault(Param.StringParam);
				OrphanedStringInputs.Remove(Param.ParamName);
				break;
			}
			default:
				// Not supported yet.
				break;
			}
		}

		for (const FName& Name : OrphanedFloatInputs)
		{
			Inputs_Float.Remove(Name);
		}
		for (const FName& Name : OrphanedTriggerInputs)
		{
			Inputs_Trigger.Remove(Name);
		}
		for (const FName& Name : OrphanedBoolInputs)
		{
			Inputs_Bool.Remove(Name);
		}
		for (const FName& Name : OrphanedIntInputs)
		{
			Inputs_Int.Remove(Name);
		}
		for (const FName& Name : OrphanedStringInputs)
		{
			Inputs_String.Remove(Name);
		}
	}
}

void UMovieSceneAudioSection::SetLooping(bool bInLooping)
{
	bLooping = bInLooping;
	UpdateCompletionMode();
}

void UMovieSceneAudioSection::SetRepeating(bool bInRepeating)
{
	bRepeating = bInRepeating;
	UpdateCompletionMode();
}

bool UMovieSceneAudioSection::IsRepeating() const
{
	return bUseRepeatingForAudioSectionsCVar ? bRepeating : bLooping;
}

bool UMovieSceneAudioSection::IsSoundLoopingAsset() const
{
	return Sound && !Sound->IsOneShot();
}

bool UMovieSceneAudioSection::IsLoopingOrRepeating() const
{
	if (IsSoundLoopingAsset())
	{
		return true;
	}

	return IsRepeating();
}

bool UMovieSceneAudioSection::IsRepeatingCVarEnabled()
{
	return bUseRepeatingForAudioSectionsCVar;
}

bool UMovieSceneAudioSection::IsPlayUntilFinishedEditable() const
{
	return bUsePlayUntilFinishedForAudioSectionsCVar && bSoundIsPlayUntilFinishedCompatible;
}

bool UMovieSceneAudioSection::IsPlayUntilFinishedActive() const
{
	return bUsePlayUntilFinishedForAudioSectionsCVar && bPlayUntilFinished;
}

void UMovieSceneAudioSection::SetPlayUntilFinished(bool bInPlayUntilFinished)
{
	if (bPlayUntilFinished == bInPlayUntilFinished)
	{
		return;
	}

	Modify();
	bPlayUntilFinished = bInPlayUntilFinished;
	MarkAsChanged();
}

TOptional<FFrameNumber> UMovieSceneAudioSection::ComputeDeterministicSoundEndFrame() const
{
	if (!Sound || !Sound->IsOneShot() || Sound->IsProcedurallyGenerated() || !HasStartFrame())
	{
		return TOptional<FFrameNumber>();
	}

	const UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return TOptional<FFrameNumber>();
	}

	const float SoundDuration = MovieSceneHelpers::GetSoundDuration(Sound);
	if (SoundDuration <= 0.f || SoundDuration == INDEFINITELY_LOOPING_DURATION)
	{
		return TOptional<FFrameNumber>();
	}

	const FFrameRate FrameRate = MovieScene->GetTickResolution();
	return GetInclusiveStartFrame() + (SoundDuration * FrameRate).FloorToFrame() - StartFrameOffset;
}

void UMovieSceneAudioSection::UpdateCompletionMode()
{
	EvalOptions.bCanEditCompletionMode = !bUsePlayUntilFinishedForAudioSectionsCVar;

	if (bUsePlayUntilFinishedForAudioSectionsCVar)
	{
		// PlayUntilFinished: cleanup is driven by the (possibly extended) evaluation range, never KeepState.
		EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
		return;
	}

	// All looping/repeating sounds are set to restore state to prevent "stuck on" sounds
	if (IsLoopingOrRepeating())
	{
		EvalOptions.CompletionMode = EMovieSceneCompletionMode::RestoreState;
	}
	else
	{
		EvalOptions.CompletionMode = CachedCompletionMode;
	}
}

void UMovieSceneAudioSection::RefreshPlayUntilFinishedCompatibilityAndMigrate()
{
	bSoundIsPlayUntilFinishedCompatible = MovieSceneAudioSectionPrivate::IsPlayUntilFinishedCompatible(Sound);

	if (GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID)
		< FUE5ReleaseStreamObjectVersion::AudioSectionPlayUntilFinished)
	{
		// PlayUntilFinished compatible sections previously saved with KeepState were effectively asking for "play until finished" semantics and are set to true. 
		// Non-PlayUntilFinished compatible sections or sections with RestoreState or ProjectDefault migrate to false.
		// Part of a property change, therefore doesn't need to use SetPlayUntilFinished() to modify the property and mark as changed
		bPlayUntilFinished = (Sound && bSoundIsPlayUntilFinishedCompatible && EvalOptions.CompletionMode == EMovieSceneCompletionMode::KeepState);
	}

	UpdateCompletionMode();
}

TRange<FFrameNumber> UMovieSceneAudioSection::ComputePlayUntilFinishedRange() const
{
	if (!bUsePlayUntilFinishedForAudioSectionsCVar
		|| !bPlayUntilFinished
		|| !bSoundIsPlayUntilFinishedCompatible
		|| !HasStartFrame()
		|| !HasEndFrame())
	{
		return GetRange();
	}

	const UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return GetRange();
	}

	FFrameNumber ExtendedEnd = GetExclusiveEndFrame();

	if (Sound && !Sound->IsProcedurallyGenerated())
	{
		// Deterministic duration (one-shot SoundWave or non-branching one-shot SoundCue):
		// extend to SectionStart + GetDuration().
		// Limitation: PitchMultiplier != 1.0 changes effective playback duration but the
		// extension uses unscaled SoundDuration; section sits silent for the surplus tail.
		if (TOptional<FFrameNumber> SoundEndFrame = ComputeDeterministicSoundEndFrame())
		{
			ExtendedEnd = FMath::Max(ExtendedEnd, *SoundEndFrame);
		}
	}
	else
	{
		// One-shot MetaSound: Extend to playback range end; the MetaSound fires OnFinished when done
		// and the audio system idles. RestoreState cleans up at the extended end.
		const TRange<FFrameNumber> PlaybackRange = MovieScene->GetPlaybackRange();
		if (PlaybackRange.HasUpperBound())
		{
			ExtendedEnd = FMath::Max(ExtendedEnd, PlaybackRange.GetUpperBoundValue());
		}
	}

	return TRange<FFrameNumber>(GetInclusiveStartFrame(), ExtendedEnd);
}

TOptional<FFrameTime> UMovieSceneAudioSection::GetOffsetTime() const
{
	return TOptional<FFrameTime>(StartFrameOffset);
}

void UMovieSceneAudioSection::MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate)
{
	if (StartFrameOffset.Value > 0)
	{
		FFrameNumber NewStartFrameOffset = ConvertFrameTime(FFrameTime(StartFrameOffset), SourceRate, DestinationRate).FloorToFrame();
		StartFrameOffset = NewStartFrameOffset;
	}
}

void UMovieSceneAudioSection::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FUE5ReleaseStreamObjectVersion::GUID);
	Super::Serialize(Ar);
}

void UMovieSceneAudioSection::PostLoad()
{
	Super::PostLoad();

	if (AudioDilationFactor_DEPRECATED != AudioDeprecatedMagicNumber)
	{
		PitchMultiplier.SetDefault(AudioDilationFactor_DEPRECATED);

		AudioDilationFactor_DEPRECATED = AudioDeprecatedMagicNumber;
	}

	if (AudioVolume_DEPRECATED != AudioDeprecatedMagicNumber)
	{
		SoundVolume.SetDefault(AudioVolume_DEPRECATED);

		AudioVolume_DEPRECATED = AudioDeprecatedMagicNumber;
	}

	TOptional<double> StartOffsetToUpgrade;
	if (AudioStartTime_DEPRECATED != AudioDeprecatedMagicNumber)
	{
		// Previously, start time in relation to the sequence. Start time was used to calculate the offset into the 
		// clip at the start of the section evaluation as such: Section Start Time - Start Time. 
		if (AudioStartTime_DEPRECATED != 0.f && HasStartFrame())
		{
			StartOffsetToUpgrade = GetInclusiveStartFrame() / GetTypedOuter<UMovieScene>()->GetTickResolution() - AudioStartTime_DEPRECATED;
		}
		AudioStartTime_DEPRECATED = AudioDeprecatedMagicNumber;
	}

	if (StartOffset_DEPRECATED != AudioDeprecatedMagicNumber)
	{
		StartOffsetToUpgrade = StartOffset_DEPRECATED;

		StartOffset_DEPRECATED = AudioDeprecatedMagicNumber;
	}

	FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();

	if (StartOffsetToUpgrade.IsSet())
	{
		FFrameRate DisplayRate = GetTypedOuter<UMovieScene>()->GetDisplayRate();
		FFrameRate TickResolution = GetTypedOuter<UMovieScene>()->GetTickResolution();

		StartFrameOffset = ConvertFrameTime(FFrameTime::FromDecimal(DisplayRate.AsDecimal() * StartOffsetToUpgrade.GetValue()), DisplayRate, TickResolution).FrameNumber;
	}

	if (GetLinkerCustomVersion(FUE5ReleaseStreamObjectVersion::GUID)
		< FUE5ReleaseStreamObjectVersion::AudioSectionRepeating)
	{
		// bLooping = true was using "repeating" behavior - transfer intent
		bRepeating = bLooping;
	}

	if (Sound && !IsInGameThread())
	{
		// IsPlayUntilFinishedCompatible calls IsOneShot, which can drive ConditionalPostLoad on child
		// nodes for certain USoundBase subclasses (e.g. USoundCue) - unsafe under async loading.
		// Defer the whole refresh-and-migrate step to the game thread.
		TWeakObjectPtr<UMovieSceneAudioSection> WeakThis(this);
		AsyncTask(ENamedThreads::GameThread, [WeakThis]()
		{
			if (TStrongObjectPtr<UMovieSceneAudioSection> Section = WeakThis.Pin())
			{
				Section->RefreshPlayUntilFinishedCompatibilityAndMigrate();
			}
		});
	}
	else
	{
		RefreshPlayUntilFinishedCompatibilityAndMigrate();
	}
}

#if WITH_EDITOR
void UMovieSceneAudioSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);


	if (PropertyChangedEvent.MemberProperty)
	{
		const FName PropertyName = PropertyChangedEvent.MemberProperty->GetFName();
		const FName EvalOptionsName = GET_MEMBER_NAME_CHECKED(UMovieSceneSection, EvalOptions);
		if (PropertyName == EvalOptionsName)
		{
			// Cache user intended changes to the completion mode
			CachedCompletionMode = EvalOptions.CompletionMode;
		}
	}

	// Update completion Mode to prevent looping sounds from playing indefinitely.
	// Also, if the sound is changed, invalidate channel proxy to regenerate input tracks.
	// This needs to be done even if Sound is nullptr.
	// Also called for undo via UMovieSceneSignedObject::PostEditUndo, where PropertyChangedEvent.MemberProperty is nullptr.
	const bool bIsSoundProperty = PropertyChangedEvent.MemberProperty && PropertyChangedEvent.MemberProperty->GetFName() == GET_MEMBER_NAME_CHECKED(UMovieSceneAudioSection, Sound);
	const bool bInvalidateChannelProxy = ( !PropertyChangedEvent.MemberProperty || bIsSoundProperty );
	SetSoundInternal(Sound, bInvalidateChannelProxy, bIsSoundProperty); // calls UpdateCompletionMode() to prevent looping sounds from using Keep State

	if (Sound)
	{
		// Clamp the StartOffset to the length of the SoundWave when not looping
		// Setting the StartOffset to the size of the SoundWave will result in silence, irrespective of the size of the section
		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		const bool bHasDeterministicDuration = !IsRepeating() && !Sound->IsProcedurallyGenerated() && Sound->IsOneShot() && MovieScene;
		if (bHasDeterministicDuration)
		{
			// SoundDuration is local-specifc, meaning it can vary based on the value of GetCurrentLanguage()
			const float SoundDuration = MovieSceneHelpers::GetSoundDuration(Sound);
			const FFrameRate FrameRate = MovieScene->GetTickResolution();
			const FFrameNumber MaxFrame = (SoundDuration * FrameRate).FloorToFrame();

			StartFrameOffset = FMath::Min(StartFrameOffset, MaxFrame);
		}
	}
}
#endif // WITH_EDITOR
	
TOptional<TRange<FFrameNumber> > UMovieSceneAudioSection::GetAutoSizeRange() const
{
	if (!Sound)
	{
		return TRange<FFrameNumber>();
	}

	const FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

	// determine initial duration
	// @todo Once we have infinite sections, we can remove this
	// @todo ^^ Why? Infinte sections would mean there's no starting time?
	FFrameTime DurationToUse = 1.f * FrameRate; // if all else fails, use 1 second duration

	// Only one-shot non-procedural sounds have a known duration
	if (Sound->IsOneShot() && !Sound->IsProcedurallyGenerated())
	{
		const float SoundDuration = MovieSceneHelpers::GetSoundDuration(Sound);
		// This should not hit if the sound has returned true for IsOneShot()
		check(SoundDuration != INDEFINITELY_LOOPING_DURATION);
		if (SoundDuration > 0)
		{
			DurationToUse = FMath::Max(SoundDuration * FrameRate - StartFrameOffset, FFrameTime(1));
		}
	}

	const int32 IFrameNumber = DurationToUse.FrameNumber.Value + static_cast<int32>(DurationToUse.GetSubFrame() + 0.5f);
	return TRange<FFrameNumber>(GetInclusiveStartFrame(), GetInclusiveStartFrame() + IFrameNumber);
}

	
void UMovieSceneAudioSection::TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys)
{
	SetFlags(RF_Transactional);

	if (TryModify())
	{
		// Capture pre-trim end so the PlayUntilFinished auto-disable below can detect the threshold transition.
		const FFrameNumber PreTrimEnd = HasEndFrame() ? GetExclusiveEndFrame() : FFrameNumber(MAX_int32);

		if (bTrimLeft)
		{
			StartFrameOffset = HasStartFrame() ? GetStartOffsetAtTrimTime(TrimTime, StartFrameOffset, GetInclusiveStartFrame()) : 0;
		}

		Super::TrimSection(TrimTime, bTrimLeft, bDeleteKeys);

		// Auto-disable PlayUntilFinished only when this trim crosses the section below the sound's natural duration.
		// Trims of an already-too-short section (e.g. user re-enabled PUF manually) leave it alone.
		// ComputeDeterministicSoundEndFrame() is shared with ComputePlayUntilFinishedRange so the
		// auto-disable threshold matches the extension threshold exactly.
		if (!bTrimLeft && bPlayUntilFinished && HasEndFrame())
		{
			if (TOptional<FFrameNumber> SoundEndFrame = ComputeDeterministicSoundEndFrame())
			{
				if (PreTrimEnd >= *SoundEndFrame && GetExclusiveEndFrame() < *SoundEndFrame)
				{
					// Part of a property change, therefore doesn't need to use SetPlayUntilFinished() to modify the property and mark as changed
					bPlayUntilFinished = false;
				}
			}
		}
	}
}

UMovieSceneSection* UMovieSceneAudioSection::SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys)
{
	const FFrameNumber InitialStartFrameOffset = StartFrameOffset;

	const FFrameNumber NewOffset = HasStartFrame() ? GetStartOffsetAtTrimTime(SplitTime, StartFrameOffset, GetInclusiveStartFrame()) : 0;

	UMovieSceneSection* NewSection = Super::SplitSection(SplitTime, bDeleteKeys);
	if (NewSection != nullptr)
	{
		UMovieSceneAudioSection* NewAudioSection = Cast<UMovieSceneAudioSection>(NewSection);
		NewAudioSection->StartFrameOffset = NewOffset;
	}

	// Restore original offset modified by splitting
	StartFrameOffset = InitialStartFrameOffset;

	return NewSection;
}

void UMovieSceneAudioSection::SetSound(USoundBase* InSound)
{
	SetSoundInternal(InSound, true, true); // For new sections, it's called from UMovieSceneAudioTrack::AddNewSoundOnRow().
}

void UMovieSceneAudioSection::SetSoundInternal(USoundBase* InSound, const bool bInvalidateChannelProxy, const bool bSoundChanged)
{
	// Sound is a UPROPERTY and can be set directly via FObjectPropertyBase::SetObjectPtrPropertyValue
	// This function is called after that fact via PostEditChangeProperty().
	// For new sections, it's called from UMovieSceneAudioTrack::AddNewSoundOnRow().

	Sound = InSound;

	// Cache PlayUntilFinished-compatibility now while we're already paying for the SoundCue node walk -- IsPlayUntilFinishedEditable()
	// is called per-tick by the property grid and ComputePlayUntilFinishedRange() runs every evaluation cycle.
	bSoundIsPlayUntilFinishedCompatible = MovieSceneAudioSectionPrivate::IsPlayUntilFinishedCompatible(Sound);

	if (bSoundChanged && InSound)
	{
		// Derive looping/repeating state from the new sound asset.
		// Only do this when the sound itself changes to prevent user edits to bLooping/bRepeating
		// from being overwritten (e.g. on undo of unrelated properties like StartFrameOffset).
		bLooping = !Sound->IsOneShot();

		// Default Play Until Finished from sound type so an asset swap from one-shot to looping
		// auto-clears the bool (the EditCondition then greys the checkbox).
		if (bUsePlayUntilFinishedForAudioSectionsCVar)
		{
			// Part of a property change, therefore doesn't need to use SetPlayUntilFinished() to modify the property and mark as changed
			bPlayUntilFinished &= bSoundIsPlayUntilFinishedCompatible;
		}

		// We don't alter bRepeating here to preserve existing intent for the track
	}

	if (bInvalidateChannelProxy)
	{
		InvalidateChannelProxy();
	}

	UpdateCompletionMode();
}

void UMovieSceneAudioSection::SetDefaultActorKey(const FMovieSceneObjectBindingID& NewBinding)
{
	Modify();
	FMovieSceneActorReferenceKey Key = AttachActorData.GetDefault();
	Key.Object = NewBinding;

	AttachActorData.SetDefault(Key);
}

USceneComponent* UMovieSceneAudioSection::GetAttachComponent(const AActor* InParentActor, const FMovieSceneActorReferenceKey& Key) const
{
	FName AttachComponentName = Key.ComponentName;
	FName AttachSocketName = Key.SocketName;

	if (AttachSocketName != NAME_None)
	{
		if (AttachComponentName != NAME_None)
		{
			TInlineComponentArray<USceneComponent*> PotentialAttachComponents(InParentActor);
			for (USceneComponent* PotentialAttachComponent : PotentialAttachComponents)
			{
				if (PotentialAttachComponent->GetFName() == AttachComponentName && PotentialAttachComponent->DoesSocketExist(AttachSocketName))
				{
					return PotentialAttachComponent;
				}
			}
		}
		else if (InParentActor->GetRootComponent()->DoesSocketExist(AttachSocketName))
		{
			return InParentActor->GetRootComponent();
		}
	}
	else if (AttachComponentName != NAME_None)
	{
		TInlineComponentArray<USceneComponent*> PotentialAttachComponents(InParentActor);
		for (USceneComponent* PotentialAttachComponent : PotentialAttachComponents)
		{
			if (PotentialAttachComponent->GetFName() == AttachComponentName)
			{
				return PotentialAttachComponent;
			}
		}
	}

	if (InParentActor->GetDefaultAttachComponent())
	{
		return InParentActor->GetDefaultAttachComponent();
	}
	else
	{
		return InParentActor->GetRootComponent();
	}
}

bool UMovieSceneAudioSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	using namespace UE::MovieScene;

	// For sections containing streaming sounds, opt into sub-sequence preroll so the
	// entity ledger won't cull this section when an outer sequence prerolls into it.
	FMovieSceneEvaluationFieldEntityMetaData MetaData = InMetaData;
	if (!MetaData.bEvaluateInSequencePreRoll)
	{
		MetaData.bEvaluateInSequencePreRoll = true;
	}

	int32 MetaDataIndex = OutFieldBuilder->AddMetaData(MetaData);

	// When PlayUntilFinished is on, extend the main + inputs entity range past the visual section
	// end so the AudioComponent stays under sequencer control until the sound finishes naturally.
	// Trigger entities continue to use EffectiveRange below so triggers don't re-fire during the tail.
	const TRange<FFrameNumber> ExtendedRange = ComputePlayUntilFinishedRange();
	
	const bool bIsATrailingSegment = TRangeBound<FFrameNumber>::MaxUpper(EffectiveRange.GetUpperBound(), ExtendedRange.GetUpperBound())  == ExtendedRange.GetUpperBound();
	const bool bExtendsThisSegment = ExtendedRange.GetUpperBound() != EffectiveRange.GetUpperBound() && bIsATrailingSegment
				&& IsPlayUntilFinishedActive();

	const TRange<FFrameNumber> MainEntityRange = bExtendsThisSegment
		? TRange<FFrameNumber>(EffectiveRange.GetLowerBound(), ExtendedRange.GetUpperBound())
		: EffectiveRange;

	// Add the default entity first.
	int32 MainEntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(0, EAudioSectionEntityType::MainEntity));
	OutFieldBuilder->AddPersistentEntity(MainEntityRange, MainEntityIndex, MetaDataIndex);

	// Preroll for streaming sounds: register the main entity for a window before the section start, 
	// so the audio system primes the first streaming chunk before Play() is called.
	// Only add this when PreRollFrames hasn't been set manually by the designer.
	if (DefaultStreamingPreRollSecondsCVar > 0.f
		&& !EnumHasAnyFlags(InMetaData.Flags, ESectionEvaluationFlags::PreRoll)
		&& GetPreRollFrames() == 0
		&& HasStartFrame())
	{
		if (const UMovieScene* MovieScene = GetTypedOuter<UMovieScene>())
		{
			const FFrameRate TickResolution = MovieScene->GetTickResolution();
			const FFrameNumber PreRollDuration = (DefaultStreamingPreRollSecondsCVar * TickResolution).FloorToFrame();
			if (PreRollDuration.Value > 0)
			{
				const FFrameNumber SectionStart = GetInclusiveStartFrame();
				const TRange<FFrameNumber> AutoPreRollRange(
					TRangeBound<FFrameNumber>::Inclusive(SectionStart - PreRollDuration),
					TRangeBound<FFrameNumber>::Exclusive(SectionStart));

				FMovieSceneEvaluationFieldEntityMetaData PreRollMetaData = MetaData;
				PreRollMetaData.Flags |= ESectionEvaluationFlags::PreRoll;

				const int32 PreRollMetaDataIndex = OutFieldBuilder->AddMetaData(PreRollMetaData);
				OutFieldBuilder->AddPersistentEntity(AutoPreRollRange, MainEntityIndex, PreRollMetaDataIndex);
			}
		}
	}

	// Each InputsEntity packs up to NumUniqueFloatChannelsPerEntity user float inputs (slots reserved for
	// SoundVolume and PitchMultiplier do not count) plus at most one string/bool/int channel per entity,
	// so the entity count must cover the largest of those four input maps.
	int32 NumInputDataEntities = FMath::DivideAndRoundUp(Inputs_Float.Num(), NumUniqueFloatChannelsPerEntity);
	NumInputDataEntities = FMath::Max(NumInputDataEntities, Inputs_String.Num());
	NumInputDataEntities = FMath::Max(NumInputDataEntities, Inputs_Bool.Num());
	NumInputDataEntities = FMath::Max(NumInputDataEntities, Inputs_Int.Num());

	// Add these extra entities to the evaluation field.
	for (int32 InputDataEntity = 0; InputDataEntity < NumInputDataEntities; ++InputDataEntity)
	{
		int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(InputDataEntity, EAudioSectionEntityType::InputsEntity));
		OutFieldBuilder->AddPersistentEntity(MainEntityRange, EntityIndex, MetaDataIndex);
	}

	// Audio triggers are added differently, as one-shot entities.
	TArray<FName> AudioTriggerNames;
	Inputs_Trigger.GetKeys(AudioTriggerNames);
	AudioTriggerNames.Sort(FNameLexicalLess());
	for (int32 TriggerIndex = 0; TriggerIndex < AudioTriggerNames.Num(); ++TriggerIndex)
	{
		FMovieSceneAudioTriggerChannel& TriggerChannel = Inputs_Trigger[AudioTriggerNames[TriggerIndex]];
		TArrayView<const FFrameNumber> Times = TriggerChannel.GetTimes();
		for (int32 Index = 0; Index < Times.Num(); ++Index)
		{
			if (EffectiveRange.Contains(Times[Index]))
			{
				TRange<FFrameNumber> TriggerRange(Times[Index]);
				int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(TriggerIndex, EAudioSectionEntityType::TriggerEntity));
				OutFieldBuilder->AddOneShotEntity(TriggerRange, EntityIndex, MetaDataIndex);
			}
		}
	}

	// Return true to indicate we've done everything ourselves.
	return true;
}

void UMovieSceneAudioSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!Sound)
	{
		return;
	}

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	const FGuid ObjectBindingID = Params.GetObjectBindingID();

	int32 EntityIndex;
	EAudioSectionEntityType EntityType;
	DecodeEntityID(Params.EntityID, EntityIndex, EntityType);

	if (EntityType == EAudioSectionEntityType::MainEntity)
	{
		const bool bHasSectionPreRoll = Params.EntityMetaData && EnumHasAnyFlags(Params.EntityMetaData->Flags, ESectionEvaluationFlags::PreRoll);
		const bool bInPreRoll = bHasSectionPreRoll;

		// Default entity... we add the main audio component data, plus the volume and pitch channels.
		OutImportedEntity->AddBuilder(
			FEntityBuilder()
			.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
			.AddTagConditional(BuiltInComponents->Tags.Root, !ObjectBindingID.IsValid())
			.Add(TrackComponents->Audio, FMovieSceneAudioComponentData{ this })
			.Add(BuiltInComponents->FloatChannel[static_cast<int32>(EBuiltinFloatChannel::VolumeChannel)], &SoundVolume)
			.Add(BuiltInComponents->FloatChannel[static_cast<int32>(EBuiltinFloatChannel::PitchChannel)], &PitchMultiplier)
			.AddTagConditional(BuiltInComponents->Tags.PreRoll, bInPreRoll)
		);
	}
	else if (EntityType == EAudioSectionEntityType::InputsEntity)
	{
		// Additional entities for custom audio input values.
		TArray<FName> InputNames;
		FMovieSceneAudioInputData InputData;

		// Slot 0 and slot 1 are reserved for SoundVolume and PitchMultiplier on the MainEntity, so user float
		// inputs are written into slots [Count..NumFloatChannels) and packed across InputsEntity records.
		int32 FloatInputStartIndex = EntityIndex * NumUniqueFloatChannelsPerEntity;
		if (FloatInputStartIndex < Inputs_Float.Num())
		{
			int32 FloatInputNum = FMath::Min(NumUniqueFloatChannelsPerEntity, Inputs_Float.Num() - FloatInputStartIndex);
			Inputs_Float.GetKeys(InputNames);
			for (int32 Offset = 0; Offset < FloatInputNum; ++Offset)
			{
				InputData.FloatInputs[Offset + static_cast<int32>(EBuiltinFloatChannel::Count)] = InputNames[FloatInputStartIndex + Offset];
			}
		}

		// Other inputs can only be added once per entity, so add one of each type that exists.
		int32 OtherInputStartIndex = EntityIndex;
		if (OtherInputStartIndex < Inputs_String.Num())
		{
			InputNames.Reset();
			Inputs_String.GetKeys(InputNames);
			InputData.StringInput = InputNames[OtherInputStartIndex];
		}
		if (OtherInputStartIndex < Inputs_Bool.Num())
		{
			InputNames.Reset();
			Inputs_Bool.GetKeys(InputNames);
			InputData.BoolInput = InputNames[OtherInputStartIndex];
		}
		if (OtherInputStartIndex < Inputs_Int.Num())
		{
			InputNames.Reset();
			Inputs_Int.GetKeys(InputNames);
			InputData.IntInput = InputNames[OtherInputStartIndex];
		}

		// Make this additional entity by adding the component that specifies what audio input channels
		// are present, plus all of these channels.
		OutImportedEntity->AddBuilder(
			FEntityBuilder()
			.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
			.AddTagConditional(BuiltInComponents->Tags.Root, !ObjectBindingID.IsValid())
			.Add(TrackComponents->Audio, FMovieSceneAudioComponentData{ this })
			.Add(TrackComponents->AudioInputs, InputData)
		);
		for (int32 Index = static_cast<int32>(EBuiltinFloatChannel::Count); Index < FMovieSceneAudioInputData::NumFloatChannels; ++Index)
		{
			FName InputName = InputData.FloatInputs[Index];
			if (!InputName.IsNone())
			{
				OutImportedEntity->AddBuilder(
					FEntityBuilder()
					.Add(BuiltInComponents->FloatChannel[Index], &Inputs_Float[InputName])
				);
			}
		}
		if (!InputData.StringInput.IsNone())
		{
			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(BuiltInComponents->StringChannel, &Inputs_String[InputData.StringInput])
			);
		}
		if (!InputData.BoolInput.IsNone())
		{
			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(BuiltInComponents->BoolChannel, &Inputs_Bool[InputData.BoolInput])
			);
		}
		if (!InputData.IntInput.IsNone())
		{
			OutImportedEntity->AddBuilder(
				FEntityBuilder()
				.Add(BuiltInComponents->IntegerChannel, &Inputs_Int[InputData.IntInput])
			);
		}
	}
	else if (EntityType == EAudioSectionEntityType::TriggerEntity)
	{
		// Additional one-shot entities for audio triggers.
		// The decoded index is the index of the name in the triggers map.
		TArray<FName> AudioTriggerNames;
		Inputs_Trigger.GetKeys(AudioTriggerNames);
		AudioTriggerNames.Sort(FNameLexicalLess());
		OutImportedEntity->AddBuilder(
			FEntityBuilder()
			.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
			.AddTagConditional(BuiltInComponents->Tags.Root, !ObjectBindingID.IsValid())
			.Add(TrackComponents->Audio, FMovieSceneAudioComponentData{ this })
			.Add(TrackComponents->AudioTriggerName, AudioTriggerNames[EntityIndex])
		);
	}
}
