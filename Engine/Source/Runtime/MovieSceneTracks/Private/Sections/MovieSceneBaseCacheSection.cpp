// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneBaseCacheSection.h"
#include "MovieScene.h"
#include "Evaluation/MovieSceneBaseCacheTemplate.h"
#include "MovieSceneTimeHelpers.h"
#include "Misc/QualifiedFrameTime.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBaseCacheSection)

FMovieSceneBaseCacheParams::FMovieSceneBaseCacheParams()
{
	PlayRate = 1.f;
	bReverse = false;
}

UMovieSceneBaseCacheSection::UMovieSceneBaseCacheSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer), ParamsPtr(nullptr)
{
	BlendType = EMovieSceneBlendType::Absolute;
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);

#if WITH_EDITOR
      	InitializePlayRate();
      #endif
}

TOptional<FFrameTime> UMovieSceneBaseCacheSection::GetOffsetTime() const
{
	return ParamsPtr ? TOptional<FFrameTime>(ParamsPtr->FirstLoopStartFrameOffset) : TOptional<FFrameTime>(FFrameNumber());
}

TOptional<TRange<FFrameNumber> > UMovieSceneBaseCacheSection::GetAutoSizeRange() const
{
	if(ParamsPtr)
	{
		const FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();
		const FFrameTime AnimationLength = ParamsPtr->GetSequenceLength() * FrameRate;
		const int32 IFrameNumber = AnimationLength.FrameNumber.Value + static_cast<int32>(AnimationLength.GetSubFrame() + 0.5f);

		return TRange<FFrameNumber>(GetInclusiveStartFrame(), GetInclusiveStartFrame() + IFrameNumber + 1);
	}
	return TRange<FFrameNumber>(0, 1);
}

void UMovieSceneBaseCacheSection::TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys)
{
	SetFlags(RF_Transactional);

	if (TryModify() && ParamsPtr)
	{
		if (bTrimLeft)
		{
			FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

			ParamsPtr->FirstLoopStartFrameOffset = HasStartFrame() ? GetFirstLoopStartOffsetAtTrimTime(TrimTime, *ParamsPtr, GetInclusiveStartFrame(), FrameRate) : 0;
		}

		Super::TrimSection(TrimTime, bTrimLeft, bDeleteKeys);
	}
}

UMovieSceneSection* UMovieSceneBaseCacheSection::SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys)
{
	if(ParamsPtr)
	{
		const FFrameNumber InitialFirstLoopStartFrameOffset = ParamsPtr->FirstLoopStartFrameOffset;

		FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

		const FFrameNumber NewOffset = HasStartFrame() ? GetFirstLoopStartOffsetAtTrimTime(SplitTime, *ParamsPtr, GetInclusiveStartFrame(), FrameRate) : 0;

		UMovieSceneSection* NewSection = Super::SplitSection(SplitTime, bDeleteKeys);
		if (NewSection != nullptr)
		{
			UMovieSceneBaseCacheSection* NewChaosCacheSection = Cast<UMovieSceneBaseCacheSection>(NewSection);
			NewChaosCacheSection->ParamsPtr->FirstLoopStartFrameOffset = NewOffset;
		}

		// Restore original offset modified by splitting
		ParamsPtr->FirstLoopStartFrameOffset = InitialFirstLoopStartFrameOffset;

		return NewSection;
	}
	return nullptr;
}

void UMovieSceneBaseCacheSection::GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const
{
	Super::GetSnapTimes(OutSnapTimes, bGetSectionBorders);
	if(ParamsPtr)
	{
		const FFrameRate   FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();
		const FFrameNumber StartFrame = GetInclusiveStartFrame();
		const FFrameNumber EndFrame = GetExclusiveEndFrame() - 1; // -1 because we don't need to add the end frame twice

		const float AnimPlayRate = FMath::IsNearlyZero(ParamsPtr->PlayRate) ? 1.0f : ParamsPtr->PlayRate;
		const float SeqLengthSeconds = ParamsPtr->GetSequenceLength() - FrameRate.AsSeconds(ParamsPtr->StartFrameOffset + ParamsPtr->EndFrameOffset) / AnimPlayRate;
		const float FirstLoopSeqLengthInSeconds = SeqLengthSeconds - FrameRate.AsSeconds(ParamsPtr->FirstLoopStartFrameOffset) / AnimPlayRate;

		const FFrameTime SequenceFrameLength = SeqLengthSeconds * FrameRate;
		const FFrameTime FirstLoopSequenceFrameLength = FirstLoopSeqLengthInSeconds * FrameRate;
		if (SequenceFrameLength.FrameNumber > 1)
		{
			// Snap to the repeat times
			bool IsFirstLoop = true;
			FFrameTime CurrentTime = StartFrame;
			while (CurrentTime < EndFrame)
			{
				OutSnapTimes.Add(CurrentTime.FrameNumber);
				if (IsFirstLoop)
				{
					CurrentTime += FirstLoopSequenceFrameLength;
					IsFirstLoop = false;
				}
				else
				{
					CurrentTime += SequenceFrameLength;
				}
			}
		}
	}
}

float UMovieSceneBaseCacheSection::MapTimeToAnimation(float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const
{
	if(ParamsPtr)
	{
		const FMovieSceneBaseCacheSectionTemplateParameters TemplateParamsPtr(GetInclusiveStartFrame(), GetExclusiveEndFrame());
		return TemplateParamsPtr.MapTimeToAnimation(*ParamsPtr, ComponentDuration, InPosition, InTickResolution);
	}
	return 0.0f;
}

#if WITH_EDITOR
void UMovieSceneBaseCacheSection::PreEditChange(FProperty* PropertyAboutToChange)
{
	if(ParamsPtr)
	{
		// Store the current play rate so that we can compute the amount to compensate the section end time when the play rate changes
		PreviousPlayRate = ParamsPtr->PlayRate;
	}
	Super::PreEditChange(PropertyAboutToChange);
}

void UMovieSceneBaseCacheSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Adjust the duration automatically if the play rate changes
	if (ParamsPtr && PropertyChangedEvent.Property != nullptr &&
		PropertyChangedEvent.Property->GetFName() == TEXT("PlayRate"))
	{
		const float NewPlayRate = ParamsPtr->PlayRate;

		if (!FMath::IsNearlyZero(NewPlayRate))
		{
			const float CurrentDuration = UE::MovieScene::DiscreteSize(GetRange());
			const float NewDuration = CurrentDuration * (PreviousPlayRate / NewPlayRate);
			SetEndFrame(GetInclusiveStartFrame() + FMath::FloorToInt(NewDuration));

			PreviousPlayRate = NewPlayRate;
		}
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMovieSceneBaseCacheSection::InitializePlayRate()
{
	PreviousPlayRate = ParamsPtr ? ParamsPtr->PlayRate : 0.0f;
}

#endif

