// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneByteSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Evaluation/MovieScenePropertyTemplates.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/MovieScenePiecewiseByteBlenderSystem.h"
#include "Tracks/MovieSceneByteTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "UObject/SequencerObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneByteSection)

UMovieSceneByteSection::UMovieSceneByteSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{ 
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);

	bSupportsInfiniteRange = true;
#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(ByteCurve, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<uint8>::Make());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(ByteCurve);

#endif
}

bool UMovieSceneByteSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneByteSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!ByteCurve.HasAnyData())
	{
		return;
	}

	const UMovieSceneByteTrack* ParentByteTrack = GetTypedOuter<UMovieSceneByteTrack>();
	const UEnum* EnumClass = ParentByteTrack ? ParentByteTrack->GetEnum() : nullptr;

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	if (EnumClass)
	{
		FPropertyTrackEntityImportHelper(TracksComponents->Enum)
			.Add(Components->ByteChannel, &ByteCurve)
			.Commit(this, Params, OutImportedEntity);
	}
	else
	{
		FPropertyTrackEntityImportHelper(TracksComponents->Byte)
			.Add(Components->ByteChannel, &ByteCurve)
			.Add(Components->BlenderType, UMovieScenePiecewiseByteBlenderSystem::StaticClass())
			.Commit(this, Params, OutImportedEntity);
	}
}

