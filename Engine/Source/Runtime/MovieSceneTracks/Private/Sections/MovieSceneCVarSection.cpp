// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneCVarSection.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "TrackInstances/MovieSceneCVarTrackInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCVarSection)

void FMovieSceneCVarOverrides::SetFromString(const FString& InString)
{
	ValuesByCVar.Reset();

	// Split the section's specified value apart by new lines, and then parse out the key/value pairs.
	TArray<FString> AllLines;
	const bool bCullEmpty = true;
	InString.ParseIntoArray(AllLines, LINE_TERMINATOR, bCullEmpty);

	// If there was no LINE_TERMINATOR or they used comma separated values, AllLines will have 1 line in it containing all of InString.
	TArray<FString> SplitLines;
	for (const FString& Line : AllLines)
	{
		TArray<FString> NewStrings;
		Line.ParseIntoArray(NewStrings, TEXT(","), bCullEmpty);

		SplitLines.Append(NewStrings);
	}

	// SplitLines should have at least 1 line in it by extension of AllLines having at least 1 line.
	for (const FString& Line : SplitLines)
	{
		// We should now have things in the format "foo=bar" or "foo bar", so we still need to split to find the value.
		TArray<FString> OutArray;
		const TCHAR* Delimiters[] = { TEXT(","), TEXT(" ") };
		Line.ParseIntoArray(OutArray, Delimiters, UE_ARRAY_COUNT(Delimiters), bCullEmpty);
		if (OutArray.Num() != 2)
		{
			// Not sure what format it's in.
			UE_LOGF(LogMovieScene, Warning, "Could not parse \"%ls\" into a key/value pair. Expected format: \"foo=5.5\" or \"foo 5.5\"!", *Line);
			continue;
		}

		ValuesByCVar.Add(OutArray[0], OutArray[1]);
	}
}

void FMovieSceneCVarOverrides::GetString(TStringBuilder<256>& OutBuilder) const
{
	for (const TPair<FString, FString>& Pair : ValuesByCVar)
	{
		if (OutBuilder.Len() != 0)
		{
			OutBuilder += TEXT(", ");
		}

		OutBuilder += Pair.Key;
		OutBuilder += TEXT(" ");
		OutBuilder += Pair.Value;
	}
}

FString FMovieSceneCVarOverrides::GetString() const
{
	TStringBuilder<256> CVarString;
	GetString(CVarString);
	return CVarString.ToString();
}

UMovieSceneCVarSection::UMovieSceneCVarSection(const FObjectInitializer& Init)
	: Super(Init)
{
	SetRange(TRange<FFrameNumber>::All());

	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);

	SetBlendType(EMovieSceneBlendType::Absolute);

	bSupportsInfiniteRange = true;
}

void UMovieSceneCVarSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FMovieSceneTrackInstanceComponent TrackInstance { decltype(FMovieSceneTrackInstanceComponent::Owner)(this), UMovieSceneCVarTrackInstance::StaticClass() };

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddTag(FBuiltInComponentTypes::Get()->Tags.Root)
		.Add(FBuiltInComponentTypes::Get()->TrackInstance, TrackInstance)
	);
}

void UMovieSceneCVarSection::SetFromString(const FString& InString)
{
	ConsoleVariables.SetFromString(InString);
}

FString UMovieSceneCVarSection::GetString() const
{
	TStringBuilder<256> ResultBuilder;

	for (const FMovieSceneConsoleVariableCollection& Collection : ConsoleVariableCollections)
	{
		if (UObject* Object = Collection.Interface.GetObject())
		{
			if (ResultBuilder.Len() != 0)
			{
				ResultBuilder += TEXT(", ");
			}
			ResultBuilder += Object->GetName();
		}
	}

	ConsoleVariables.GetString(ResultBuilder);
	return ResultBuilder.ToString();
}

