// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneCameraCutSection.h"

#include "MovieScene.h"
#include "Tracks/MovieSceneCameraCutTrack.h"
#include "MovieSceneSequenceID.h"
#include "IMovieScenePlayer.h"
#include "Camera/CameraComponent.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"
#include "EntitySystem/TrackInstance/MovieSceneTrackInstanceSystem.h"
#include "Systems/MovieSceneComponentTransformSystem.h"
#include "TrackInstances/MovieSceneCameraCutTrackInstance.h"
#include "Tracks/MovieScene3DTransformTrack.h"
#include "Tracks/MovieSceneTransformTrack.h"
#include "UObject/LinkerLoad.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraCutSection)

/* UMovieSceneCameraCutSection interface
 *****************************************************************************/

UMovieSceneCameraCutSection::UMovieSceneCameraCutSection(const FObjectInitializer& Init)
	: Super(Init)
{
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
		 EMovieSceneCompletionMode::RestoreState : 
		 EMovieSceneCompletionMode::ProjectDefault);

	SetBlendType(EMovieSceneBlendType::Absolute);
}

void UMovieSceneCameraCutSection::OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState)
{
	UE::MovieScene::FFixedObjectBindingID FixedBindingID = CameraBindingID.ResolveToFixed(LocalSequenceID, SharedPlaybackState);

	if (OldFixedToNewFixedMap.Contains(FixedBindingID))
	{
		Modify();

		const FMovieSceneSequenceHierarchy* Hierarchy = SharedPlaybackState->GetHierarchy();
		CameraBindingID = OldFixedToNewFixedMap[FixedBindingID].ConvertToRelative(LocalSequenceID, Hierarchy);
	}
}

void UMovieSceneCameraCutSection::GetReferencedBindings(TArray<FGuid>& OutBindings)
{
	OutBindings.Add(CameraBindingID.GetGuid());
}

void UMovieSceneCameraCutSection::PostLoad()
{
	Super::PostLoad();

	if (CameraGuid_DEPRECATED.IsValid())
	{
		if (!CameraBindingID.IsValid())
		{
			CameraBindingID = UE::MovieScene::FRelativeObjectBindingID(CameraGuid_DEPRECATED);
		}
		CameraGuid_DEPRECATED.Invalidate();
	}
}

UCameraComponent* UMovieSceneCameraCutSection::GetFirstCamera(IMovieScenePlayer& Player, FMovieSceneSequenceID SequenceID) const
{
	for (TWeakObjectPtr<> WeakObject : CameraBindingID.ResolveBoundObjects(SequenceID, Player))
	{
		if (UObject* Object = WeakObject.Get())
		{
			UCameraComponent* Camera = MovieSceneHelpers::CameraComponentFromRuntimeObject(Object);
			if (Camera)
			{
				return Camera;
			}
		}
	}

	return nullptr;
}

#if WITH_EDITOR

void UMovieSceneCameraCutSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	const FName PropertyName = PropertyChangedEvent.GetPropertyName();
	if (PropertyName == GET_MEMBER_NAME_CHECKED(UMovieSceneCameraCutSection, SectionRange))
	{
		if (UMovieSceneCameraCutTrack* Track = GetTypedOuter<UMovieSceneCameraCutTrack>())
		{
			Track->OnSectionMoved(*this, EPropertyChangeType::ValueSet);
		}
	}
}

#endif

void UMovieSceneCameraCutSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FMovieSceneTrackInstanceComponent TrackInstance { decltype(FMovieSceneTrackInstanceComponent::Owner)(this), UMovieSceneCameraCutTrackInstance::StaticClass() };

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddTag(FBuiltInComponentTypes::Get()->Tags.Root)
		.Add(FBuiltInComponentTypes::Get()->TrackInstance, TrackInstance)
	);
}

void UMovieSceneCameraCutSection::ComputeInitialCameraCutTransform()
{
	using namespace UE::MovieScene;

	// Clear the compiled transform value.
	bHasInitialCameraCutTransform = false;

	// Is there even an initial time for us to compute a transform?
	if (!GetRange().HasLowerBound())
	{
		return;
	}

	// Find the transform track for our bound camera.
	UMovieScene3DTransformTrack* CameraTransformTrack = nullptr;
	if (CameraBindingID.IsValid())
	{
		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		check(MovieScene);

		if (const FMovieSceneBinding* Binding = MovieScene->FindBinding(CameraBindingID.GetGuid()))
		{
			for (UMovieSceneTrack* Track : Binding->GetTracks())
			{
				CameraTransformTrack = Cast<UMovieScene3DTransformTrack>(Track);
				if (CameraTransformTrack)
				{
					break;
				}
			}
		}
	}

	// Does the bound camera have a transform track?
	if (CameraTransformTrack == nullptr)
	{
		return;
	}

	// Ok, let's evaluate the transform track at our start time.
	FSystemInterrogator Interrogator;

	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &Interrogator.GetLinker()->EntityManager);

	Interrogator.ImportTrack(CameraTransformTrack, FInterrogationChannel::Default());

	Interrogator.AddInterrogation(GetTrueRange().GetLowerBoundValue());

	Interrogator.Update();

	TArray<FTransform> TempTransforms;
	Interrogator.QueryWorldSpaceTransforms(FInterrogationChannel::Default(), TempTransforms);

	if (TempTransforms.Num() > 0)
	{
		// Store it so we can place it on our imported entities.
		InitialCameraCutTransform = TempTransforms[0];
		bHasInitialCameraCutTransform = true;
	}
}

