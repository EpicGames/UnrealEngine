// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneCameraShakeSection.h"
#include "UObject/SequencerObjectVersion.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraShakeSection)


UMovieSceneCameraShakeSection::UMovieSceneCameraShakeSection(const FObjectInitializer& ObjectInitializer)
	: Super( ObjectInitializer )
{
	ShakeClass_DEPRECATED = nullptr;
	PlayScale_DEPRECATED = 1.f;
	PlaySpace_DEPRECATED = ECameraShakePlaySpace::CameraLocal;
	UserDefinedPlaySpace_DEPRECATED = FRotator::ZeroRotator;

	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);
}

void UMovieSceneCameraShakeSection::PostLoad()
{
	if (ShakeClass_DEPRECATED != nullptr)
	{
		ShakeData.ShakeClass = ShakeClass_DEPRECATED;
	}

	if (PlayScale_DEPRECATED != 1.f)
	{
		ShakeData.PlayScale = PlayScale_DEPRECATED;
	}

	if (PlaySpace_DEPRECATED != ECameraShakePlaySpace::CameraLocal)
	{
		ShakeData.PlaySpace = PlaySpace_DEPRECATED;
	}

	if (UserDefinedPlaySpace_DEPRECATED != FRotator::ZeroRotator)
	{
		ShakeData.UserDefinedPlaySpace = UserDefinedPlaySpace_DEPRECATED;
	}

	Super::PostLoad();
}

UObject* UMovieSceneCameraShakeSection::GetSourceObject() const
{
	return ShakeData.ShakeClass;
}

void UMovieSceneCameraShakeSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!ShakeData.ShakeClass)
	{
		return;
	}

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	const FGuid ObjectBindingID = Params.GetObjectBindingID();
	FMovieSceneCameraShakeComponentData ComponentData(ShakeData, *this);

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
		.Add(TrackComponents->CameraShake, ComponentData)
	);
}
