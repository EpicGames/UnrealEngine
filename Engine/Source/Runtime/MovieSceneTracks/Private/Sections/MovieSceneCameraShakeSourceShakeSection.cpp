// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneCameraShakeSourceShakeSection.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraShakeSourceShakeSection)

UMovieSceneCameraShakeSourceShakeSection::UMovieSceneCameraShakeSourceShakeSection(const FObjectInitializer& ObjectInitializer)
	: Super( ObjectInitializer )
{
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);
}

UObject* UMovieSceneCameraShakeSourceShakeSection::GetSourceObject() const
{
	return ShakeData.ShakeClass;
}

void UMovieSceneCameraShakeSourceShakeSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	const FGuid ObjectBindingID = Params.GetObjectBindingID();
	FMovieSceneCameraShakeComponentData ComponentData(ShakeData, *this);

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
		.Add(TrackComponents->CameraShake, ComponentData)
	);
}

