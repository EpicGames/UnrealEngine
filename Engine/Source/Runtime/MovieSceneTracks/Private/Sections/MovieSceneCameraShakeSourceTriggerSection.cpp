// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneCameraShakeSourceTriggerSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "Systems/MovieSceneCameraShakeSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraShakeSourceTriggerSection)

UMovieSceneCameraShakeSourceTriggerSection::UMovieSceneCameraShakeSourceTriggerSection(const FObjectInitializer& Init)
	: Super(Init)
{
#if WITH_EDITOR
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(Channel, FMovieSceneChannelMetaData());
#else
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(Channel);
#endif
}

bool UMovieSceneCameraShakeSourceTriggerSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	TArrayView<const FFrameNumber> Times = Channel.GetData().GetTimes();
	for (int32 Index = 0; Index < Times.Num(); ++Index)
	{
		if (EffectiveRange.Contains(Times[Index]))
		{
			TRange<FFrameNumber> Range(Times[Index]);
			OutFieldBuilder->AddOneShotEntity(Range, this, Index, MetaDataIndex);
		}
	}

	return true;
}

void UMovieSceneCameraShakeSourceTriggerSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const int32 ShakeIndex = static_cast<int32>(Params.EntityID);

	TArrayView<const FFrameNumber> Times  = Channel.GetData().GetTimes();
	TArrayView<const FMovieSceneCameraShakeSourceTrigger> Shakes = Channel.GetData().GetValues();
	if (!ensureMsgf(
			Shakes.IsValidIndex(ShakeIndex), 
			TEXT("Attempting to import a camera shake entity for an invalid index (Index: %d, Num: %d)"), ShakeIndex, Shakes.Num()))
	{
		return;
	}

	UMovieSceneCameraShakeInstantiatorSystem* ShakeSystem = EntityLinker->LinkSystem<UMovieSceneCameraShakeInstantiatorSystem>();
	if (ensure(ShakeSystem))
	{
		ShakeSystem->AddShakeTrigger(Params.Sequence.InstanceHandle, Params.GetObjectBindingID(), Times[ShakeIndex], Shakes[ShakeIndex]);

		// Mimic the structure changing in order to ensure that the instantiation phase runs
		EntityLinker->EntityManager.MimicStructureChanged();
	}
}

