// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneCinematicShotSection.h"
#include "MovieSceneSequence.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCinematicShotSection)

/* UMovieSceneCinematicshotSection structors
 *****************************************************************************/

UMovieSceneCinematicShotSection::UMovieSceneCinematicShotSection(const FObjectInitializer& ObjInitializer)
	: Super(ObjInitializer)
{ }

FString UMovieSceneCinematicShotSection::GetShotDisplayName() const
{
	if (ShotDisplayName.IsEmpty() && GetSequence())
	{
		return GetSequence()->GetName();
	}
	return ShotDisplayName;
}

void UMovieSceneCinematicShotSection::PostLoad()
{
	Super::PostLoad();

	if (!DisplayName_DEPRECATED.IsEmpty())
	{
		ShotDisplayName = DisplayName_DEPRECATED.ToString();
		DisplayName_DEPRECATED = FText::GetEmpty();
	}
}

