// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneColorSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Styling/SlateColor.h"
#include "Tracks/MovieSceneColorTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "UObject/SequencerObjectVersion.h"
#include "UObject/StructOnScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneColorSection)

#if WITH_EDITOR
struct FColorSectionEditorData
{
	FColorSectionEditorData()
	{
		MetaData[0].SetIdentifiers("Color.R", FCommonChannelData::ChannelR);
		MetaData[0].SubPropertyPath = TEXT("R");
		MetaData[0].SortOrder = 0;
		MetaData[0].Color = FCommonChannelData::RedChannelColor;
		MetaData[0].bCanCollapseToTrack = false;

		MetaData[1].SetIdentifiers("Color.G", FCommonChannelData::ChannelG);
		MetaData[1].SubPropertyPath = TEXT("G");
		MetaData[1].SortOrder = 1;
		MetaData[1].Color = FCommonChannelData::GreenChannelColor;
		MetaData[1].bCanCollapseToTrack = false;

		MetaData[2].SetIdentifiers("Color.B", FCommonChannelData::ChannelB);
		MetaData[2].SubPropertyPath = TEXT("B");
		MetaData[2].SortOrder = 2;
		MetaData[2].Color = FCommonChannelData::BlueChannelColor;
		MetaData[2].bCanCollapseToTrack = false;

		MetaData[3].SetIdentifiers("Color.A", FCommonChannelData::ChannelA);
		MetaData[3].SubPropertyPath = TEXT("A");
		MetaData[3].SortOrder = 3;
		MetaData[3].bCanCollapseToTrack = false;

		ExternalValues[0].OnGetExternalValue = ExtractChannelR;
		ExternalValues[1].OnGetExternalValue = ExtractChannelG;
		ExternalValues[2].OnGetExternalValue = ExtractChannelB;
		ExternalValues[3].OnGetExternalValue = ExtractChannelA;
	}

	static FLinearColor GetPropertyValue(UObject& InObject, FTrackInstancePropertyBindings& Bindings)
	{
		const FName SlateColorName("SlateColor");
	
		const UStruct* ColorStruct = Bindings.GetPropertyStruct(InObject);
		if (ColorStruct != nullptr)
		{
			if (ColorStruct->GetFName() == SlateColorName)
			{
				return Bindings.GetCurrentValue<FSlateColor>(InObject).GetSpecifiedColor();
			}
	
			if (ColorStruct->GetFName() == NAME_LinearColor)
			{
				return Bindings.GetCurrentValue<FLinearColor>(InObject);
			}
	
			if (ColorStruct->GetFName() == NAME_Color)
			{
				return Bindings.GetCurrentValue<FColor>(InObject);
			}
		}
		return FLinearColor(0.f,0.f,0.f,0.f);
	}
	
	static TOptional<float> ExtractChannelR(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? GetPropertyValue(InObject, *Bindings).R : TOptional<float>();
	}
	static TOptional<float> ExtractChannelG(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? GetPropertyValue(InObject, *Bindings).G : TOptional<float>();
	}
	static TOptional<float> ExtractChannelB(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? GetPropertyValue(InObject, *Bindings).B : TOptional<float>();
	}
	static TOptional<float> ExtractChannelA(UObject& InObject, FTrackInstancePropertyBindings* Bindings)
	{
		return Bindings ? GetPropertyValue(InObject, *Bindings).A : TOptional<float>();
	}

	FMovieSceneChannelMetaData      MetaData[4];
	TMovieSceneExternalValue<float> ExternalValues[4];
};
#endif

/* FMovieSceneColorKeyStruct interface
 *****************************************************************************/

void FMovieSceneColorKeyStruct::PropagateChanges(const FPropertyChangedEvent& ChangeEvent)
{
	KeyStructInterop.Apply(Time);
}


/* UMovieSceneColorSection structors
 *****************************************************************************/

UMovieSceneColorSection::UMovieSceneColorSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);
	BlendType = EMovieSceneBlendType::Absolute;
	bSupportsInfiniteRange = true;

	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR

	static FColorSectionEditorData EditorData;
	Channels.Add(RedCurve,   EditorData.MetaData[0], EditorData.ExternalValues[0]);
	Channels.Add(GreenCurve, EditorData.MetaData[1], EditorData.ExternalValues[1]);
	Channels.Add(BlueCurve,  EditorData.MetaData[2], EditorData.ExternalValues[2]);
	Channels.Add(AlphaCurve, EditorData.MetaData[3], EditorData.ExternalValues[3]);

#else

	Channels.Add(RedCurve);
	Channels.Add(GreenCurve);
	Channels.Add(BlueCurve);
	Channels.Add(AlphaCurve);

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

/* UMovieSceneSection interface
 *****************************************************************************/

TSharedPtr<FStructOnScope> UMovieSceneColorSection::GetKeyStruct(TArrayView<const FKeyHandle> KeyHandles)
{
	if (!ChannelProxy)
	{
		GetChannelProxy();
	}
	
	TSharedRef<FStructOnScope> KeyStruct = MakeShareable(new FStructOnScope(FMovieSceneColorKeyStruct::StaticStruct()));
	auto Struct = (FMovieSceneColorKeyStruct*)KeyStruct->GetStructMemory();

	Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneFloatChannel>(0), &Struct->Color.R, KeyHandles));
	Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneFloatChannel>(1), &Struct->Color.G, KeyHandles));
	Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneFloatChannel>(2), &Struct->Color.B, KeyHandles));
	Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneFloatChannel>(3), &Struct->Color.A, KeyHandles));

	Struct->KeyStructInterop.SetStartingValues();
	Struct->Time = Struct->KeyStructInterop.GetUnifiedKeyTime().Get(0);

	return KeyStruct;
}

bool UMovieSceneColorSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneColorSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!RedCurve.HasAnyData() && !GreenCurve.HasAnyData() && !BlueCurve.HasAnyData() && !AlphaCurve.HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->Color)
		.AddConditional(Components->FloatChannel[0], &RedCurve, RedCurve.HasAnyData())
		.AddConditional(Components->FloatChannel[1], &GreenCurve, GreenCurve.HasAnyData())
		.AddConditional(Components->FloatChannel[2], &BlueCurve, BlueCurve.HasAnyData())
		.AddConditional(Components->FloatChannel[3], &AlphaCurve, AlphaCurve.HasAnyData())
		.Commit(this, Params, OutImportedEntity);
}

