// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneComponentMaterialParameterSection.h"
#include "Channels/MovieSceneChannelProxy.h"

#include "Evaluation/MovieSceneEvaluationField.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Tracks/MovieSceneMaterialTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneComponentMaterialParameterSection)

#if WITH_EDITOR
#define LOCTEXT_NAMESPACE "MaterialTrackEditor"
#endif

namespace UE::MovieScene
{

	/* Entity IDs are an encoded type and index, with the upper 8 bits being the type, and the lower 24 bits as the index */
	uint32 EncodeMaterialParameterEntityID(int32 InIndex, uint8 InType)
	{
		check(InIndex >= 0 && InIndex < int32(0x00FFFFFF));
		return static_cast<uint32>(InIndex) | (uint32(InType) << 24);
	}
	void DecodeMaterialParameterEntityID(uint32 InEntityID, int32& OutIndex, uint8& OutType)
	{
		// Mask out the type to get the index
		OutIndex = static_cast<int32>(InEntityID & 0x00FFFFFF);
		OutType = InEntityID >> 24;
	}


}// namespace UE::MovieScene

FScalarMaterialParameterInfoAndCurve::FScalarMaterialParameterInfoAndCurve(const FMaterialParameterInfo& InParameterInfo)
{
	ParameterInfo = InParameterInfo;
}

FColorMaterialParameterInfoAndCurves::FColorMaterialParameterInfoAndCurves(const FMaterialParameterInfo& InParameterInfo)
{
	ParameterInfo = InParameterInfo;
}

UMovieSceneComponentMaterialParameterSection::UMovieSceneComponentMaterialParameterSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	bSupportsInfiniteRange = true;
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);
}

EMovieSceneChannelProxyType UMovieSceneComponentMaterialParameterSection::CacheChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR

	auto GetMaterialParameterDisplayText = [](const FText& ParameterName, const FString& InLayerName, const FString& InAssetName)
	{
		FText DisplayName = ParameterName;
		if (!InLayerName.IsEmpty() && !InAssetName.IsEmpty())
		{
			DisplayName = FText::Format(LOCTEXT("MaterialParameterDisplayText", "{0} ({1}.{2})"), ParameterName, FText::FromString(InLayerName), FText::FromString(InAssetName));
		}
		return DisplayName;
	};
	auto GetMaterialParameterPath = [](const FName& ParameterName, const FString& InLayerName, const FString& InAssetName)
	{
		FString Path = ParameterName.ToString();
		if (!InLayerName.IsEmpty() && !InAssetName.IsEmpty())
		{
			Path = FString::Printf(TEXT("%s.%s.%s"), *InLayerName, *InAssetName, *ParameterName.ToString());
		}
		return Path;
	};
	auto GetMaterialParameterTooltipText = [](IMovieScenePlayer* Player, FGuid BindingID, FMovieSceneSequenceID SequenceID, FString ParameterPath)
	{
		return FText::Format(LOCTEXT("MaterialParameterPath", "Path: {0}"), FText::FromString(ParameterPath));
	};
	int32 SortOrder = 0;
	for (FScalarMaterialParameterInfoAndCurve& Scalar : ScalarParameterInfosAndCurves)
	{
		FString ParameterPath = *GetMaterialParameterPath(Scalar.ParameterInfo.Name, Scalar.ParameterLayerName, Scalar.ParameterAssetName);
		FText ParameterDisplayName = GetMaterialParameterDisplayText(FText::FromName(Scalar.ParameterInfo.Name), Scalar.ParameterLayerName, Scalar.ParameterAssetName);
		FMovieSceneChannelMetaData MetaData(*ParameterPath, ParameterDisplayName);
		// Prevent single channels from collapsing to the track node
		MetaData.GetTooltipTextDelegate.BindLambda(GetMaterialParameterTooltipText, ParameterPath);
		MetaData.bCanCollapseToTrack = false;
		MetaData.SortOrder = SortOrder++;
		Channels.Add(Scalar.ParameterCurve, MetaData, TMovieSceneExternalValue<float>());
	}
	for (FColorMaterialParameterInfoAndCurves& Color : ColorParameterInfosAndCurves)
	{
		FString ParameterPath = GetMaterialParameterPath(Color.ParameterInfo.Name, Color.ParameterLayerName, Color.ParameterAssetName);
		FText ParameterDisplayName = GetMaterialParameterDisplayText(FText::FromName(Color.ParameterInfo.Name), Color.ParameterLayerName, Color.ParameterAssetName);
		FText Group = FText::FromString(ParameterPath);

		FText RChannelDisplayName = !Color.ParameterChannelNames.R.IsEmpty() ? Color.ParameterChannelNames.R : FCommonChannelData::ChannelR;
		FText GChannelDisplayName = !Color.ParameterChannelNames.G.IsEmpty() ? Color.ParameterChannelNames.G : FCommonChannelData::ChannelG;
		FText BChannelDisplayName = !Color.ParameterChannelNames.B.IsEmpty() ? Color.ParameterChannelNames.B : FCommonChannelData::ChannelB;
		FText AChannelDisplayName = !Color.ParameterChannelNames.A.IsEmpty() ? Color.ParameterChannelNames.A : FCommonChannelData::ChannelA;

		FMovieSceneChannelMetaData MetaData_R(*(ParameterPath + TEXT("R")), RChannelDisplayName, Group);
		MetaData_R.GetTooltipTextDelegate.BindLambda(GetMaterialParameterTooltipText, ParameterPath + TEXT(".R"));
		MetaData_R.SortOrder = SortOrder++;
		MetaData_R.Color = FCommonChannelData::RedChannelColor;
		MetaData_R.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, ParameterDisplayName.ToString());
		MetaData_R.GetGroupTooltipTextDelegate.BindLambda(GetMaterialParameterTooltipText, ParameterPath);

		FMovieSceneChannelMetaData MetaData_G(*(ParameterPath + TEXT("G")), GChannelDisplayName, Group);
		MetaData_G.GetTooltipTextDelegate.BindLambda(GetMaterialParameterTooltipText, ParameterPath + TEXT(".G"));
		MetaData_G.SortOrder = SortOrder++;
		MetaData_G.Color = FCommonChannelData::GreenChannelColor;
		MetaData_G.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, ParameterDisplayName.ToString());
		MetaData_G.GetGroupTooltipTextDelegate.BindLambda(GetMaterialParameterTooltipText, ParameterPath);

		FMovieSceneChannelMetaData MetaData_B(*(ParameterPath + TEXT("B")), BChannelDisplayName, Group);
		MetaData_B.GetTooltipTextDelegate.BindLambda(GetMaterialParameterTooltipText, ParameterPath + TEXT(".B"));
		MetaData_B.SortOrder = SortOrder++;
		MetaData_B.Color = FCommonChannelData::BlueChannelColor;
		MetaData_B.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, ParameterDisplayName.ToString());
		MetaData_B.GetGroupTooltipTextDelegate.BindLambda(GetMaterialParameterTooltipText, ParameterPath);

		FMovieSceneChannelMetaData MetaData_A(*(ParameterPath + TEXT("A")), AChannelDisplayName, Group);
		MetaData_A.GetTooltipTextDelegate.BindLambda(GetMaterialParameterTooltipText, ParameterPath + TEXT(".A"));
		MetaData_A.SortOrder = SortOrder++;
		MetaData_A.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, ParameterDisplayName.ToString());
		MetaData_A.GetGroupTooltipTextDelegate.BindLambda(GetMaterialParameterTooltipText, ParameterPath);

		Channels.Add(Color.RedCurve, MetaData_R, TMovieSceneExternalValue<float>());
		Channels.Add(Color.GreenCurve, MetaData_G, TMovieSceneExternalValue<float>());
		Channels.Add(Color.BlueCurve, MetaData_B, TMovieSceneExternalValue<float>());
		Channels.Add(Color.AlphaCurve, MetaData_A, TMovieSceneExternalValue<float>());
	}
#else

	for (FScalarMaterialParameterInfoAndCurve& Scalar : ScalarParameterInfosAndCurves)
	{
		Channels.Add(Scalar.ParameterCurve);
	}

	for (FColorMaterialParameterInfoAndCurves& Color : ColorParameterInfosAndCurves)
	{
		Channels.Add(Color.RedCurve);
		Channels.Add(Color.GreenCurve);
		Channels.Add(Color.BlueCurve);
		Channels.Add(Color.AlphaCurve);
	}
#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));

	return EMovieSceneChannelProxyType::Dynamic;
}

void UMovieSceneComponentMaterialParameterSection::PostEditImport()
{
	Super::PostEditImport();

	CacheChannelProxy();
}
#if WITH_EDITOR
void UMovieSceneComponentMaterialParameterSection::PostEditUndo()
{
	Super::PostEditUndo();

	CacheChannelProxy();
}
#endif
void UMovieSceneComponentMaterialParameterSection::AddScalarParameterKey(const FMaterialParameterInfo& InParameterInfo, FFrameNumber InTime, float InValue, const FString& InLayerName, const FString& InAssetName, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	FMovieSceneFloatChannel* ExistingChannel = nullptr;
	for (FScalarMaterialParameterInfoAndCurve& ScalarParameterInfoAndCurve : ScalarParameterInfosAndCurves)
	{
		if (ScalarParameterInfoAndCurve.ParameterInfo == InParameterInfo)
		{
			ExistingChannel = &ScalarParameterInfoAndCurve.ParameterCurve;
			break;
		}
	}
	if (ExistingChannel == nullptr)
	{
		const int32 NewIndex = ScalarParameterInfosAndCurves.Add(FScalarMaterialParameterInfoAndCurve(InParameterInfo));
#if WITH_EDITOR
		ScalarParameterInfosAndCurves[NewIndex].ParameterLayerName = InLayerName;
		ScalarParameterInfosAndCurves[NewIndex].ParameterAssetName = InAssetName;
#endif
		ExistingChannel = &ScalarParameterInfosAndCurves[NewIndex].ParameterCurve;
		CacheChannelProxy();
	}

	EMovieSceneKeyInterpolation Interpolation = GetInterpolationMode(ExistingChannel, InTime, DefaultInterpolation);
	AddKeyToChannel(ExistingChannel, InTime, InValue, Interpolation);

	if (TryModify())
	{
		SetRange(TRange<FFrameNumber>::Hull(TRange<FFrameNumber>(InTime), GetRange()));
	}
}

void UMovieSceneComponentMaterialParameterSection::AddColorParameterKey(const FMaterialParameterInfo& InParameterInfo, FFrameNumber InTime, FLinearColor InValue, const FString& InLayerName, const FString& InAssetName, const FParameterChannelNames& InChannelNames, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	FColorMaterialParameterInfoAndCurves* ExistingCurves = nullptr;
	for (FColorMaterialParameterInfoAndCurves& ColorParameterInfoAndCurve : ColorParameterInfosAndCurves)
	{
		if (ColorParameterInfoAndCurve.ParameterInfo == InParameterInfo)
		{
			ExistingCurves = &ColorParameterInfoAndCurve;
			break;
		}
	}
	if (ExistingCurves == nullptr)
	{
		int32 NewIndex = ColorParameterInfosAndCurves.Add(FColorMaterialParameterInfoAndCurves(InParameterInfo));
		ExistingCurves = &ColorParameterInfosAndCurves[NewIndex];
#if WITH_EDITOR
		ColorParameterInfosAndCurves[NewIndex].ParameterLayerName = InLayerName;
		ColorParameterInfosAndCurves[NewIndex].ParameterAssetName = InAssetName;
		ColorParameterInfosAndCurves[NewIndex].ParameterChannelNames = InChannelNames;
#endif
		CacheChannelProxy();
	}

	EMovieSceneKeyInterpolation RedInterpolation = GetInterpolationMode(&ExistingCurves->RedCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->RedCurve, InTime, InValue.R, RedInterpolation);

	EMovieSceneKeyInterpolation GreenInterpolation = GetInterpolationMode(&ExistingCurves->GreenCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->GreenCurve, InTime, InValue.G, GreenInterpolation);

	EMovieSceneKeyInterpolation BlueInterpolation = GetInterpolationMode(&ExistingCurves->BlueCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->BlueCurve, InTime, InValue.B, BlueInterpolation);

	EMovieSceneKeyInterpolation AlphaInterpolation = GetInterpolationMode(&ExistingCurves->AlphaCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->AlphaCurve, InTime, InValue.A, AlphaInterpolation);



	if (TryModify())
	{
		SetRange(TRange<FFrameNumber>::Hull(TRange<FFrameNumber>(InTime), GetRange()));
	}
}

bool UMovieSceneComponentMaterialParameterSection::RemoveScalarParameter(const FMaterialParameterInfo& InParameterInfo)
{
	for (int32 i = 0; i < ScalarParameterInfosAndCurves.Num(); i++)
	{
		if (ScalarParameterInfosAndCurves[i].ParameterInfo == InParameterInfo)
		{
			ScalarParameterInfosAndCurves.RemoveAt(i);
			CacheChannelProxy();
			return true;
		}
	}
	return false;
}

bool UMovieSceneComponentMaterialParameterSection::RemoveColorParameter(const FMaterialParameterInfo& InParameterInfo)
{
	for (int32 i = 0; i < ColorParameterInfosAndCurves.Num(); i++)
	{
		if (ColorParameterInfosAndCurves[i].ParameterInfo == InParameterInfo)
		{
			ColorParameterInfosAndCurves.RemoveAt(i);
			CacheChannelProxy();
			return true;
		}
	}
	return false;
}

#if WITH_EDITOR
bool UMovieSceneComponentMaterialParameterSection::RemoveScalarParameter(FName InParameterPath)
{
	TArray<FString> ParameterPathArray;
	InParameterPath.ToString().ParseIntoArray(ParameterPathArray, TEXT("."));
	for (int32 i = 0; i < ScalarParameterInfosAndCurves.Num(); i++)
	{
		bool bFound = false;
		if (ParameterPathArray.Num() == 3)
		{
			bFound = ScalarParameterInfosAndCurves[i].ParameterLayerName == *ParameterPathArray[0]
				&& ScalarParameterInfosAndCurves[i].ParameterAssetName == ParameterPathArray[1]
				&& ScalarParameterInfosAndCurves[i].ParameterInfo.Name == ParameterPathArray[2];
		}
		else if (ParameterPathArray.Num() == 1)
		{
			bFound = ScalarParameterInfosAndCurves[i].ParameterInfo.Name == *ParameterPathArray[0];
		}
		if (bFound)
		{
			ScalarParameterInfosAndCurves.RemoveAt(i);
			CacheChannelProxy();
			return true;
		}
	}
	return false;
}

bool UMovieSceneComponentMaterialParameterSection::RemoveColorParameter(FName InParameterPath)
{
	TArray<FString> ParameterPathArray;
	InParameterPath.ToString().ParseIntoArray(ParameterPathArray, TEXT("."));
	for (int32 i = 0; i < ColorParameterInfosAndCurves.Num(); i++)
	{
		bool bFound = false;
		if (ParameterPathArray.Num() == 3)
		{
			bFound = ColorParameterInfosAndCurves[i].ParameterLayerName == *ParameterPathArray[0]
				&& ColorParameterInfosAndCurves[i].ParameterAssetName == ParameterPathArray[1]
				&& ColorParameterInfosAndCurves[i].ParameterInfo.Name == ParameterPathArray[2];
		}
		else if (ParameterPathArray.Num() == 1)
		{
			bFound = ColorParameterInfosAndCurves[i].ParameterInfo.Name == *ParameterPathArray[0];
		}
		if (bFound)
		{
			ColorParameterInfosAndCurves.RemoveAt(i);
			CacheChannelProxy();
			return true;
		}
	}
	return false;
}
#endif

void UMovieSceneComponentMaterialParameterSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	uint8 ParameterType = 0;
	int32 EntityIndex = 0;
	DecodeMaterialParameterEntityID(Params.EntityID, EntityIndex, ParameterType);

	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponentTypes = FMovieSceneTracksComponentTypes::Get();

	FGuid ObjectBindingID = Params.GetObjectBindingID();
	
	// Find material info from the outer track
	FComponentMaterialInfo MaterialInfo;
	if (UMovieSceneComponentMaterialTrack* MaterialTrack = GetTypedOuter<UMovieSceneComponentMaterialTrack>())
	{
		MaterialInfo = MaterialTrack->GetMaterialInfo();
	}

	TEntityBuilder<TAddConditional<FGuid>> BaseBuilder = FEntityBuilder()
		.AddConditional(BuiltInComponentTypes->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid());

	switch (ParameterType)
	{
	case 0:
	{
		const FScalarMaterialParameterInfoAndCurve& Scalar = ScalarParameterInfosAndCurves[EntityIndex];

		if (Scalar.ParameterCurve.HasAnyData())
		{
			OutImportedEntity->AddBuilder(
				BaseBuilder
				.Add(TracksComponentTypes->ScalarMaterialParameterInfo, Scalar.ParameterInfo)
				.Add(BuiltInComponentTypes->FloatChannel[0], &Scalar.ParameterCurve)
				.Add(TracksComponentTypes->ComponentMaterialInfo, MaterialInfo)
				// If the section has no valid blend type (legacy data), make it use absolute blending.
				// Otherwise, the base section class will add the appropriate blend type tag in BuildDefaultComponents.
				.AddTagConditional(BuiltInComponentTypes->Tags.AbsoluteBlend, !GetBlendType().IsValid())
			);
		}
		break;
	}
	case 1:
	{
		const FColorMaterialParameterInfoAndCurves& Color = ColorParameterInfosAndCurves[EntityIndex];
		if (Color.RedCurve.HasAnyData() || Color.GreenCurve.HasAnyData() || Color.BlueCurve.HasAnyData() || Color.AlphaCurve.HasAnyData())
		{
			OutImportedEntity->AddBuilder(
				BaseBuilder
				.Add(TracksComponentTypes->ColorMaterialParameterInfo, Color.ParameterInfo)
				.AddConditional(BuiltInComponentTypes->FloatChannel[0], &Color.RedCurve, Color.RedCurve.HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[1], &Color.GreenCurve, Color.GreenCurve.HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[2], &Color.BlueCurve, Color.BlueCurve.HasAnyData())
				.AddConditional(BuiltInComponentTypes->FloatChannel[3], &Color.AlphaCurve, Color.AlphaCurve.HasAnyData())
				.Add(TracksComponentTypes->ComponentMaterialInfo, MaterialInfo)
				// If the section has no valid blend type (legacy data), make it use absolute blending.
				// Otherwise, the base section class will add the appropriate blend type tag in BuildDefaultComponents.
				.AddTagConditional(BuiltInComponentTypes->Tags.AbsoluteBlend, !GetBlendType().IsValid())
			);
		}
		break;
	}
	}
}

bool UMovieSceneComponentMaterialParameterSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	// By default, material parameter sections do not populate any evaluation field entries
	// that is the job of its outer UMovieSceneTrack through a call to ExternalPopulateEvaluationField.

	// Ideally we wouldn't have this exception for MaterialParameterSections, but deprecated behavior from MaterialTracks using ParameterSections have this behavior, and when a track implements PopulateEvaluationFieldImpl, the version
	// on Sections is not called.

	return true;
}

void UMovieSceneComponentMaterialParameterSection::ExternalPopulateEvaluationField(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	using namespace UE::MovieScene;

	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	// We use the top 8 bits of EntityID to encode the type of parameter
	const int32 NumScalarID = ScalarParameterInfosAndCurves.Num();
	const int32 NumColorID = ColorParameterInfosAndCurves.Num();

	for (int32 Index = 0; Index < NumScalarID; ++Index)
	{
		const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeMaterialParameterEntityID(Index, 0));
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
	}
	for (int32 Index = 0; Index < NumColorID; ++Index)
	{
		const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeMaterialParameterEntityID(Index, 1));
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
	}
}

#if WITH_EDITOR
#undef LOCTEXT_NAMESPACE
#endif