// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneConstrainedSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneConstrainedSection)


IMovieSceneConstrainedSection::IMovieSceneConstrainedSection() : bDoNotRemoveChannel(false)
{
}


void IMovieSceneConstrainedSection::SetDoNoRemoveChannel(bool bInDoNotRemoveChannel)
{
	bDoNotRemoveChannel = bInDoNotRemoveChannel;
}

UTickableConstraint* IMovieSceneConstrainedSection::FindConstraint(const FName InConstraintName) const
{
	const TArray<FConstraintAndActiveChannel>& Constraints = GetConstraintsChannels();
	const int32 ConstraintIndex = Constraints.IndexOfByPredicate([InConstraintName](const FConstraintAndActiveChannel& ConstraintChannel)
		{
			UTickableConstraint* Constraint = ConstraintChannel.GetConstraint().Get();
			return Constraint && Constraint->GetFName() == InConstraintName;
		});
	return ConstraintIndex != INDEX_NONE ? Constraints[ConstraintIndex].GetConstraint() : nullptr;
}

UTickableConstraint* IMovieSceneConstrainedSection::FindConstraint(FMovieSceneChannel* InChannel) const
{
	const TArray<FConstraintAndActiveChannel>& Constraints = GetConstraintsChannels();
	const int32 ConstraintIndex = Constraints.IndexOfByPredicate([InChannel](const FConstraintAndActiveChannel& ConstraintChannel)
		{
			return (InChannel && &ConstraintChannel.ActiveChannel == InChannel);

		});
	return ConstraintIndex != INDEX_NONE ? Constraints[ConstraintIndex].GetConstraint() : nullptr;
}


bool IMovieSceneConstrainedSection::RemoveConstraint(UWorld* InWorld, const FName InConstraintName, const bool bDoNotCompensate) const
{
	UTickableConstraint* Constraint = FindConstraint(InConstraintName);
	if (InWorld && Constraint)
	{
		return FConstraintsManagerController::Get(InWorld).RemoveConstraint(Constraint, bDoNotCompensate);
	}
	return false;
}

bool IMovieSceneConstrainedSection::RemoveAllConstraints(UWorld* InWorld, const bool bDoNotCompensate) const
{
	bool bRemoved = false;

	if (InWorld)
	{
		const TArray<FConstraintAndActiveChannel> ConstraintAndActiveChannels = GetConstraintsChannels();
		for (const FConstraintAndActiveChannel& ActiveChannel : ConstraintAndActiveChannels)
		{
			if (FConstraintsManagerController::Get(InWorld).RemoveConstraint(ActiveChannel.GetConstraint(), bDoNotCompensate))
			{
				bRemoved = true;
			}
		}
	}
	
	return bRemoved;
}
