// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneCustomPrimitiveDataSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Tracks/MovieSceneCustomPrimitiveDataTrack.h"
#include "MovieSceneTracksComponentTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "SceneTypes.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialInterface.h"
#include "Materials/MaterialFunctionInterface.h"
#include "Materials/MaterialLayersFunctions.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCustomPrimitiveDataSection)

#define LOCTEXT_NAMESPACE "CustomPrimitiveDataTrack"

namespace UE::MovieScene
{
	/* Entity IDs are an encoded type, index, and offset, with the upper 8 bits being the type, the next 8 bits being the offset, and the lower 16 bits as the index */
	uint32 EncodeEntityID(int16 InIndex, uint8 InType, uint8 InOffset)
	{
		check(InIndex >= 0 && InIndex < int32(0x0000FFFF));
		return static_cast<uint32>(InIndex) | (uint32(InType) << 24) | uint32(InOffset) << 16;
	}
	void DecodeEntityID(uint32 InEntityID, int16& OutIndex, uint8& OutType, uint8& OutOffset)
	{
		// Mask out the type to get the index
		OutIndex = static_cast<int16>(InEntityID & 0x0000FFFF);
		OutType = InEntityID >> 24;
		OutOffset = static_cast<uint8>((InEntityID >> 16) & 0x000000FF);
	}


}// namespace UE::MovieScene

void UMovieSceneCustomPrimitiveDataSection::ReconstructChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR

	UMovieSceneCustomPrimitiveDataTrack* CPDTrack = GetTypedOuter<UMovieSceneCustomPrimitiveDataTrack>();
	auto GetCPDTooltipText = [CPDTrack](IMovieScenePlayer* Player, FGuid ObjectBindingID, FMovieSceneSequenceID SequenceID, int32 StartIndex, EMaterialParameterType ParameterType, FText ParameterTypeText)
	{
		auto GetMaterialParameterLayerName = [](const FMaterialLayersFunctions& Layers, const FMaterialParameterInfo& InParameterInfo)
		{
			FString LayerName;
			if (Layers.EditorOnly.LayerNames.IsValidIndex(InParameterInfo.Index))
			{
				LayerName = Layers.GetLayerName(InParameterInfo.Index).ToString();
			}
			return LayerName;
		};
		auto GetMaterialParameterAssetName = [](const FMaterialLayersFunctions& Layers, const FMaterialParameterInfo& InParameterInfo)
		{
			FString AssetName;
			if (InParameterInfo.Association == EMaterialParameterAssociation::LayerParameter && Layers.Layers.IsValidIndex(InParameterInfo.Index))
			{
				AssetName = Layers.Layers[InParameterInfo.Index]->GetName();
			}
			else if (InParameterInfo.Association == EMaterialParameterAssociation::BlendParameter && Layers.Blends.IsValidIndex(InParameterInfo.Index))
			{
				AssetName = Layers.Blends[InParameterInfo.Index]->GetName();
			}
			return AssetName;
		};

		auto GetMaterialParameterDisplayName = [](const FMaterialParameterInfo& InParameterInfo, const FString& InMaterialName, const FString& InLayerName, const FString& InAssetName, uint8 CPDIndex)
		{
			FText DisplayName = FText::Format(LOCTEXT("MaterialAndParameterDisplayName", "{0} (Index: {1}, Asset: {2})"), FText::FromName(InParameterInfo.Name), FText::AsNumber(CPDIndex), FText::FromString(InMaterialName));
			if (!InLayerName.IsEmpty() && !InAssetName.IsEmpty())
			{
				DisplayName = FText::Format(LOCTEXT("MaterialParameterDisplayName", "{0} (Index: {1}, Asset: {2}.{3}.{4})"),
					FText::FromName(InParameterInfo.Name),
					FText::AsNumber(CPDIndex),
					FText::FromString(InMaterialName),
					FText::FromString(InLayerName),
					FText::FromString(InAssetName));
			}
			return DisplayName;
		};

		TSortedMap<uint8, TArray<FCustomPrimitiveDataMaterialParametersData>> CPDMetadata;
		CPDTrack->GetCPDMaterialData(*Player, ObjectBindingID, SequenceID, CPDMetadata);
		if (const TArray<FCustomPrimitiveDataMaterialParametersData>* DataArray = CPDMetadata.Find(StartIndex))
		{
			FTextBuilder UsedInMaterialTextBuilder;
			UsedInMaterialTextBuilder.AppendLine(FText::Format(LOCTEXT("CustomPrimitiveData_Tooltip_HasMaterials", "Custom Primitize Data of type {0} starting at data index {1}. Used for material parameter(s):"), ParameterTypeText, FText::AsNumber(StartIndex)));
			for (const FCustomPrimitiveDataMaterialParametersData& Data : *DataArray)
			{
				if (Data.MaterialParameterType == ParameterType && Data.MaterialAsset.IsValid())
				{
					FMaterialLayersFunctions Layers;
					Data.MaterialAsset->GetMaterialLayers(Layers);
					FString LayerName = GetMaterialParameterLayerName(Layers, Data.ParameterInfo);
					FString AssetName = GetMaterialParameterAssetName(Layers, Data.ParameterInfo);
					FText MaterialDisplayName = GetMaterialParameterDisplayName(Data.ParameterInfo, Data.MaterialAsset->GetName(), LayerName, AssetName, StartIndex);

					UsedInMaterialTextBuilder.AppendLine(MaterialDisplayName);
				}
			}
			return UsedInMaterialTextBuilder.ToText();
		}
		else
		{
			return FText::Format(LOCTEXT("CustomPrimitiveData_Tooltip_NoMaterials", "Custom Primitive Data of type {0} starting at data index {1}. Not used in materials."), ParameterTypeText, FText::AsNumber(StartIndex));
		}
	};

	ChannelsUsedBitmap = 0;
	check(CPDTrack);
	for (FScalarParameterNameAndCurve& Scalar : GetScalarParameterNamesAndCurves())
	{
		FMovieSceneChannelMetaData MetaData(Scalar.ParameterName, FText::Format(LOCTEXT("CustomPrimitiveData_Scalar_DisplayName", "Scalar (Index {0})"), FText::FromName(Scalar.ParameterName)));
		// Should be the start index
		check(Scalar.ParameterName.ToString().IsNumeric());
		uint64 StartIndex = FCString::Atoi(*Scalar.ParameterName.ToString());
		check(StartIndex >= 0 && StartIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats);
		
		// Prevent single channels from collapsing to the track node
		MetaData.bCanCollapseToTrack = false;
		MetaData.SortOrder = StartIndex;
		ChannelsUsedBitmap |= ((uint64)1 << StartIndex);

		MetaData.GetTooltipTextDelegate.BindLambda(GetCPDTooltipText, StartIndex, EMaterialParameterType::Scalar, LOCTEXT("CustomPrimitiveData_ParameterType_Scalar", "Scalar"));

		Channels.Add(Scalar.ParameterCurve, MetaData, TMovieSceneExternalValue<float>());
	}

	for (FVector2DParameterNameAndCurves& Vector2D : GetVector2DParameterNamesAndCurves())
	{
		FString ParameterString = Vector2D.ParameterName.ToString(); 
		check(ParameterString.IsNumeric()); 
		uint64 StartIndex = FCString::Atoi64(*ParameterString);
		check(StartIndex >= 0 && StartIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats - 1);
		ChannelsUsedBitmap |= ((uint64)0b11 << StartIndex);
		FText Group = FText::FromString(ParameterString);
		FString GroupDisplayName = FText::Format(LOCTEXT("CustomPrimitiveData_Vector2D_DisplayName", "Vector2D (Index {0})"), FText::FromName(Vector2D.ParameterName)).ToString();

		FMovieSceneChannelMetaData MetaData_X = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".X")), FCommonChannelData::ChannelX, Group);
		MetaData_X.SortOrder = StartIndex;
		MetaData_X.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, GroupDisplayName);
		MetaData_X.GetGroupTooltipTextDelegate.BindLambda(GetCPDTooltipText, StartIndex, EMaterialParameterType::None, LOCTEXT("CustomPrimitiveData_ParameterType_Vector2D", "Vector2D"));

		FMovieSceneChannelMetaData MetaData_Y = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Y")), FCommonChannelData::ChannelY, Group);
		MetaData_Y.SortOrder = StartIndex + 1;
		MetaData_Y.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, GroupDisplayName);
		MetaData_Y.GetGroupTooltipTextDelegate.BindLambda(GetCPDTooltipText, StartIndex, EMaterialParameterType::None, LOCTEXT("CustomPrimitiveData_ParameterType_Vector2D", "Vector2D"));

		Channels.Add(Vector2D.XCurve, MetaData_X, TMovieSceneExternalValue<float>());
		Channels.Add(Vector2D.YCurve, MetaData_Y, TMovieSceneExternalValue<float>());
	}

	for (FVectorParameterNameAndCurves& Vector : GetVectorParameterNamesAndCurves())
	{
		FString ParameterString = Vector.ParameterName.ToString();
		check(ParameterString.IsNumeric());
		uint64 StartIndex = FCString::Atoi64(*ParameterString);
		check(StartIndex >= 0 && StartIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats - 2);
		ChannelsUsedBitmap |= ((uint64)0b111 << StartIndex);
		FText Group = FText::FromString(ParameterString);
		FString GroupDisplayName = FText::Format(LOCTEXT("CustomPrimitiveData_Vector_DisplayName", "Vector (Index {0})"), FText::FromName(Vector.ParameterName)).ToString();
		
		FMovieSceneChannelMetaData MetaData_X = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".X")), FCommonChannelData::ChannelX, Group);
		MetaData_X.SortOrder = StartIndex;
		MetaData_X.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, GroupDisplayName);
		MetaData_X.GetGroupTooltipTextDelegate.BindLambda(GetCPDTooltipText, StartIndex, EMaterialParameterType::Vector, LOCTEXT("CustomPrimitiveData_ParameterType_Vector", "Vector"));

		FMovieSceneChannelMetaData MetaData_Y = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Y")), FCommonChannelData::ChannelY, Group);
		MetaData_Y.SortOrder = StartIndex + 1;
		MetaData_Y.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, GroupDisplayName);
		MetaData_Y.GetGroupTooltipTextDelegate.BindLambda(GetCPDTooltipText, StartIndex, EMaterialParameterType::Vector, LOCTEXT("CustomPrimitiveData_ParameterType_Vector", "Vector"));

		FMovieSceneChannelMetaData MetaData_Z = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Z")), FCommonChannelData::ChannelZ, Group);
		MetaData_Z.SortOrder = StartIndex + 2;
		MetaData_Z.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, GroupDisplayName);
		MetaData_Z.GetGroupTooltipTextDelegate.BindLambda(GetCPDTooltipText, StartIndex, EMaterialParameterType::Vector, LOCTEXT("CustomPrimitiveData_ParameterType_Vector", "Vector"));

		Channels.Add(Vector.XCurve, MetaData_X, TMovieSceneExternalValue<float>());
		Channels.Add(Vector.YCurve, MetaData_Y, TMovieSceneExternalValue<float>());
		Channels.Add(Vector.ZCurve, MetaData_Z, TMovieSceneExternalValue<float>());
	}

	for (FColorParameterNameAndCurves& Color : GetColorParameterNamesAndCurves())
	{
		FString ParameterString = Color.ParameterName.ToString();
		check(ParameterString.IsNumeric());
		uint64 StartIndex = FCString::Atoi64(*ParameterString);
		check(StartIndex >= 0 && StartIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats - 3);
		ChannelsUsedBitmap |= ((uint64)0b1111 << StartIndex);
		FText Group = FText::FromString(ParameterString);
		FString GroupDisplayName = FText::Format(LOCTEXT("CustomPrimitiveData_Color_DisplayName", "Color (Index {0})"), FText::FromName(Color.ParameterName)).ToString();
		
		FMovieSceneChannelMetaData MetaData_R(*(ParameterString + TEXT("R")), FCommonChannelData::ChannelR, Group);
		MetaData_R.SortOrder = StartIndex;
		MetaData_R.Color = FCommonChannelData::RedChannelColor;
		MetaData_R.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, GroupDisplayName);
		MetaData_R.GetGroupTooltipTextDelegate.BindLambda(GetCPDTooltipText, StartIndex, EMaterialParameterType::Vector, LOCTEXT("CustomPrimitiveData_ParameterType_Color", "Color"));

		FMovieSceneChannelMetaData MetaData_G(*(ParameterString + TEXT("G")), FCommonChannelData::ChannelG, Group);
		MetaData_G.SortOrder = StartIndex + 1;
		MetaData_G.Color = FCommonChannelData::GreenChannelColor;
		MetaData_G.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, GroupDisplayName);
		MetaData_G.GetGroupTooltipTextDelegate.BindLambda(GetCPDTooltipText, StartIndex, EMaterialParameterType::Vector, LOCTEXT("CustomPrimitiveData_ParameterType_Color", "Color"));

		FMovieSceneChannelMetaData MetaData_B(*(ParameterString + TEXT("B")), FCommonChannelData::ChannelB, Group);
		MetaData_B.SortOrder = StartIndex + 2;
		MetaData_B.Color = FCommonChannelData::BlueChannelColor;
		MetaData_B.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, GroupDisplayName);
		MetaData_B.GetGroupTooltipTextDelegate.BindLambda(GetCPDTooltipText, StartIndex, EMaterialParameterType::Vector, LOCTEXT("CustomPrimitiveData_ParameterType_Color", "Color"));

		FMovieSceneChannelMetaData MetaData_A(*(ParameterString + TEXT("A")), FCommonChannelData::ChannelA, Group);
		MetaData_A.PropertyMetaData.Add(FCommonChannelData::GroupDisplayName, GroupDisplayName);
		MetaData_A.GetGroupTooltipTextDelegate.BindLambda(GetCPDTooltipText, StartIndex, EMaterialParameterType::Vector, LOCTEXT("CustomPrimitiveData_ParameterType_Color", "Color"));
		MetaData_A.SortOrder = StartIndex + 3;

		Channels.Add(Color.RedCurve, MetaData_R, TMovieSceneExternalValue<float>());
		Channels.Add(Color.GreenCurve, MetaData_G, TMovieSceneExternalValue<float>());
		Channels.Add(Color.BlueCurve, MetaData_B, TMovieSceneExternalValue<float>());
		Channels.Add(Color.AlphaCurve, MetaData_A, TMovieSceneExternalValue<float>());
	}
#else

	for (FScalarParameterNameAndCurve& Scalar : GetScalarParameterNamesAndCurves())
	{
		Channels.Add(Scalar.ParameterCurve);
	}
	for (FVector2DParameterNameAndCurves& Vector2D : GetVector2DParameterNamesAndCurves())
	{
		Channels.Add(Vector2D.XCurve);
		Channels.Add(Vector2D.YCurve);
	}

	for (FVectorParameterNameAndCurves& Vector : GetVectorParameterNamesAndCurves())
	{
		Channels.Add(Vector.XCurve);
		Channels.Add(Vector.YCurve);
		Channels.Add(Vector.ZCurve);
	}

	for (FColorParameterNameAndCurves& Color : GetColorParameterNamesAndCurves())
	{
		Channels.Add(Color.RedCurve);
		Channels.Add(Color.GreenCurve);
		Channels.Add(Color.BlueCurve);
		Channels.Add(Color.AlphaCurve);
	}

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

void UMovieSceneCustomPrimitiveDataSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	IMovieSceneParameterSectionExtender* Extender = GetImplementingOuter<IMovieSceneParameterSectionExtender>();
	if (!ensureMsgf(Extender, TEXT("It is not valid for a UMovieSceneParameterSection to be used for importing entities outside of an outer chain that implements IMovieSceneParameterSectionExtender")))
	{
		return;
	}

	uint8 ParameterType = 0;
	int16 EntityIndex = 0;
	uint8 Offset = 0;
	DecodeEntityID(Params.EntityID, EntityIndex, ParameterType, Offset);

	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponentTypes = FMovieSceneTracksComponentTypes::Get();

	FGuid ObjectBindingID = Params.GetObjectBindingID();

	TEntityBuilder<TAddConditional<FGuid>> BaseBuilder = FEntityBuilder()
		.AddConditional(BuiltInComponentTypes->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid());

	auto GetParameterName = [](FName BaseOffsetName, uint8 Offset)
	{
		if (Offset == 0)
		{
			return BaseOffsetName;
		}
		FString BaseOffsetString = BaseOffsetName.ToString();
		check(BaseOffsetString.IsNumeric());
		int BaseOffset = FCString::Atoi(*BaseOffsetString);
		return FName(FString::FromInt(BaseOffset + Offset));
	};

	switch (ParameterType)
	{
	case 0:
	{
		const FScalarParameterNameAndCurve& Scalar = ScalarParameterNamesAndCurves[EntityIndex];

		if (Scalar.ParameterCurve.HasAnyData())
		{
			OutImportedEntity->AddBuilder(
				BaseBuilder
				.Add(TracksComponentTypes->ScalarParameterName, Scalar.ParameterName)
				.Add(BuiltInComponentTypes->FloatChannel[0], &Scalar.ParameterCurve)
			);
		}
		break;
	}
	case 1:
	{
		const FVector2DParameterNameAndCurves& Vector2D = Vector2DParameterNamesAndCurves[EntityIndex];
		check(Offset >= 0 && Offset <= 1);
		const FMovieSceneFloatChannel& Vector2DCurve = Offset == 0 ? Vector2D.XCurve : Vector2D.YCurve;
		if (Vector2DCurve.HasAnyData())
		{
			OutImportedEntity->AddBuilder(
				BaseBuilder
				.Add(TracksComponentTypes->ScalarParameterName, GetParameterName(Vector2D.ParameterName, Offset))
				.AddConditional(BuiltInComponentTypes->FloatChannel[0], &Vector2DCurve, Vector2DCurve.HasAnyData())
			);
		}
		break;
	}
	case 2:
	{
		const FVectorParameterNameAndCurves& Vector = VectorParameterNamesAndCurves[EntityIndex];

		check(Offset >= 0 && Offset <= 2);
		const FMovieSceneFloatChannel& VectorCurve = Offset == 0 ? Vector.XCurve : Offset == 1 ? Vector.YCurve : Vector.ZCurve;
		if (VectorCurve.HasAnyData())
		{
			OutImportedEntity->AddBuilder(
				BaseBuilder
				.Add(TracksComponentTypes->ScalarParameterName, GetParameterName(Vector.ParameterName, Offset))
				.AddConditional(BuiltInComponentTypes->FloatChannel[0], &VectorCurve, VectorCurve.HasAnyData())
			);
		}
		break;
	}
	case 3:
	{
		const FColorParameterNameAndCurves& Color = ColorParameterNamesAndCurves[EntityIndex];
		check(Offset >= 0 && Offset <= 3);
		const FMovieSceneFloatChannel& ColorCurve = Offset == 0 ? Color.RedCurve : Offset == 1 ? Color.GreenCurve : Offset == 2 ? Color.BlueCurve : Color.AlphaCurve;
		if (ColorCurve.HasAnyData())
		{
			OutImportedEntity->AddBuilder(
				BaseBuilder
				.Add(TracksComponentTypes->ScalarParameterName, GetParameterName(Color.ParameterName, Offset))
				.AddTag(TracksComponentTypes->Tags.CustomPrimitiveData)
				.AddConditional(BuiltInComponentTypes->FloatChannel[0], &ColorCurve, ColorCurve.HasAnyData())
			);
		}
		break;
	}
	}

	Extender->ExtendEntity(this, EntityLinker, Params, OutImportedEntity);
}

void UMovieSceneCustomPrimitiveDataSection::ExternalPopulateEvaluationField(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	using namespace UE::MovieScene;

	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	// We add a separate entity for each float channel, and encode in the name the type, the index, and the offset (for each channel)

	const int32 NumScalarID = ScalarParameterNamesAndCurves.Num();
	const int32 NumVector2DID = Vector2DParameterNamesAndCurves.Num();
	const int32 NumVectorID = VectorParameterNamesAndCurves.Num();
	const int32 NumColorID = ColorParameterNamesAndCurves.Num();

	for (int32 Index = 0; Index < NumScalarID; ++Index)
	{
		const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 0, 0));
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
	}
	for (int32 Index = 0; Index < NumVector2DID; ++Index)
	{
		const int32 EntityIndex0 = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 1, 0));
		const int32 EntityIndex1 = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 1, 1));
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex0, MetaDataIndex);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex1, MetaDataIndex);
	}
	for (int32 Index = 0; Index < NumVectorID; ++Index)
	{
		const int32 EntityIndex0 = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 2, 0));
		const int32 EntityIndex1 = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 2, 1));
		const int32 EntityIndex2 = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 2, 2));
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex0, MetaDataIndex);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex1, MetaDataIndex);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex2, MetaDataIndex);
	}
	for (int32 Index = 0; Index < NumColorID; ++Index)
	{
		const int32 EntityIndex0 = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 3, 0));
		const int32 EntityIndex1 = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 3, 1));
		const int32 EntityIndex2 = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 3, 2));
		const int32 EntityIndex3 = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 3, 3));
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex0, MetaDataIndex);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex1, MetaDataIndex);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex2, MetaDataIndex);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex3, MetaDataIndex);
	}
}


#undef LOCTEXT_NAMESPACE