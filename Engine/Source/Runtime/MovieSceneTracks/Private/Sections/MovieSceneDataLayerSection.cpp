// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneDataLayerSection.h"
#include "MovieSceneTracksComponentTypes.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDataLayerSection)

UMovieSceneDataLayerSection::UMovieSceneDataLayerSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	DesiredState = EDataLayerRuntimeState::Activated;
	PrerollState = EDataLayerRuntimeState::Activated;
	bFlushOnActivated = true;
	bFlushOnUnload = false;
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::RestoreState);
}

EDataLayerRuntimeState UMovieSceneDataLayerSection::GetDesiredState() const
{
	return DesiredState;
}

void UMovieSceneDataLayerSection::SetDesiredState(EDataLayerRuntimeState InDesiredState)
{
	DesiredState = InDesiredState;
}

EDataLayerRuntimeState UMovieSceneDataLayerSection::GetPrerollState() const
{
	return PrerollState;
}

void UMovieSceneDataLayerSection::SetPrerollState(EDataLayerRuntimeState InPrerollState)
{
	PrerollState = InPrerollState;
}

bool UMovieSceneDataLayerSection::GetFlushOnActivated() const
{
	return bFlushOnActivated;
}

void UMovieSceneDataLayerSection::SetFlushOnActivated(bool bInFlushOnActivated)
{
	bFlushOnActivated = bInFlushOnActivated;
}

bool UMovieSceneDataLayerSection::GetFlushOnUnload() const
{
	return bFlushOnUnload;
}

void UMovieSceneDataLayerSection::SetFlushOnUnload(bool bInFlushOnUnload)
{
	bFlushOnUnload = bInFlushOnUnload;
}

void UMovieSceneDataLayerSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FMovieSceneDataLayerComponentData ComponentData{decltype(FMovieSceneDataLayerComponentData::Section)(this) };

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(FMovieSceneTracksComponentTypes::Get()->DataLayer, ComponentData)
	);
}

