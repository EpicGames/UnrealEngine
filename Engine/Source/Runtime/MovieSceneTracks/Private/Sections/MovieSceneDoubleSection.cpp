// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneDoubleSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "Evaluation/MovieSceneEvaluationCustomVersion.h"
#include "Evaluation/MovieScenePropertyTemplate.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/MovieSceneDoublePropertySystem.h"
#include "Tracks/MovieSceneDoubleTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "UObject/SequencerObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDoubleSection)

UMovieSceneDoubleSection::UMovieSceneDoubleSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);
	BlendType = EMovieSceneBlendType::Absolute;
	bSupportsInfiniteRange = true;
}

EMovieSceneChannelProxyType UMovieSceneDoubleSection::CacheChannelProxy()
{
#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(DoubleCurve, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<double>::Make());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(DoubleCurve);

#endif

	return EMovieSceneChannelProxyType::Static;
}

bool UMovieSceneDoubleSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneDoubleSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!DoubleCurve.HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->Double)
		.Add(Components->DoubleChannel[0], &DoubleCurve)
		.Commit(this, Params, OutImportedEntity);
}

