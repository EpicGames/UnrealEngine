// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneEnumSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "UObject/SequencerObjectVersion.h"
#include "UObject/SequencerObjectVersion.h"
#include "UObject/SequencerObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEnumSection)

UMovieSceneEnumSection::UMovieSceneEnumSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);
	bSupportsInfiniteRange = true;
#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(EnumCurve, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<uint8>::Make());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(EnumCurve);

#endif
}

bool UMovieSceneEnumSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneEnumSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!EnumCurve.HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->Enum)
		.Add(Components->ByteChannel, &EnumCurve)
		.Commit(this, Params, OutImportedEntity);
}

