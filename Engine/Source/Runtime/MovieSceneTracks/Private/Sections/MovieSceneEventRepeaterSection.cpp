// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneEventRepeaterSection.h"
#include "Tracks/MovieSceneEventTrack.h"

#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneSequenceInstance.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Systems/MovieSceneEventSystems.h"
#include "MovieSceneTracksComponentTypes.h"

#include "Evaluation/MovieSceneEvaluationField.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEventRepeaterSection)


void UMovieSceneEventRepeaterSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (Event.Ptrs.Function == nullptr)
	{
		return;
	}

	UMovieSceneEventTrack*   EventTrack     = GetTypedOuter<UMovieSceneEventTrack>();
	const FSequenceInstance& ThisInstance   = EntityLinker->GetInstanceRegistry()->GetInstance(Params.Sequence.InstanceHandle);
	FMovieSceneContext       Context        = ThisInstance.GetContext();

	if (Context.IsSilent())
	{
		return;
	}
	else if (Context.GetDirection() == EPlayDirection::Forwards && !EventTrack->bFireEventsWhenForwards)
	{
		return;
	}
	else if (Context.GetDirection() == EPlayDirection::Backwards && !EventTrack->bFireEventsWhenBackwards)
	{
		return;
	}
	else if (!GetRange().Contains(Context.GetTime().FrameNumber))
	{
		return;
	}

	UMovieSceneEventSystem* EventSystem = nullptr;
	bool bMimicChanged = false;

	if (EventTrack->EventPosition == EFireEventsAtPosition::AtStartOfEvaluation)
	{
		bMimicChanged = true;
		EventSystem = EntityLinker->LinkSystem<UMovieScenePreSpawnEventSystem>();
	}
	else if (EventTrack->EventPosition == EFireEventsAtPosition::AfterSpawn)
	{
		bMimicChanged = true;
		EventSystem = EntityLinker->LinkSystem<UMovieScenePostSpawnEventSystem>();
	}
	else
	{
		EventSystem = EntityLinker->LinkSystem<UMovieScenePostEvalEventSystem>();
	}

	TOptional<FFrameTime> RootTime = Context.GetSequenceToRootSequenceTransform().TryTransformTime(Context.GetTime());
	if (!RootTime)
	{
		return;
	}

	FMovieSceneEventTriggerData TriggerData = {
		Event.Ptrs,
		Params.GetObjectBindingID(),
		ThisInstance.GetSequenceID(),
		RootTime.GetValue()
	};

	EventSystem->AddEvent(ThisInstance.GetRootInstanceHandle(), TriggerData);

	if (bMimicChanged)
	{
		// Mimic the structure changing in order to ensure that the instantiation phase runs
		EntityLinker->EntityManager.MimicStructureChanged();
	}
}

bool UMovieSceneEventRepeaterSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	OutFieldBuilder->AddOneShotEntity(EffectiveRange, this, 0, OutFieldBuilder->AddMetaData(InMetaData));
	return true;
}

