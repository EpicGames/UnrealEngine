// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneEventSection.h"
#include "EngineGlobals.h"
#include "IMovieScenePlayer.h"
#include "UObject/ReleaseObjectVersion.h"
#include "UObject/LinkerLoad.h"
#include "MovieSceneFwd.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Serialization/MemoryArchive.h"
#include "StructUtils/UserDefinedStruct.h"
#include "MovieSceneFrameMigration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEventSection)

/* Custom version specifically for event parameter struct serialization (serialized into FMovieSceneEventParameters::StructBytes) */
namespace EEventParameterVersion
{
	enum Type
	{
		// First version, serialized with either FMemoryWriter or FEventParameterWriter (both are compatible with FEventParameterReader)
		First = 0,

		// -------------------------------------------------------------------
		LastPlusOne,
		LatestVersion = LastPlusOne - 1
	};
}

namespace
{
	/* Register the custom version so that we can easily make changes to this serialization in future */
	const FGuid EventParameterVersionGUID(0x509D354F, 0xF6E6492F, 0xA74985B2, 0x073C631C);
	FCustomVersionRegistration GRegisterEventParameterVersion(EventParameterVersionGUID, EEventParameterVersion::LatestVersion, TEXT("EventParameter"));

	/** Magic number that is always added to the start of a serialized event parameter to signify that it has a custom version header. Absense implies no custom version (before version info was added) */
	static const uint32 VersionMagicNumber = 0xA1B2C3D4;
}

/** Custom archive overloads for serializing event struct parameter payloads */
class FEventParameterArchive : public FMemoryArchive
{
public:
	/** Serialize a soft object path */
	virtual FArchive& operator<<(FSoftObjectPath& Value) override
	{
		Value.SerializePath(*this);
		return *this;
	}

	/** Serialize a soft object ptr */
	virtual FArchive& operator<<(FSoftObjectPtr& Value) override
	{
		FSoftObjectPath Ref = Value.ToSoftObjectPath();
		*this << Ref;

		if (IsLoading())
		{
			Value = FSoftObjectPtr(Ref);
		}

		return *this;
	}

	// Unsupported serialization
	virtual FArchive& operator<<(UObject*& Res) override 					{ SetError(); return *this; }
	virtual FArchive& operator<<(FLazyObjectPtr& LazyObjectPtr) override 	{ SetError(); return *this; }
	virtual FArchive& operator<<(FWeakObjectPtr& Value) override 			{ SetError(); return *this; }
};

/** Custom archive used for writing event parameter struct payloads */
class FEventParameterWriter final : public FEventParameterArchive
{
public:
	/** Constructor from a destination byte array */
	FEventParameterWriter(TArray<uint8>& InBytes)
		: Bytes(InBytes)
	{
		ArNoDelta = true;
		this->SetIsSaving(true);
		this->SetIsPersistent(true);
		UsingCustomVersion(EventParameterVersionGUID);
	}

	/**
	 * Write the specified source ptr (an instance of StructPtr) into the destination byte array
	 * @param StructPtr 	The struct representing the type of Source
	 * @param Source 		Source pointer to the instance of the payload to write
	 */
	void Write(UScriptStruct* StructPtr, uint8* Source)
	{
		FArchive& Ar = *this;

		// Write the magic number to signify that we have the custom version info
		uint32 Magic = VersionMagicNumber;
		Ar << Magic;

		// Store the position of the serialized CVOffset
		int64 CVOffsetPos = Tell();

		int32 CVOffset = 0;
		Ar << CVOffset;

		// Write the struct itself
		StructPtr->SerializeTaggedProperties(Ar, Source, StructPtr, nullptr);

		CVOffset = Tell();

		// Write the custom version info at the end (it may have changed as a result of SerializeTaggedProperties if they use custom versions)
		FCustomVersionContainer CustomVersions = GetCustomVersions();
		CustomVersions.Serialize(*this);

		// Seek back to the offset pos, and write the custom version info offset
		Seek(CVOffsetPos);
		Ar << CVOffset;
	}

	virtual FString GetArchiveName() const override
	{
		return TEXT("FEventParameterWriter");
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		const int64 NumBytesToAdd = Offset + Num - Bytes.Num();
		if (NumBytesToAdd > 0)
		{
			const int64 NewArrayCount = Bytes.Num() + NumBytesToAdd;
			check(NewArrayCount < MAX_int32);

			Bytes.AddUninitialized( (int32)NumBytesToAdd );
		}

		if (Num)
		{
			FMemory::Memcpy(&Bytes[Offset], Data, Num);
			Offset += Num;
		}
	}

private:
	TArray<uint8>& Bytes;
};

class FEventParameterReader final : public FEventParameterArchive
{
public:
	FEventParameterReader(const TArray<uint8>& InBytes)
		: Bytes(InBytes)
	{
		this->SetIsLoading(true);
		UsingCustomVersion(EventParameterVersionGUID);
	}

	/**
	 * Read the source data buffer as a StructPtr type, into the specified destination instance
	 * @param StructPtr 	The struct representing the type of Dest
	 * @param Dest 			Destination instance to receive the deserialized data
	 */
	void Read(UScriptStruct* StructPtr, uint8* Dest)
	{
		bool bHasCustomVersion = false;
		// Optionally deserialize the custom version header, provided it was serialized
		if (Bytes.Num() >= 8)
		{
			uint32 Magic = 0;
			*this << Magic;

			if (Magic == VersionMagicNumber)
			{
				int32 CVOffset = 0;
				*this << CVOffset;

				int64 DataStartPos = Tell();
				Seek(CVOffset);

				// Read the custom version info
				FCustomVersionContainer CustomVersions;
				CustomVersions.Serialize(*this);
				SetCustomVersions(CustomVersions);

				// Seek back to the start of the struct data
				Seek(DataStartPos);

				bHasCustomVersion = true;
			}
		}

		if (!bHasCustomVersion)
		{
			// Force the very first custom version
			SetCustomVersion(EventParameterVersionGUID, EEventParameterVersion::First, "EventParameter");
			// The magic number was not valid, so ensure we're right at the start (this data pre-dates the custom version info)
			Seek(0);
		}

		// Serialize the struct itself
		StructPtr->SerializeTaggedProperties(*this, Dest, StructPtr, nullptr);
	}

	virtual FString GetArchiveName() const override
	{
		return TEXT("FEventParameterReader");
	}

	virtual void Serialize(void* Data, int64 Num) override
	{
		if (Num && !IsError())
		{
			// Only serialize if we have the requested amount of data
			if (Offset + Num <= Bytes.Num())
			{
				FMemory::Memcpy(Data, &Bytes[Offset], Num);
				Offset += Num;
			}
			else
			{
				SetError();
			}
		}
	}

private:
	const TArray<uint8>& Bytes;
};

bool operator==(const FMovieSceneEventParameters& A, const FMovieSceneEventParameters& B)
{
	UScriptStruct* StructA = A.GetStructType();
	UScriptStruct* StructB = B.GetStructType();

	if (StructA != StructB)
	{
		return false;
	}

	if (!StructA)
	{
		return true;
	}

	FStructOnScope StructContainerA(StructA);
	A.GetInstance(StructContainerA);
	uint8* InstA = StructContainerA.GetStructMemory();

	FStructOnScope StructContainerB(StructB);
	B.GetInstance(StructContainerB);
	uint8* InstB = StructContainerB.GetStructMemory();

	return InstA ? StructA->CompareScriptStruct(InstA, InstB, 0) : InstB == nullptr;
}

bool operator!=(const FMovieSceneEventParameters& A, const FMovieSceneEventParameters& B)
{
	return !(A == B);
}

void FMovieSceneEventParameters::OverwriteWith(uint8* InstancePtr)
{
	check(InstancePtr);

	if (UScriptStruct* StructPtr = GetStructType())
	{
		FEventParameterWriter(StructBytes).Write(StructPtr, InstancePtr);
	}
	else
	{
		StructBytes.Reset();
	}
}

void FMovieSceneEventParameters::GetInstance(FStructOnScope& OutStruct) const
{
	UScriptStruct* StructPtr = GetStructType();
	OutStruct.Initialize(StructPtr);

#if WITH_EDITOR
	if (UUserDefinedStruct* UDS = Cast<UUserDefinedStruct>(StructPtr))
	{
		UDS->InitializeDefaultValue(OutStruct.GetStructMemory());
	}
#endif

	uint8* Memory = OutStruct.GetStructMemory();
	if (StructPtr && StructPtr->GetStructureSize() > 0 && StructBytes.Num())
	{
		// Deserialize the struct bytes into the struct memory
		FEventParameterReader ParamReader(StructBytes);
		ParamReader.SetUEVer(PackageFileVersion);
		ParamReader.SetLicenseeUEVer(LicenseePackageFileVersion);
		ParamReader.Read(StructPtr, Memory);
	}
}

bool FMovieSceneEventParameters::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	if (Ar.IsLoading() && Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::EventSectionParameterStringAssetRef)
	{
		UScriptStruct* StructPtr = nullptr;
		Ar << StructPtr;
		StructType = StructPtr;
	}
	else
	{
		Ar << StructType;
	}
	
	Ar << StructBytes;

	if (Ar.IsLoading())
	{
		PackageFileVersion = Ar.UEVer();
		LicenseePackageFileVersion = Ar.LicenseeUEVer();
	}

	return true;
}

#if WITH_EDITORONLY_DATA
void FMovieSceneEventSectionData::PostSerialize(const FArchive& Ar)
{
	
	if (KeyTimes_DEPRECATED.Num())
	{
		FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();

		TArray<FEventPayload> OldValues = KeyValues;
		Times.Reset(KeyTimes_DEPRECATED.Num());
		KeyValues.Reset(KeyTimes_DEPRECATED.Num());
		for (int32 Index = 0; Index < KeyTimes_DEPRECATED.Num(); ++Index)
		{
			FFrameNumber KeyTime = UpgradeLegacyMovieSceneTime(nullptr, LegacyFrameRate, KeyTimes_DEPRECATED[Index]);
			ConvertInsertAndSort<FEventPayload>(Index, KeyTime, OldValues[Index], Times, KeyValues);
		}
		KeyTimes_DEPRECATED.Empty();
	}
}
#endif

void FMovieSceneEventSectionData::GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles)
{
	GetData().GetKeys(WithinRange, OutKeyTimes, OutKeyHandles);
}

void FMovieSceneEventSectionData::GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes)
{
	GetData().GetKeyTimes(InHandles, OutKeyTimes);
}

void FMovieSceneEventSectionData::SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes)
{
	GetData().SetKeyTimes(InHandles, InKeyTimes);
}

void FMovieSceneEventSectionData::DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles)
{
	GetData().DuplicateKeys(InHandles, OutNewHandles);
}

void FMovieSceneEventSectionData::DeleteKeys(TArrayView<const FKeyHandle> InHandles)
{
	GetData().DeleteKeys(InHandles);
}

void FMovieSceneEventSectionData::DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore)
{
	GetData().DeleteKeysFrom(InTime, bDeleteKeysBefore);
}

void FMovieSceneEventSectionData::RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer)
{
	GetData().RemapTimes(Retimer);
}

TRange<FFrameNumber> FMovieSceneEventSectionData::ComputeEffectiveRange() const
{
	return GetData().GetTotalRange();
}

int32 FMovieSceneEventSectionData::GetNumKeys() const
{
	return Times.Num();
}

void FMovieSceneEventSectionData::Reset()
{
	Times.Reset();
	KeyValues.Reset();
	KeyHandles.Reset();
}

void FMovieSceneEventSectionData::Offset(FFrameNumber DeltaPosition)
{
	GetData().Offset(DeltaPosition);
}


FKeyHandle FMovieSceneEventSectionData::GetHandle(int32 Index)
{
	return GetData().GetHandle(Index);
}

int32 FMovieSceneEventSectionData::GetIndex(FKeyHandle Handle)
{
	return GetData().GetIndex(Handle);
}

/* UMovieSceneSection structors
 *****************************************************************************/

UMovieSceneEventSection::UMovieSceneEventSection()
{
#if WITH_EDITORONLY_DATA
	bIsInfinite_DEPRECATED = true;
#endif
	bSupportsInfiniteRange = true;
	SetRange(TRange<FFrameNumber>::All());

#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(EventData, FMovieSceneChannelMetaData());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(EventData);

#endif
}

void UMovieSceneEventSection::PostLoad()
{
	if (Events_DEPRECATED.GetKeys().Num())
	{
		TMovieSceneChannelData<FEventPayload> ChannelData = EventData.GetData();

		FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();

		for (FNameCurveKey EventKey : Events_DEPRECATED.GetKeys())
		{
			FFrameNumber KeyTime = UpgradeLegacyMovieSceneTime(this, LegacyFrameRate, EventKey.Time);
			ChannelData.AddKey(KeyTime, FEventPayload(EventKey.Value));
		}

		MarkAsChanged();
	}

	Super::PostLoad();
}
