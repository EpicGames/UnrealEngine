// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneEventSectionBase.h"
#include "Modules/ModuleManager.h"
#include "Evaluation/MovieSceneEvaluationCustomVersion.h"
#include "UObject/ReleaseObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEventSectionBase)

#if WITH_EDITOR

#include "Engine/Blueprint.h"
#include "Kismet2/BlueprintEditorUtils.h"

UMovieSceneEventSectionBase::FFixupPayloadParameterNameEvent UMovieSceneEventSectionBase::FixupPayloadParameterNameEvent;
UMovieSceneEventSectionBase::FUpgradeLegacyEventEndpoint UMovieSceneEventSectionBase::UpgradeLegacyEventEndpoint;
UMovieSceneEventSectionBase::FPostDuplicateEvent UMovieSceneEventSectionBase::PostDuplicateSectionEvent;
UMovieSceneEventSectionBase::FRemoveForCookEvent UMovieSceneEventSectionBase::RemoveForCookEvent;

void UMovieSceneEventSectionBase::OnPostCompile(UBlueprint* Blueprint)
{
	if (Blueprint->GeneratedClass)
	{
		for (FMovieSceneEvent& EntryPoint : GetAllEntryPoints())
		{
			if (EntryPoint.CompiledFunctionName != NAME_None)
			{
				// @todo: Validate that the function is good
				EntryPoint.Ptrs.Function = Blueprint->GeneratedClass->FindFunctionByName(EntryPoint.CompiledFunctionName);

				if (EntryPoint.Ptrs.Function && EntryPoint.BoundObjectPinName != NAME_None)
				{
					EntryPoint.Ptrs.BoundObjectProperty = EntryPoint.Ptrs.Function->FindPropertyByName(EntryPoint.BoundObjectPinName);
					check(!EntryPoint.Ptrs.BoundObjectProperty.Get() || EntryPoint.Ptrs.BoundObjectProperty->GetOwner<UObject>() == EntryPoint.Ptrs.Function);
					if (CastField<FObjectProperty>(EntryPoint.Ptrs.BoundObjectProperty.Get()) || CastField<FInterfaceProperty>(EntryPoint.Ptrs.BoundObjectProperty.Get()))
					{
					}
				}
				else
				{
					EntryPoint.Ptrs.BoundObjectProperty = nullptr;
				}
			}
			else
			{
				EntryPoint.Ptrs.Function = nullptr;
				EntryPoint.Ptrs.BoundObjectProperty = nullptr;
			}

			EntryPoint.CompiledFunctionName = NAME_None;
		}

		if (!Blueprint->bIsRegeneratingOnLoad)
		{
			MarkAsChanged();
			MarkPackageDirty();
		}
	}

	Blueprint->OnCompiled().RemoveAll(this);
}

void UMovieSceneEventSectionBase::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	PostDuplicateSectionEvent.Execute(this);
}

void UMovieSceneEventSectionBase::PostRename(UObject* OldOuter, const FName OldName)
{
	if (OldOuter != GetOuter())
	{
		Super::PostRename(OldOuter, OldName);

		PostDuplicateSectionEvent.Execute(this);
	}
}

void UMovieSceneEventSectionBase::RemoveForCook()
{
	RemoveForCookEvent.Execute(this);

	Super::RemoveForCook();
}

void UMovieSceneEventSectionBase::AttemptUpgrade()
{
	if (!bDataUpgradeRequired)
	{
		return;
	}

	const bool bUpgradeSuccess = UpgradeLegacyEventEndpoint.IsBound() ? UpgradeLegacyEventEndpoint.Execute(this) : false;
	if (bUpgradeSuccess)
	{
		bDataUpgradeRequired = false;
	}
}

#endif


UMovieSceneEventSectionBase::UMovieSceneEventSectionBase(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
#if WITH_EDITOR
	bDataUpgradeRequired = true;
#endif
}

void UMovieSceneEventSectionBase::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FMovieSceneEvaluationCustomVersion::GUID);
	Ar.UsingCustomVersion(FReleaseObjectVersion::GUID);

	Super::Serialize(Ar);

#if WITH_EDITOR

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FMovieSceneEvaluationCustomVersion::GUID) < FMovieSceneEvaluationCustomVersion::DeprecateEventGUIDs
			|| Ar.CustomVer(FReleaseObjectVersion::GUID) < FReleaseObjectVersion::FixupCopiedEventSections)
		{
			AttemptUpgrade();
		}
		else
		{
			bDataUpgradeRequired = false;
		}
	}
#endif
}
