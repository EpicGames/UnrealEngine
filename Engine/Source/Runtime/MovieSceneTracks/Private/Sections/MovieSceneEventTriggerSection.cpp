// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneEventTriggerSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Tracks/MovieSceneEventTrack.h"
#include "Systems/MovieSceneEventSystems.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Evaluation/MovieSceneEvaluationField.h"

#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEventTriggerSection)


UMovieSceneEventTriggerSection::UMovieSceneEventTriggerSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	bSupportsInfiniteRange = true;
	SetRange(TRange<FFrameNumber>::All());

#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(EventChannel, FMovieSceneChannelMetaData());

#endif
}

void UMovieSceneEventTriggerSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const int32 EventIndex = static_cast<int32>(Params.EntityID);

	TArrayView<const FFrameNumber>     Times  = EventChannel.GetData().GetTimes();
	TArrayView<const FMovieSceneEvent> Events = EventChannel.GetData().GetValues();
	if (!ensureMsgf(Events.IsValidIndex(EventIndex), TEXT("Attempting to import an event entity for an invalid index (Index: %d, Num: %d)"), EventIndex, Events.Num()))
	{
		return;
	}

	if (Events[EventIndex].Ptrs.Function == nullptr)
	{
		return;
	}

	UMovieSceneEventTrack*   EventTrack     = GetTypedOuter<UMovieSceneEventTrack>();
	const FSequenceInstance& ThisInstance   = EntityLinker->GetInstanceRegistry()->GetInstance(Params.Sequence.InstanceHandle);
	FMovieSceneContext       Context        = ThisInstance.GetContext();

	// Don't allow events to fire when playback is in a stopped state. This can occur when stopping 
	// playback and returning the current position to the start of playback. It's not desireable to have 
	// all the events from the last playback position to the start of playback be fired.
	if (Context.GetStatus() == EMovieScenePlayerStatus::Stopped || Context.IsSilent())
	{
		return;
	}
	else if (Context.GetDirection() == EPlayDirection::Forwards && !EventTrack->bFireEventsWhenForwards)
	{
		return;
	}
	else if (Context.GetDirection() == EPlayDirection::Backwards && !EventTrack->bFireEventsWhenBackwards)
	{
		return;
	}

	UMovieSceneEventSystem* EventSystem = nullptr;
	bool bMimicChanged = false;

	if (EventTrack->EventPosition == EFireEventsAtPosition::AtStartOfEvaluation)
	{
		bMimicChanged = true;
		EventSystem = EntityLinker->LinkSystem<UMovieScenePreSpawnEventSystem>();
	}
	else if (EventTrack->EventPosition == EFireEventsAtPosition::AfterSpawn)
	{
		bMimicChanged = true;
		EventSystem = EntityLinker->LinkSystem<UMovieScenePostSpawnEventSystem>();
	}
	else
	{
		EventSystem = EntityLinker->LinkSystem<UMovieScenePostEvalEventSystem>();
	}

	TOptional<FFrameTime> RootTime = Context.GetSequenceToRootSequenceTransform().TryTransformTime(Times[EventIndex]);
	if (!RootTime)
	{
		return;
	}

	FMovieSceneEventTriggerData TriggerData = {
		Events[EventIndex].Ptrs,
		Params.GetObjectBindingID(),
		ThisInstance.GetSequenceID(),
		RootTime.GetValue()
	};

	EventSystem->AddEvent(ThisInstance.GetRootInstanceHandle(), TriggerData);

	if (bMimicChanged)
	{
		// Mimic the structure changing in order to ensure that the instantiation phase runs
		EntityLinker->EntityManager.MimicStructureChanged();
	}
}

bool UMovieSceneEventTriggerSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	TArrayView<const FFrameNumber> Times = EventChannel.GetData().GetTimes();
	for (int32 Index = 0; Index < Times.Num(); ++Index)
	{
		if (EffectiveRange.Contains(Times[Index]))
		{
			TRange<FFrameNumber> Range(Times[Index]);
			OutFieldBuilder->AddOneShotEntity(Range, this, Index, MetaDataIndex);
		}
	}

	return true;
}

