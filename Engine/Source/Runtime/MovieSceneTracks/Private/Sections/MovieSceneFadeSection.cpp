// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneFadeSection.h"
#include "UObject/SequencerObjectVersion.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneFadeSection)


/* UMovieSceneFadeSection structors
 *****************************************************************************/

UMovieSceneFadeSection::UMovieSceneFadeSection()
	: FadeColor(FLinearColor::Black)
	, bFadeAudio(false)
{
#if WITH_EDITORONLY_DATA
	bIsInfinite_DEPRECATED = true;
#endif

	SetRange(TRange<FFrameNumber>::All());

	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);

	BlendType = EMovieSceneBlendType::Absolute;
	bSupportsInfiniteRange = true;

#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(FloatCurve, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<float>::Make());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(FloatCurve);

#endif
}

void UMovieSceneFadeSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!FloatCurve.HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddTag(BuiltInComponents->Tags.Root)
		.Add(BuiltInComponents->FloatChannel[0], &FloatCurve)
		.Add(TrackComponents->Fade, FFadeComponentData{ FadeColor, (bool)bFadeAudio })
	);
}

