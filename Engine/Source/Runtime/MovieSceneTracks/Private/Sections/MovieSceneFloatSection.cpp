// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneFloatSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Channels/MovieSceneSectionChannelOverrideRegistry.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "Evaluation/MovieSceneEvaluationCustomVersion.h"
#include "Evaluation/MovieScenePropertyTemplate.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/MovieSceneFloatPropertySystem.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "UObject/SequencerObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneFloatSection)

UMovieSceneFloatSection::UMovieSceneFloatSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);
	BlendType = EMovieSceneBlendType::Absolute;
	bSupportsInfiniteRange = true;
}

EMovieSceneChannelProxyType UMovieSceneFloatSection::CacheChannelProxy()
{
	using namespace UE::MovieScene;

#if WITH_EDITOR

	ChannelProxy = MakeChannelProxy(OverrideRegistry, FloatCurve, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<float>::Make());

#else

	ChannelProxy = MakeChannelProxy(OverrideRegistry, FloatCurve);

#endif

	return EMovieSceneChannelProxyType::Static;
}

bool UMovieSceneFloatSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneFloatSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!FloatCurve.HasAnyData() && (!OverrideRegistry || OverrideRegistry->NumChannels() == 0))
	{
		return;
	}

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackWithOverridableChannelsEntityImportHelper(TracksComponents->Float, this)
		.Add(Components->FloatChannel[0], &FloatCurve,
			// NAME_None below because we don't specify any names in CacheChannelProxy
			FChannelOverrideEntityImportParams{ NAME_None, Components->DoubleResult[0] })
		.Commit(this, Params, OutImportedEntity);
}

UMovieSceneSectionChannelOverrideRegistry* UMovieSceneFloatSection::GetChannelOverrideRegistry(bool bCreateIfMissing)
{
	if (bCreateIfMissing && OverrideRegistry == nullptr)
	{
		OverrideRegistry = NewObject<UMovieSceneSectionChannelOverrideRegistry>(this, NAME_None, RF_Transactional);
	}
	return OverrideRegistry;
}

UE::MovieScene::FChannelOverrideProviderTraitsHandle UMovieSceneFloatSection::GetChannelOverrideProviderTraits() const
{
	// NAME_None because we don't specify any names in CacheChannelProxy
	UE::MovieScene::TSingleChannelOverrideProviderTraits<FMovieSceneFloatChannel> Traits(NAME_None);
	return UE::MovieScene::FChannelOverrideProviderTraitsHandle(Traits);
}

void UMovieSceneFloatSection::OnChannelOverridesChanged()
{
	ChannelProxy = nullptr;
}

#if WITH_EDITOR

void UMovieSceneFloatSection::PostPaste()
{
	Super::PostPaste();
	
	if (OverrideRegistry)
	{
		OverrideRegistry->OnPostPaste();
	}
}

#endif

