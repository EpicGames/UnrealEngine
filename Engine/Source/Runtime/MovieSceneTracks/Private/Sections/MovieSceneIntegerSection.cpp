// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneIntegerSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/MovieScenePiecewiseIntegerBlenderSystem.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "UObject/SequencerObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneIntegerSection)

UMovieSceneIntegerSection::UMovieSceneIntegerSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);
	BlendType = EMovieSceneBlendType::Absolute;
	bSupportsInfiniteRange = true;
#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(IntegerCurve, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<int32>::Make());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(IntegerCurve);

#endif
}

bool UMovieSceneIntegerSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneIntegerSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!IntegerCurve.HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->Integer)
		.Add(Components->IntegerChannel, &IntegerCurve)
		.Add(Components->BlenderType, UMovieScenePiecewiseIntegerBlenderSystem::StaticClass())
		.Commit(this, Params, OutImportedEntity);
}

