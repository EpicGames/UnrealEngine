// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneLevelVisibilitySection.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/MovieSceneLevelVisibilitySystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLevelVisibilitySection)


UMovieSceneLevelVisibilitySection::UMovieSceneLevelVisibilitySection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	Visibility = ELevelVisibility::Visible;
}


ELevelVisibility UMovieSceneLevelVisibilitySection::GetVisibility() const
{
	return Visibility;
}


void UMovieSceneLevelVisibilitySection::SetVisibility( ELevelVisibility InVisibility )
{
	Visibility = InVisibility;
}

void UMovieSceneLevelVisibilitySection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FLevelVisibilityComponentData LevelVisibilityData{decltype(FLevelVisibilityComponentData::Section)(this) };

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(FMovieSceneTracksComponentTypes::Get()->LevelVisibility, LevelVisibilityData)
	);
}

