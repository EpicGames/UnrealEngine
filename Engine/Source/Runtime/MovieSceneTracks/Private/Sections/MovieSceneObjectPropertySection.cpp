// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneObjectPropertySection.h"
#include "Tracks/MovieSceneObjectPropertyTrack.h"
#include "Channels/MovieSceneChannelProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneObjectPropertySection)


UMovieSceneObjectPropertySection::UMovieSceneObjectPropertySection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	bSupportsInfiniteRange = true;
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);
	SetRange(TRange<FFrameNumber>::All());

#if WITH_EDITOR
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(ObjectChannel, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<UObject*>::Make());
#else
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(ObjectChannel);
#endif
}

void UMovieSceneObjectPropertySection::PostLoad()
{
	if (UMovieSceneObjectPropertyTrack* PropertyTrack = Cast<UMovieSceneObjectPropertyTrack>(GetOuter()))
	{
		UClass* PropertyTrackPropertyClass = PropertyTrack->PropertyClass;
		UClass* ObjectChannelPropertyClass = ObjectChannel.GetPropertyClass();
		if (PropertyTrackPropertyClass != nullptr && ObjectChannelPropertyClass == nullptr)
		{
			UE_LOGF(LogMovieScene, Warning, "Mismatch in property track's property class %ls and object channel's property class (null). Setting object channel's property class to %ls.", *PropertyTrackPropertyClass->GetName(), *PropertyTrackPropertyClass->GetName());
			ObjectChannel.SetPropertyClass(PropertyTrackPropertyClass);
		}
		else if (ObjectChannelPropertyClass != nullptr && PropertyTrackPropertyClass == nullptr)
		{
			UE_LOGF(LogMovieScene, Warning, "Mismatch in object channel's property class %ls and property track's property class (null). Setting property track's property class to %ls.", *ObjectChannelPropertyClass->GetName(), *ObjectChannelPropertyClass->GetName());
			PropertyTrack->PropertyClass = ObjectChannelPropertyClass;
		}
		else if (PropertyTrackPropertyClass != nullptr && ObjectChannelPropertyClass != nullptr)
		{
			ensureMsgf(PropertyTrackPropertyClass == ObjectChannelPropertyClass, TEXT("Mismatch in property track's property class %s and object channel's property class %s"), *PropertyTrackPropertyClass->GetName(), *ObjectChannelPropertyClass->GetName());
		}
	}

	Super::PostLoad();
}

bool UMovieSceneObjectPropertySection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneObjectPropertySection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->Object)
		.Add(Components->ObjectPathChannel, &ObjectChannel)
		.Commit(this, Params, OutImportedEntity);
}