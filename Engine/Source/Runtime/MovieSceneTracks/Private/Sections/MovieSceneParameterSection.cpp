// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneParameterSection.h"
#include "UObject/SequencerObjectVersion.h"
#include "Channels/MovieSceneChannelProxy.h"

#include "Evaluation/MovieSceneEvaluationField.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneParameterSection)


namespace UE::MovieScene
{

/* Entity IDs are an encoded type and index, with the upper 8 bits being the type, and the lower 24 bits as the index */
uint32 EncodeEntityID(int32 InIndex, uint8 InType)
{
	check(InIndex >= 0 && InIndex < int32(0x00FFFFFF));
	return static_cast<uint32>(InIndex) | (uint32(InType) << 24);
}
void DecodeEntityID(uint32 InEntityID, int32& OutIndex, uint8& OutType)
{
	// Mask out the type to get the index
	OutIndex = static_cast<int32>(InEntityID & 0x00FFFFFF);
	OutType = InEntityID >> 24;
}


}// namespace UE::MovieScene

void IMovieSceneParameterSectionExtender::ExtendEntity(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	ExtendEntityImpl(Section, EntityLinker, Params, OutImportedEntity);
}

UMovieSceneParameterSection::UMovieSceneParameterSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	bSupportsInfiniteRange = true;
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);
}	

void UMovieSceneParameterSection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		ReconstructChannelProxy();
	}
}

void UMovieSceneParameterSection::PostEditImport()
{
	Super::PostEditImport();

	ReconstructChannelProxy();
}


void UMovieSceneParameterSection::ReconstructChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR

	int32 SortOrder = 0;
	for (FScalarParameterNameAndCurve& Scalar : GetScalarParameterNamesAndCurves())
	{
		FMovieSceneChannelMetaData MetaData(Scalar.ParameterName, FText::FromName(Scalar.ParameterName));
		// Prevent single channels from collapsing to the track node
		MetaData.bCanCollapseToTrack = false;
		MetaData.SortOrder = SortOrder++;
		Channels.Add(Scalar.ParameterCurve, MetaData, TMovieSceneExternalValue<float>());
	}

	for (FBoolParameterNameAndCurve& Bool : GetBoolParameterNamesAndCurves())
	{
		FMovieSceneChannelMetaData MetaData(Bool.ParameterName, FText::FromName(Bool.ParameterName));
		// Prevent single channels from collapsing to the track node
		MetaData.bCanCollapseToTrack = false;		
		MetaData.SortOrder = SortOrder++;
		Channels.Add(Bool.ParameterCurve, MetaData, TMovieSceneExternalValue<bool>());
	}
	for (FVector2DParameterNameAndCurves& Vector2D : GetVector2DParameterNamesAndCurves())
	{
		FString ParameterString = Vector2D.ParameterName.ToString();
		FText Group = FText::FromString(ParameterString);
		FMovieSceneChannelMetaData MetaData_X = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".X")), FCommonChannelData::ChannelX, Group);
		MetaData_X.SortOrder = SortOrder++;
		FMovieSceneChannelMetaData MetaData_Y = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Y")), FCommonChannelData::ChannelY, Group);
		MetaData_Y.SortOrder = SortOrder++;

		Channels.Add(Vector2D.XCurve, MetaData_X, TMovieSceneExternalValue<float>());
		Channels.Add(Vector2D.YCurve, MetaData_Y, TMovieSceneExternalValue<float>());
	}
	for (FVectorParameterNameAndCurves& Vector : GetVectorParameterNamesAndCurves())
	{
		FString ParameterString = Vector.ParameterName.ToString();
		FText Group = FText::FromString(ParameterString);
		FMovieSceneChannelMetaData MetaData_X = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".X")), FCommonChannelData::ChannelX, Group);
		MetaData_X.SortOrder = SortOrder++;
		FMovieSceneChannelMetaData MetaData_Y = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Y")), FCommonChannelData::ChannelY, Group);
		MetaData_Y.SortOrder = SortOrder++;
		FMovieSceneChannelMetaData MetaData_Z = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Z")), FCommonChannelData::ChannelZ, Group);
		MetaData_Z.SortOrder = SortOrder++;

		Channels.Add(Vector.XCurve, MetaData_X, TMovieSceneExternalValue<float>());
		Channels.Add(Vector.YCurve, MetaData_Y, TMovieSceneExternalValue<float>());
		Channels.Add(Vector.ZCurve, MetaData_Z, TMovieSceneExternalValue<float>());
	}
	for (FColorParameterNameAndCurves& Color : GetColorParameterNamesAndCurves())
	{
		FString ParameterString = Color.ParameterName.ToString();
		FText Group = FText::FromString(ParameterString);

		FMovieSceneChannelMetaData MetaData_R(*(ParameterString + TEXT("R")), FCommonChannelData::ChannelR, Group);
		MetaData_R.SortOrder = SortOrder++;
		MetaData_R.Color = FCommonChannelData::RedChannelColor;

		FMovieSceneChannelMetaData MetaData_G(*(ParameterString + TEXT("G")), FCommonChannelData::ChannelG, Group);
		MetaData_G.SortOrder = SortOrder++;
		MetaData_G.Color = FCommonChannelData::GreenChannelColor;

		FMovieSceneChannelMetaData MetaData_B(*(ParameterString + TEXT("B")), FCommonChannelData::ChannelB, Group);
		MetaData_B.SortOrder = SortOrder++;
		MetaData_B.Color = FCommonChannelData::BlueChannelColor;

		FMovieSceneChannelMetaData MetaData_A(*(ParameterString + TEXT("A")), FCommonChannelData::ChannelA, Group);
		MetaData_A.SortOrder = SortOrder++;

		Channels.Add(Color.RedCurve, MetaData_R, TMovieSceneExternalValue<float>());
		Channels.Add(Color.GreenCurve, MetaData_G, TMovieSceneExternalValue<float>());
		Channels.Add(Color.BlueCurve, MetaData_B, TMovieSceneExternalValue<float>());
		Channels.Add(Color.AlphaCurve, MetaData_A, TMovieSceneExternalValue<float>());
	}

	for (FTransformParameterNameAndCurves& Transform : GetTransformParameterNamesAndCurves())
	{
		FString ParameterString = Transform.ParameterName.ToString();
		FText Group = FText::FromString(ParameterString);

		FMovieSceneChannelMetaData MetaData_Tx = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Translation.X")), FCommonChannelData::ChannelX, Group);
		MetaData_Tx.SortOrder = SortOrder++;
		FMovieSceneChannelMetaData MetaData_Ty = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Translation.Y")), FCommonChannelData::ChannelY, Group);
		MetaData_Ty.SortOrder = SortOrder++;
		FMovieSceneChannelMetaData MetaData_Tz = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Translation.Z")), FCommonChannelData::ChannelZ, Group);
		MetaData_Tz.SortOrder = SortOrder++;

		Channels.Add(Transform.Translation[0], MetaData_Tx, TMovieSceneExternalValue<float>());
		Channels.Add(Transform.Translation[1], MetaData_Ty, TMovieSceneExternalValue<float>());
		Channels.Add(Transform.Translation[2], MetaData_Tz, TMovieSceneExternalValue<float>());

		FMovieSceneChannelMetaData MetaData_Rx = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Rotation.X")), FCommonChannelData::ChannelX, Group);
		MetaData_Rx.SortOrder = SortOrder++;
		FMovieSceneChannelMetaData MetaData_Ry = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Rotation.Y")), FCommonChannelData::ChannelY, Group);
		MetaData_Ry.SortOrder = SortOrder++;
		FMovieSceneChannelMetaData MetaData_Rz = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Rotation.Z")), FCommonChannelData::ChannelZ, Group);
		MetaData_Rz.SortOrder = SortOrder++;

		Channels.Add(Transform.Rotation[0], MetaData_Rx, TMovieSceneExternalValue<float>());
		Channels.Add(Transform.Rotation[1], MetaData_Ry, TMovieSceneExternalValue<float>());
		Channels.Add(Transform.Rotation[2], MetaData_Rz, TMovieSceneExternalValue<float>());

		FMovieSceneChannelMetaData MetaData_Sx = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Scale.X")), FCommonChannelData::ChannelX, Group);
		MetaData_Sx.SortOrder = SortOrder++;
		FMovieSceneChannelMetaData MetaData_Sy = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Scale.Y")), FCommonChannelData::ChannelY, Group);
		MetaData_Sy.SortOrder = SortOrder++;
		FMovieSceneChannelMetaData MetaData_Sz = FMovieSceneChannelMetaData(*(ParameterString + TEXT(".Scale.Z")), FCommonChannelData::ChannelZ, Group);
		MetaData_Sz.SortOrder = SortOrder++;

		Channels.Add(Transform.Scale[0], MetaData_Sx, TMovieSceneExternalValue<float>());
		Channels.Add(Transform.Scale[1], MetaData_Sy, TMovieSceneExternalValue<float>());
		Channels.Add(Transform.Scale[2], MetaData_Sz, TMovieSceneExternalValue<float>());

	}
#else

	for (FScalarParameterNameAndCurve& Scalar : GetScalarParameterNamesAndCurves())
	{
		Channels.Add(Scalar.ParameterCurve);
	}
	for (FBoolParameterNameAndCurve& Bool : GetBoolParameterNamesAndCurves())
	{
		Channels.Add(Bool.ParameterCurve);
	}
	for (FVector2DParameterNameAndCurves& Vector2D : GetVector2DParameterNamesAndCurves())
	{
		Channels.Add(Vector2D.XCurve);
		Channels.Add(Vector2D.YCurve);
	}
	for (FColorParameterNameAndCurves& Color : GetColorParameterNamesAndCurves())
	{
		Channels.Add(Color.RedCurve);
		Channels.Add(Color.GreenCurve);
		Channels.Add(Color.BlueCurve);
		Channels.Add(Color.AlphaCurve);
	}

	for (FTransformParameterNameAndCurves& Transform : GetTransformParameterNamesAndCurves())
	{
		Channels.Add(Transform.Translation[0]);
		Channels.Add(Transform.Translation[1]);
		Channels.Add(Transform.Translation[2]);

		Channels.Add(Transform.Rotation[0]);
		Channels.Add(Transform.Rotation[1]);
		Channels.Add(Transform.Rotation[2]);

		Channels.Add(Transform.Scale[0]);
		Channels.Add(Transform.Scale[1]);
		Channels.Add(Transform.Scale[2]);

	}

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

void UMovieSceneParameterSection::AddScalarParameterKey( FName InParameterName, FFrameNumber InTime, float InValue, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	FMovieSceneFloatChannel* ExistingChannel = nullptr;
	for ( FScalarParameterNameAndCurve& ScalarParameterNameAndCurve : ScalarParameterNamesAndCurves )
	{
		if ( ScalarParameterNameAndCurve.ParameterName == InParameterName )
		{
			ExistingChannel = &ScalarParameterNameAndCurve.ParameterCurve;
			break;
		}
	}
	if ( ExistingChannel == nullptr )
	{
		const int32 NewIndex = ScalarParameterNamesAndCurves.Add( FScalarParameterNameAndCurve( InParameterName ) );
		ExistingChannel = &ScalarParameterNamesAndCurves[NewIndex].ParameterCurve;

		ReconstructChannelProxy();
	}

	EMovieSceneKeyInterpolation Interpolation = GetInterpolationMode(ExistingChannel, InTime, DefaultInterpolation);
	AddKeyToChannel(ExistingChannel, InTime, InValue, Interpolation);

	if (TryModify())
	{
		SetRange(TRange<FFrameNumber>::Hull(TRange<FFrameNumber>(InTime), GetRange()));
	}
}

void UMovieSceneParameterSection::AddBoolParameterKey(FName InParameterName, FFrameNumber InTime, bool InValue)
{
	FMovieSceneBoolChannel* ExistingChannel = nullptr;
	for (FBoolParameterNameAndCurve& BoolParameterNameAndCurve : BoolParameterNamesAndCurves)
	{
		if (BoolParameterNameAndCurve.ParameterName == InParameterName)
		{
			ExistingChannel = &BoolParameterNameAndCurve.ParameterCurve;
			break;
		}
	}
	if (ExistingChannel == nullptr)
	{
		const int32 NewIndex = BoolParameterNamesAndCurves.Add(FBoolParameterNameAndCurve(InParameterName));
		ExistingChannel = &BoolParameterNamesAndCurves[NewIndex].ParameterCurve;

		ReconstructChannelProxy();
	}

	ExistingChannel->GetData().UpdateOrAddKey(InTime, InValue);

	if (TryModify())
	{
		SetRange(TRange<FFrameNumber>::Hull(TRange<FFrameNumber>(InTime), GetRange()));
	}
}


void UMovieSceneParameterSection::AddVector2DParameterKey(FName InParameterName, FFrameNumber InTime, FVector2D InValue, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	FVector2DParameterNameAndCurves* ExistingCurves = nullptr;
	for (FVector2DParameterNameAndCurves& VectorParameterNameAndCurve : Vector2DParameterNamesAndCurves)
	{
		if (VectorParameterNameAndCurve.ParameterName == InParameterName)
		{
			ExistingCurves = &VectorParameterNameAndCurve;
			break;
		}
	}
	if (ExistingCurves == nullptr)
	{
		int32 NewIndex = Vector2DParameterNamesAndCurves.Add(FVector2DParameterNameAndCurves(InParameterName));
		ExistingCurves = &Vector2DParameterNamesAndCurves[NewIndex];

		ReconstructChannelProxy();
	}

	EMovieSceneKeyInterpolation XInterpolation = GetInterpolationMode(&ExistingCurves->XCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->XCurve, InTime, InValue.X, XInterpolation);
	EMovieSceneKeyInterpolation YInterpolation = GetInterpolationMode(&ExistingCurves->YCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->YCurve, InTime, InValue.Y, YInterpolation);

	if (TryModify())
	{
		SetRange(TRange<FFrameNumber>::Hull(TRange<FFrameNumber>(InTime), GetRange()));
	}
}

void UMovieSceneParameterSection::AddVectorParameterKey( FName InParameterName, FFrameNumber InTime, FVector InValue, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	FVectorParameterNameAndCurves* ExistingCurves = nullptr;
	for ( FVectorParameterNameAndCurves& VectorParameterNameAndCurve : VectorParameterNamesAndCurves )
	{
		if ( VectorParameterNameAndCurve.ParameterName == InParameterName )
		{
			ExistingCurves = &VectorParameterNameAndCurve;
			break;
		}
	}
	if ( ExistingCurves == nullptr )
	{
		int32 NewIndex = VectorParameterNamesAndCurves.Add( FVectorParameterNameAndCurves( InParameterName ) );
		ExistingCurves = &VectorParameterNamesAndCurves[NewIndex];

		ReconstructChannelProxy();
	}

	EMovieSceneKeyInterpolation XInterpolation = GetInterpolationMode(&ExistingCurves->XCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->XCurve, InTime, InValue.X, XInterpolation);
	EMovieSceneKeyInterpolation YInterpolation = GetInterpolationMode(&ExistingCurves->YCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->YCurve, InTime, InValue.Y, YInterpolation);
	EMovieSceneKeyInterpolation ZInterpolation = GetInterpolationMode(&ExistingCurves->ZCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->ZCurve, InTime, InValue.Z, ZInterpolation);

	if (TryModify())
	{
		SetRange(TRange<FFrameNumber>::Hull(TRange<FFrameNumber>(InTime), GetRange()));
	}
}

void UMovieSceneParameterSection::AddColorParameterKey( FName InParameterName, FFrameNumber InTime, FLinearColor InValue, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	FColorParameterNameAndCurves* ExistingCurves = nullptr;
	for ( FColorParameterNameAndCurves& ColorParameterNameAndCurve : ColorParameterNamesAndCurves )
	{
		if ( ColorParameterNameAndCurve.ParameterName == InParameterName )
		{
			ExistingCurves = &ColorParameterNameAndCurve;
			break;
		}
	}
	if ( ExistingCurves == nullptr )
	{
		int32 NewIndex = ColorParameterNamesAndCurves.Add( FColorParameterNameAndCurves( InParameterName ) );
		ExistingCurves = &ColorParameterNamesAndCurves[NewIndex];

		ReconstructChannelProxy();
	}
	
	EMovieSceneKeyInterpolation RedInterpolation = GetInterpolationMode(&ExistingCurves->RedCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->RedCurve, InTime, InValue.R, RedInterpolation);
	EMovieSceneKeyInterpolation GreenInterpolation = GetInterpolationMode(&ExistingCurves->GreenCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->GreenCurve, InTime, InValue.G, GreenInterpolation);
	EMovieSceneKeyInterpolation BlueInterpolation = GetInterpolationMode(&ExistingCurves->BlueCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->BlueCurve, InTime, InValue.B, BlueInterpolation);
	EMovieSceneKeyInterpolation AlphaInterpolation = GetInterpolationMode(&ExistingCurves->AlphaCurve, InTime, DefaultInterpolation);
	AddKeyToChannel(&ExistingCurves->AlphaCurve, InTime, InValue.A, AlphaInterpolation);

	if (TryModify())
	{
		SetRange(TRange<FFrameNumber>::Hull(TRange<FFrameNumber>(InTime), GetRange()));
	}
}

void UMovieSceneParameterSection::AddTransformParameterKey(FName InParameterName, FFrameNumber InTime, const FTransform& InValue, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	FTransformParameterNameAndCurves* ExistingCurves = nullptr;
	for (FTransformParameterNameAndCurves& TransformParameterNamesAndCurve : TransformParameterNamesAndCurves)
	{
		if (TransformParameterNamesAndCurve.ParameterName == InParameterName)
		{
			ExistingCurves = &TransformParameterNamesAndCurve;
			break;
		}
	}
	if (ExistingCurves == nullptr)
	{
		int32 NewIndex = TransformParameterNamesAndCurves.Add(FTransformParameterNameAndCurves(InParameterName));
		ExistingCurves = &TransformParameterNamesAndCurves[NewIndex];

		ReconstructChannelProxy();
	}
	FVector Translation = InValue.GetTranslation();
	FRotator Rotator = InValue.GetRotation().Rotator();
	FVector Scale = InValue.GetScale3D();


	////todo
	ExistingCurves->Translation[0].GetData().UpdateOrAddKey(InTime, FMovieSceneFloatValue(Translation[0]));
	ExistingCurves->Translation[1].GetData().UpdateOrAddKey(InTime, FMovieSceneFloatValue(Translation[1]));
	ExistingCurves->Translation[2].GetData().UpdateOrAddKey(InTime, FMovieSceneFloatValue(Translation[2]));

	ExistingCurves->Rotation[0].GetData().UpdateOrAddKey(InTime, FMovieSceneFloatValue(Rotator.Roll));
	ExistingCurves->Rotation[1].GetData().UpdateOrAddKey(InTime, FMovieSceneFloatValue(Rotator.Pitch));
	ExistingCurves->Rotation[2].GetData().UpdateOrAddKey(InTime, FMovieSceneFloatValue(Rotator.Yaw));

	ExistingCurves->Scale[0].GetData().UpdateOrAddKey(InTime, FMovieSceneFloatValue(Scale[0]));
	ExistingCurves->Scale[1].GetData().UpdateOrAddKey(InTime, FMovieSceneFloatValue(Scale[1]));
	ExistingCurves->Scale[2].GetData().UpdateOrAddKey(InTime, FMovieSceneFloatValue(Scale[2]));

	if (TryModify())
	{
		SetRange(TRange<FFrameNumber>::Hull(TRange<FFrameNumber>(InTime), GetRange()));
	}
}

bool UMovieSceneParameterSection::RemoveScalarParameter( FName InParameterName )
{
	for ( int32 i = 0; i < ScalarParameterNamesAndCurves.Num(); i++ )
	{
		if ( ScalarParameterNamesAndCurves[i].ParameterName == InParameterName )
		{
			ScalarParameterNamesAndCurves.RemoveAt(i);
			ReconstructChannelProxy();
			return true;
		}
	}
	return false;
}

bool UMovieSceneParameterSection::RemoveBoolParameter(FName InParameterName)
{
	for (int32 i = 0; i < BoolParameterNamesAndCurves.Num(); i++)
	{
		if (BoolParameterNamesAndCurves[i].ParameterName == InParameterName)
		{
			BoolParameterNamesAndCurves.RemoveAt(i);
			ReconstructChannelProxy();
			return true;
		}
	}
	return false;
}

bool UMovieSceneParameterSection::RemoveVector2DParameter(FName InParameterName)
{
	for (int32 i = 0; i < Vector2DParameterNamesAndCurves.Num(); i++)
	{
		if (Vector2DParameterNamesAndCurves[i].ParameterName == InParameterName)
		{
			Vector2DParameterNamesAndCurves.RemoveAt(i);
			ReconstructChannelProxy();
			return true;
		}
	}
	return false;
}

bool UMovieSceneParameterSection::RemoveVectorParameter( FName InParameterName )
{
	for ( int32 i = 0; i < VectorParameterNamesAndCurves.Num(); i++ )
	{
		if ( VectorParameterNamesAndCurves[i].ParameterName == InParameterName )
		{
			VectorParameterNamesAndCurves.RemoveAt( i );
			ReconstructChannelProxy();
			return true;
		}
	}
	return false;
}

bool UMovieSceneParameterSection::RemoveColorParameter( FName InParameterName )
{
	for ( int32 i = 0; i < ColorParameterNamesAndCurves.Num(); i++ )
	{
		if ( ColorParameterNamesAndCurves[i].ParameterName == InParameterName )
		{
			ColorParameterNamesAndCurves.RemoveAt( i );
			ReconstructChannelProxy();
			return true;
		}
	}
	return false;
}

bool UMovieSceneParameterSection::RemoveTransformParameter(FName InParameterName)
{
	for (int32 i = 0; i < TransformParameterNamesAndCurves.Num(); i++)
	{
		if (TransformParameterNamesAndCurves[i].ParameterName == InParameterName)
		{
			TransformParameterNamesAndCurves.RemoveAt(i);
			ReconstructChannelProxy();
			return true;
		}
	}
	return false;
}

TArray<FScalarParameterNameAndCurve>& UMovieSceneParameterSection::GetScalarParameterNamesAndCurves()
{
	return ScalarParameterNamesAndCurves;
}

const TArray<FScalarParameterNameAndCurve>& UMovieSceneParameterSection::GetScalarParameterNamesAndCurves() const
{
	return ScalarParameterNamesAndCurves;
}

TArray<FBoolParameterNameAndCurve>& UMovieSceneParameterSection::GetBoolParameterNamesAndCurves() 
{
	return BoolParameterNamesAndCurves;
}

const TArray<FBoolParameterNameAndCurve>& UMovieSceneParameterSection::GetBoolParameterNamesAndCurves() const
{
	return BoolParameterNamesAndCurves;
}

TArray<FVector2DParameterNameAndCurves>& UMovieSceneParameterSection::GetVector2DParameterNamesAndCurves()
{
	return Vector2DParameterNamesAndCurves;
}

const TArray<FVector2DParameterNameAndCurves>& UMovieSceneParameterSection::GetVector2DParameterNamesAndCurves() const
{
	return Vector2DParameterNamesAndCurves;
}

TArray<FVectorParameterNameAndCurves>& UMovieSceneParameterSection::GetVectorParameterNamesAndCurves() 
{
	return VectorParameterNamesAndCurves;
}

const TArray<FVectorParameterNameAndCurves>& UMovieSceneParameterSection::GetVectorParameterNamesAndCurves() const
{
	return VectorParameterNamesAndCurves;
}

TArray<FColorParameterNameAndCurves>& UMovieSceneParameterSection::GetColorParameterNamesAndCurves()
{
	return ColorParameterNamesAndCurves;
}

const TArray<FColorParameterNameAndCurves>& UMovieSceneParameterSection::GetColorParameterNamesAndCurves() const
{
	return ColorParameterNamesAndCurves;
}

TArray<FTransformParameterNameAndCurves>& UMovieSceneParameterSection::GetTransformParameterNamesAndCurves() 
{
	return TransformParameterNamesAndCurves;
}

const TArray<FTransformParameterNameAndCurves>& UMovieSceneParameterSection::GetTransformParameterNamesAndCurves() const
{
	return TransformParameterNamesAndCurves;
}

void UMovieSceneParameterSection::GetParameterNames( TSet<FName>& ParameterNames ) const
{
	for ( const FScalarParameterNameAndCurve& ScalarParameterNameAndCurve : ScalarParameterNamesAndCurves )
	{
		ParameterNames.Add( ScalarParameterNameAndCurve.ParameterName );
	}
	for ( const FVectorParameterNameAndCurves& VectorParameterNameAndCurves : VectorParameterNamesAndCurves )
	{
		ParameterNames.Add( VectorParameterNameAndCurves.ParameterName );
	}
	for ( const FColorParameterNameAndCurves& ColorParameterNameAndCurves : ColorParameterNamesAndCurves )
	{
		ParameterNames.Add( ColorParameterNameAndCurves.ParameterName );
	}
	for (const FTransformParameterNameAndCurves& TransformParameterNamesAndCurve : TransformParameterNamesAndCurves)
	{
		ParameterNames.Add(TransformParameterNamesAndCurve.ParameterName);
	}
}

void UMovieSceneParameterSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	IMovieSceneParameterSectionExtender* Extender = GetImplementingOuter<IMovieSceneParameterSectionExtender>();
	if (!ensureMsgf(Extender, TEXT("It is not valid for a UMovieSceneParameterSection to be used for importing entities outside of an outer chain that implements IMovieSceneParameterSectionExtender")))
	{
		return;
	}

	uint8 ParameterType = 0;
	int32 EntityIndex = 0;
	DecodeEntityID(Params.EntityID, EntityIndex, ParameterType);

	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponentTypes = FMovieSceneTracksComponentTypes::Get();

	FGuid ObjectBindingID = Params.GetObjectBindingID();

	TEntityBuilder<TAddConditional<FGuid>> BaseBuilder = FEntityBuilder()
		.AddConditional(BuiltInComponentTypes->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid());

	switch (ParameterType)
	{
		case 0:
		{
			const FScalarParameterNameAndCurve& Scalar = ScalarParameterNamesAndCurves[EntityIndex];

			if (Scalar.ParameterCurve.HasAnyData())
			{
				OutImportedEntity->AddBuilder(
					BaseBuilder
					.Add(TracksComponentTypes->ScalarParameterName, Scalar.ParameterName)
					.Add(BuiltInComponentTypes->FloatChannel[0], &Scalar.ParameterCurve)
				);
			}
			break;
		}
		case 1:
		{
			const FBoolParameterNameAndCurve& Bool = BoolParameterNamesAndCurves[EntityIndex];

			if (Bool.ParameterCurve.HasAnyData())
			{
				OutImportedEntity->AddBuilder(
					BaseBuilder
					.Add(TracksComponentTypes->BoolParameterName, Bool.ParameterName)
					.Add(BuiltInComponentTypes->BoolChannel, &Bool.ParameterCurve)
				);
			}
			break;
		}
		case 2:
		{
			const FVector2DParameterNameAndCurves& Vector2D = Vector2DParameterNamesAndCurves[EntityIndex];
			if (Vector2D.XCurve.HasAnyData() || Vector2D.YCurve.HasAnyData())
			{
				OutImportedEntity->AddBuilder(
					BaseBuilder
					.Add(TracksComponentTypes->Vector2DParameterName, Vector2D.ParameterName)
					.AddConditional(BuiltInComponentTypes->FloatChannel[0], &Vector2D.XCurve, Vector2D.XCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[1], &Vector2D.YCurve, Vector2D.YCurve.HasAnyData())
				);
			}
			break;
		}
		case 3:
		{
			const FVectorParameterNameAndCurves& Vector = VectorParameterNamesAndCurves[EntityIndex];

			if (Vector.XCurve.HasAnyData() || Vector.YCurve.HasAnyData() || Vector.ZCurve.HasAnyData())
			{
				OutImportedEntity->AddBuilder(
					BaseBuilder
					.Add(TracksComponentTypes->VectorParameterName, Vector.ParameterName)
					.AddConditional(BuiltInComponentTypes->FloatChannel[0], &Vector.XCurve, Vector.XCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[1], &Vector.YCurve, Vector.YCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[2], &Vector.ZCurve, Vector.ZCurve.HasAnyData())
				);
			}
			break;
		}
		case 4:
		{
			const FColorParameterNameAndCurves& Color = ColorParameterNamesAndCurves[EntityIndex];
			if (Color.RedCurve.HasAnyData() || Color.GreenCurve.HasAnyData() || Color.BlueCurve.HasAnyData() || Color.AlphaCurve.HasAnyData())
			{
				OutImportedEntity->AddBuilder(
					BaseBuilder
					.Add(TracksComponentTypes->ColorParameterName, Color.ParameterName)
					.AddConditional(BuiltInComponentTypes->FloatChannel[0], &Color.RedCurve, Color.RedCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[1], &Color.GreenCurve, Color.GreenCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[2], &Color.BlueCurve, Color.BlueCurve.HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[3], &Color.AlphaCurve, Color.AlphaCurve.HasAnyData())
				);
			}
			break;
		}
		case 5:
		{
			const FTransformParameterNameAndCurves& Transform = TransformParameterNamesAndCurves[EntityIndex];
			if (Transform.Translation[0].HasAnyData() || Transform.Translation[1].HasAnyData() || Transform.Translation[2].HasAnyData() ||
				Transform.Rotation[0].HasAnyData() || Transform.Rotation[1].HasAnyData() || Transform.Rotation[2].HasAnyData()
				|| Transform.Scale[0].HasAnyData() || Transform.Scale[1].HasAnyData() || Transform.Scale[2].HasAnyData())
			{
				OutImportedEntity->AddBuilder(
					BaseBuilder
					.Add(TracksComponentTypes->TransformParameterName, Transform.ParameterName)
					.AddConditional(BuiltInComponentTypes->FloatChannel[0], &Transform.Translation[0], Transform.Translation[0].HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[1], &Transform.Translation[1], Transform.Translation[1].HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[2], &Transform.Translation[2], Transform.Translation[2].HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[3], &Transform.Rotation[0], Transform.Rotation[0].HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[4], &Transform.Rotation[1], Transform.Rotation[1].HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[5], &Transform.Rotation[2], Transform.Rotation[2].HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[6], &Transform.Scale[0], Transform.Scale[0].HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[7], &Transform.Scale[1], Transform.Scale[1].HasAnyData())
					.AddConditional(BuiltInComponentTypes->FloatChannel[8], &Transform.Scale[2], Transform.Scale[2].HasAnyData())
				);
			}
			break;
		}
	}

	Extender->ExtendEntity(this, EntityLinker, Params, OutImportedEntity);
}

bool UMovieSceneParameterSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	// By default, parameter sections do not populate any evaluation field entries
	// that is the job of its outer UMovieSceneTrack through a call to ExternalPopulateEvaluationField
	return true;
}

void UMovieSceneParameterSection::ExternalPopulateEvaluationField(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	using namespace UE::MovieScene;

	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	// We use the top 8 bits of EntityID to encode the type of parameter
	const int32 NumScalarID    = ScalarParameterNamesAndCurves.Num();
	const int32 NumBoolID      = BoolParameterNamesAndCurves.Num();
	const int32 NumVector2DID  = Vector2DParameterNamesAndCurves.Num();
	const int32 NumVectorID    = VectorParameterNamesAndCurves.Num();
	const int32 NumColorID     = ColorParameterNamesAndCurves.Num();
	const int32 NumTransformID = TransformParameterNamesAndCurves.Num();

	for (int32 Index = 0; Index < NumScalarID; ++Index)
	{
		if (ScalarParameterNamesAndCurves[Index].ParameterCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 0));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}
	for (int32 Index = 0; Index < NumBoolID; ++Index)
	{
		if (BoolParameterNamesAndCurves[Index].ParameterCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 1));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}
	for (int32 Index = 0; Index < NumVector2DID; ++Index)
	{
		const FVector2DParameterNameAndCurves& Vector2D = Vector2DParameterNamesAndCurves[Index];
		if (Vector2D.XCurve.HasAnyData() || Vector2D.YCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 2));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}
	for (int32 Index = 0; Index < NumVectorID; ++Index)
	{
		const FVectorParameterNameAndCurves& Vector = VectorParameterNamesAndCurves[Index];
		if (Vector.XCurve.HasAnyData() || Vector.YCurve.HasAnyData() || Vector.ZCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 3));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}
	for (int32 Index = 0; Index < NumColorID; ++Index)
	{
		const FColorParameterNameAndCurves& Color = ColorParameterNamesAndCurves[Index];
		if (Color.RedCurve.HasAnyData() || Color.GreenCurve.HasAnyData() || Color.BlueCurve.HasAnyData() || Color.AlphaCurve.HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 4));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}
	for (int32 Index = 0; Index < NumTransformID; ++Index)
	{
		const FTransformParameterNameAndCurves& Transform = TransformParameterNamesAndCurves[Index];
		if (Transform.Translation[0].HasAnyData() || Transform.Translation[1].HasAnyData() || Transform.Translation[2].HasAnyData() ||
			Transform.Rotation[0].HasAnyData() || Transform.Rotation[1].HasAnyData() || Transform.Rotation[2].HasAnyData() ||
			Transform.Scale[0].HasAnyData() || Transform.Scale[1].HasAnyData() || Transform.Scale[2].HasAnyData())
		{
			const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, EncodeEntityID(Index, 5));
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}
}
