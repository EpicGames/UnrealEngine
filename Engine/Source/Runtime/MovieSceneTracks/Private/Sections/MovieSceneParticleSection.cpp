// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneParticleSection.h"
#include "UObject/SequencerObjectVersion.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneParticleSection)

FMovieSceneParticleChannel::FMovieSceneParticleChannel()
{
	SetEnum(StaticEnum<EParticleKey>());
}

UMovieSceneParticleSection::UMovieSceneParticleSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
#if WITH_EDITORONLY_DATA
	bIsInfinite_DEPRECATED = true;
#endif

	SetRange(TRange<FFrameNumber>::All());

	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);

#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(ParticleKeys, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<uint8>());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(ParticleKeys);

#endif
}
