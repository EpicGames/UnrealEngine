// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieScenePrimitiveMaterialSection.h"
#include "Tracks/MovieScenePrimitiveMaterialTrack.h"
#include "Tracks/MovieSceneMaterialTrack.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Materials/MaterialInterface.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePrimitiveMaterialSection)

namespace UE::MovieScene
{
	uint32 PrimitiveMaterialParameterSectionDefaultValueID = uint32(-1);
}

UMovieScenePrimitiveMaterialSection::UMovieScenePrimitiveMaterialSection(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	bSupportsInfiniteRange = true;
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);
	SetRange(TRange<FFrameNumber>::All());

	MaterialChannel.SetPropertyClass(UMaterialInterface::StaticClass());

#if WITH_EDITOR
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MaterialChannel, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<UObject*>::Make());
#else
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MaterialChannel);
#endif
}

bool UMovieScenePrimitiveMaterialSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	using namespace UE::MovieScene;

	TMovieSceneChannelData<const FMovieSceneObjectPathChannelKeyValue> ChannelData = MaterialChannel.GetData();

	TArrayView<const FFrameNumber> Times = ChannelData.GetTimes();
	TArrayView<const FMovieSceneObjectPathChannelKeyValue> Values = ChannelData.GetValues();

	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	if (Times.Num() == 0)
	{
		const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(this, PrimitiveMaterialParameterSectionDefaultValueID);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		return true;
	}

	// Add entities for each key range in the channel
	TRangeBound<FFrameNumber> StartBound = EffectiveRange.GetLowerBound();

	// Find the first effective key
	int32 ValueIndex = FMath::Min(StartBound.IsOpen() ? 0 : Algo::UpperBound(Times, UE::MovieScene::DiscreteInclusiveLower(StartBound)), Times.Num()-1);
	if (!Times.IsValidIndex(ValueIndex))
	{
		return true;
	}

	int32 LastEntityIndex = OutFieldBuilder->FindOrAddEntity(this, ValueIndex);

	for ( ; ValueIndex < Times.Num(); ++ValueIndex)
	{
		if (!EffectiveRange.Contains(Times[ValueIndex]))
		{
			break;
		}

		// Add the last range to the tree
		TRange<FFrameNumber> Range(StartBound, TRangeBound<FFrameNumber>::Exclusive(Times[ValueIndex]));
		if (!Range.IsEmpty())
		{
			OutFieldBuilder->AddPersistentEntity(Range, LastEntityIndex, MetaDataIndex);

			StartBound = TRangeBound<FFrameNumber>::Inclusive(Times[ValueIndex]);
			LastEntityIndex = OutFieldBuilder->FindOrAddEntity(this, ValueIndex);
		}
	}

	TRange<FFrameNumber> TailRange(StartBound, EffectiveRange.GetUpperBound());
	if (!TailRange.IsEmpty())
	{
		OutFieldBuilder->AddPersistentEntity(TailRange, LastEntityIndex, MetaDataIndex);
	}

	return true;
}

void UMovieScenePrimitiveMaterialSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* Linker, const FEntityImportParams& ImportParams, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	UMovieScenePrimitiveMaterialTrack* Track = GetTypedOuter<UMovieScenePrimitiveMaterialTrack>();
	check(Track);

	const FComponentMaterialInfo& MaterialInfo = Track->GetMaterialInfo();

	FBuiltInComponentTypes* BuiltInComponentTypes = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponentTypes = FMovieSceneTracksComponentTypes::Get();

	FMovieSceneObjectPathChannelKeyValue ValueToImport;

	if (ImportParams.EntityID == PrimitiveMaterialParameterSectionDefaultValueID)
	{
		// Default value
		ValueToImport = MaterialChannel.GetDefault();
	}
	else
	{
		const int32 ChannelValueIndex = static_cast<int32>(ImportParams.EntityID);
		ValueToImport = MaterialChannel.GetData().GetValues()[ChannelValueIndex];
	}

	FGuid ObjectBindingID = ImportParams.GetObjectBindingID();

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddConditional(BuiltInComponentTypes->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
		.Add(BuiltInComponentTypes->ObjectResult, FObjectComponent::Strong(ValueToImport.Get()))
		.Add(TracksComponentTypes->ComponentMaterialInfo, MaterialInfo)
	);
}
