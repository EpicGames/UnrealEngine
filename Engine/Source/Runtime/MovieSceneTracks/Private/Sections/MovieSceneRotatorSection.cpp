// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneRotatorSection.h"

#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Tracks/MovieScenePropertyTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneRotatorSection)

#if WITH_EDITOR
struct FRotatorSectionEditorData
{
	FRotatorSectionEditorData()
	{
		MetaData[0].SetIdentifiers(TEXT("Rotation.Y"), FCommonChannelData::ChannelY);
		MetaData[0].SubPropertyPath = TEXT("Pitch");
		MetaData[0].SortOrder = 1;
		MetaData[0].Color = FCommonChannelData::GreenChannelColor;
		MetaData[0].bCanCollapseToTrack = false;
		ExternalValues[0].OnGetExternalValue = ExtractChannelY;

		MetaData[1].SetIdentifiers(TEXT("Rotation.Z"), FCommonChannelData::ChannelZ);
		MetaData[1].SubPropertyPath = TEXT("Yaw");
		MetaData[1].SortOrder = 2;
		MetaData[1].Color = FCommonChannelData::BlueChannelColor;
		MetaData[1].bCanCollapseToTrack = false;
		ExternalValues[1].OnGetExternalValue = ExtractChannelZ;

		MetaData[2].SetIdentifiers(TEXT("Rotation.X"), FCommonChannelData::ChannelX);
		MetaData[2].SubPropertyPath = TEXT("Roll");
		MetaData[2].SortOrder = 0;
		MetaData[2].Color = FCommonChannelData::RedChannelColor;
		MetaData[2].bCanCollapseToTrack = false;
		ExternalValues[2].OnGetExternalValue = ExtractChannelX;
	}

	static FRotator GetPropertyValue(const UObject& InObject, FTrackInstancePropertyBindings& InBindings)
	{
		if (const UStruct* RotatorStruct = InBindings.GetPropertyStruct(InObject))
		{
			if (RotatorStruct->GetFName() == NAME_Rotator)
			{
				return InBindings.GetCurrentValue<FRotator>(InObject);
			}
		}

		return FRotator::ZeroRotator;
	}

	static TOptional<double> ExtractChannelX(UObject& InObject, FTrackInstancePropertyBindings* InBindings)
	{
		return InBindings ? GetPropertyValue(InObject, *InBindings).Roll : TOptional<double>();
	}

	static TOptional<double> ExtractChannelY(UObject& InObject, FTrackInstancePropertyBindings* InBindings)
	{
		return InBindings ? GetPropertyValue(InObject, *InBindings).Pitch : TOptional<double>();
	}

	static TOptional<double> ExtractChannelZ(UObject& InObject, FTrackInstancePropertyBindings* InBindings)
	{
		return InBindings ? GetPropertyValue(InObject, *InBindings).Yaw : TOptional<double>();
	}

	FMovieSceneChannelMetaData MetaData[3];
	TMovieSceneExternalValue<double> ExternalValues[3];
};
#endif

UMovieSceneRotatorSection::UMovieSceneRotatorSection(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);
	bSupportsInfiniteRange = true;
	BlendType = EMovieSceneBlendType::Absolute;

	Rotation[0].SetDefault(0.f);
	Rotation[1].SetDefault(0.f);
	Rotation[2].SetDefault(0.f);

	FMovieSceneChannelProxyData Channels;

#if WITH_EDITOR
	static FRotatorSectionEditorData EditorData;
	Channels.Add(Rotation[0], EditorData.MetaData[0], EditorData.ExternalValues[0]);
	Channels.Add(Rotation[1], EditorData.MetaData[1], EditorData.ExternalValues[1]);
	Channels.Add(Rotation[2], EditorData.MetaData[2], EditorData.ExternalValues[2]);
#else
	Channels.Add(Rotation[0]);
	Channels.Add(Rotation[1]);
	Channels.Add(Rotation[2]);
#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

bool UMovieSceneRotatorSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& InEffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, InEffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneRotatorSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* InEntityLinker, const FEntityImportParams& InParams, FImportedEntity* OutImportedEntity)
{
	if (!Rotation[0].HasAnyData() && !Rotation[1].HasAnyData() && !Rotation[2].HasAnyData())
	{
		return;
	}

	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->Rotator)
		.AddConditional(BuiltInComponents->DoubleChannel[0], &Rotation[0], Rotation[0].HasAnyData())
		.AddConditional(BuiltInComponents->DoubleChannel[1], &Rotation[1], Rotation[1].HasAnyData())
		.AddConditional(BuiltInComponents->DoubleChannel[2], &Rotation[2], Rotation[2].HasAnyData())
		.Commit(this, InParams, OutImportedEntity);
}
