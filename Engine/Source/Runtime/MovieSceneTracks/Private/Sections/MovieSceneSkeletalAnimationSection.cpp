// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Decorations/IMovieSceneChannelDecoration.h"
#include "Animation/AnimSequence.h"
#include "AnimSequencerInstanceProxy.h"
#include "Logging/MessageLog.h"
#include "MovieScene.h"
#include "UObject/SequencerObjectVersion.h"
#include "MovieSceneTimeHelpers.h"
#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "BoneContainer.h"
#include "MovieSceneTransformTypes.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AnimSequenceDecompressionContext.h"
#include "Animation/AttributesRuntime.h"
#include "Misc/FrameRate.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneSectionTimingParameters.h"
#include "Variants/MovieSceneTimeWarpGetter.h"
#include "Systems/MovieSceneSkeletalAnimationSystem.h"
#include "Decorations/MovieSceneLanguagePreviewDecoration.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSkeletalAnimationSection)

#define LOCTEXT_NAMESPACE "MovieSceneSkeletalAnimationSection"

TAutoConsoleVariable<bool> CVarStartTransformOffsetInBoneSpace(TEXT("Sequencer.StartTransformOffsetInBoneSpace"), true,
	TEXT("When true we offset the start offsets for skeletal animation matching in bone space, if false we do it in root space, by default true"));


namespace
{
	FName DefaultSlotName( "DefaultSlot" );
	float SkeletalDeprecatedMagicNumber = TNumericLimits<float>::Lowest();
}

FMovieSceneSkeletalAnimationParams::FMovieSceneSkeletalAnimationParams()
{
	Animation = nullptr;
	MirrorDataTable = nullptr; 
	StartOffset_DEPRECATED = SkeletalDeprecatedMagicNumber;
	EndOffset_DEPRECATED = SkeletalDeprecatedMagicNumber;
	PlayRate = 1.f;
	bReverse = false;
	SlotName = DefaultSlotName;
	Weight.SetDefault(1.f);
	bSkipAnimNotifiers = false;
	bForceCustomMode = false;
	SwapRootBone = ESwapRootBone::SwapRootBone_None;
	bLinearPlaybackWhenScaled = false;
}

FMovieSceneSequenceTransform FMovieSceneSkeletalAnimationParams::MakeTransform(const FFrameRate& OuterFrameRate, const TRange<FFrameNumber>& OuterRange, UAnimSequenceBase* OverrideSequence, bool bClampToOuterRange, bool bForceLoop) const
{
	if (!OverrideSequence)
	{
		OverrideSequence = Animation;
	}

	if (!OverrideSequence)
	{
		return FMovieSceneSequenceTransform();
	}

	const double SequenceLength = OverrideSequence->GetPlayLength();
	const FFrameTime AnimationLength = SequenceLength * OuterFrameRate;
	const int32 LengthInFrames = AnimationLength.FrameNumber.Value + (int)(AnimationLength.GetSubFrame() + 0.5f) + 1;

	const bool bLooping = bForceLoop || ((UE::MovieScene::DiscreteSize(OuterRange) + FirstLoopStartFrameOffset + StartFrameOffset + EndFrameOffset) > LengthInFrames);

	FMovieSceneSectionTimingParametersSeconds TimingParams = {
		PlayRate.ShallowCopy(),
		StartFrameOffset / OuterFrameRate,
		EndFrameOffset / OuterFrameRate,
		FirstLoopStartFrameOffset / OuterFrameRate,
		bLooping,
		!bLooping,
		bReverse,
		bClampToOuterRange
	};

	return TimingParams.MakeTransform(OuterFrameRate, OuterRange, SequenceLength, OverrideSequence->RateScale);
}

double FMovieSceneSkeletalAnimationParams::MapTimeToAnimation(const UMovieSceneSection* InSection, FFrameTime InPosition, FFrameRate InTickResolution, UAnimSequenceBase* OverrideSequence) const
{
	const FFrameNumber SectionStartTime = InSection->GetInclusiveStartFrame();
	const FFrameNumber SectionEndTime = InSection->GetExclusiveEndFrame();
	return MapTimeToAnimation(SectionStartTime, SectionEndTime, InPosition, InTickResolution, OverrideSequence);
}

double FMovieSceneSkeletalAnimationParams::MapTimeToAnimation(FFrameNumber InSectionStartTime, FFrameNumber InSectionEndTime, FFrameTime InPosition, FFrameRate InTickResolution, UAnimSequenceBase* OverrideSequence) const
{
	return MakeTransform(InTickResolution, TRange<FFrameNumber>(InSectionStartTime, InSectionEndTime), OverrideSequence).TransformTime(InPosition).AsDecimal();
}

UMovieSceneSkeletalAnimationSection::UMovieSceneSkeletalAnimationSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	AnimSequence_DEPRECATED = nullptr;
	Animation_DEPRECATED = nullptr;
	StartOffset_DEPRECATED = 0.f;
	EndOffset_DEPRECATED = 0.f;
	PlayRate_DEPRECATED = 1.f;
	bReverse_DEPRECATED = false;
	SlotName_DEPRECATED = DefaultSlotName;
#if WITH_EDITORONLY_DATA
	bShowSkeleton = false;
#endif

	BlendType = EMovieSceneBlendType::Absolute;
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);

	StartLocationOffset = FVector::ZeroVector;
	StartRotationOffset = FRotator::ZeroRotator;
	bMatchWithPrevious = true;
	MatchedBoneName = NAME_None;
	MatchedLocationOffset = FVector::ZeroVector;
	MatchedRotationOffset = FRotator::ZeroRotator;

	bMatchTranslation = true;
	bMatchRotationYaw = true;
	bMatchRotationRoll = false;
	bMatchRotationPitch = false;
	bMatchIncludeZHeight = false;
	bDebugForceTickPose = false;
}

UAnimSequenceBase* UMovieSceneSkeletalAnimationSection::GetAnimation() const
{
	return Params.Animation;
}


UAnimSequenceBase* UMovieSceneSkeletalAnimationSection::GetPlaybackAnimation() const
{
	return UMovieSceneLanguagePreviewDecoration::FindLocalizedAsset(Params.Animation, this);
}


EMovieSceneChannelProxyType UMovieSceneSkeletalAnimationSection::CacheChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

	if (Params.PlayRate.GetType() == EMovieSceneTimeWarpType::Custom)
	{
		UMovieSceneTimeWarpGetter* Custom = Params.PlayRate.AsCustom();
		if (Custom)
		{
			Custom->PopulateChannelProxy(Channels, UMovieSceneTimeWarpGetter::EAllowTopLevelChannels::No);
		}
	}

	for (UObject* Decoration : GetDecorations())
	{
		if (IMovieSceneChannelDecoration* ChannelDecoration = Cast<IMovieSceneChannelDecoration>(Decoration))
		{
			// Result doesn't matter because we always return EMovieSceneChannelProxyType::Dynamic anyway
			ChannelDecoration->PopulateChannelProxy(Channels);
		}
	}



#if WITH_EDITOR

	static FMovieSceneChannelMetaData MetaData("Weight", LOCTEXT("WeightChannelName", "Weight"));
	MetaData.bCanCollapseToTrack = false;

	Channels.Add(Params.Weight, MetaData, TMovieSceneExternalValue<float>());

#else

	Channels.Add(Params.Weight);

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
	return EMovieSceneChannelProxyType::Dynamic;
}

void UMovieSceneSkeletalAnimationSection::DeleteChannels(TArrayView<const FName> ChannelNames)
{
	bool bDeletedAny = false;

	if (Params.PlayRate.GetType() == EMovieSceneTimeWarpType::Custom && TryModify())
	{
		if (UMovieSceneTimeWarpGetter* Getter = Params.PlayRate.AsCustom())
		{
			for (FName ChannelName : ChannelNames)
			{
				bDeletedAny |= Getter->DeleteChannel(Params.PlayRate, ChannelName);
			}
		}
	}

	if (bDeletedAny)
	{
		ChannelProxy = nullptr;
	}
}

TOptional<FFrameTime> UMovieSceneSkeletalAnimationSection::GetOffsetTime() const
{
	return TOptional<FFrameTime>(Params.FirstLoopStartFrameOffset);
}

void UMovieSceneSkeletalAnimationSection::MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate)
{
	if (Params.StartFrameOffset.Value > 0)
	{
		FFrameNumber NewStartFrameOffset = ConvertFrameTime(FFrameTime(Params.StartFrameOffset), SourceRate, DestinationRate).FloorToFrame();
		Params.StartFrameOffset = NewStartFrameOffset;
	}

	if (Params.EndFrameOffset.Value > 0)
	{
		FFrameNumber NewEndFrameOffset = ConvertFrameTime(FFrameTime(Params.EndFrameOffset), SourceRate, DestinationRate).FloorToFrame();
		Params.EndFrameOffset = NewEndFrameOffset;
	}
	if (Params.FirstLoopStartFrameOffset.Value > 0)
	{
		FFrameNumber NewFirstLoopStartFrameOffset = ConvertFrameTime(FFrameTime(Params.FirstLoopStartFrameOffset), SourceRate, DestinationRate).FloorToFrame();
		Params.FirstLoopStartFrameOffset = NewFirstLoopStartFrameOffset;
	}
}

void UMovieSceneSkeletalAnimationSection::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FSequencerObjectVersion::GUID);
	Super::Serialize(Ar);
}

void UMovieSceneSkeletalAnimationSection::PostLoad()
{
	if (AnimSequence_DEPRECATED)
	{
		Params.Animation = AnimSequence_DEPRECATED;
	}

	if (Animation_DEPRECATED != nullptr)
	{
		Params.Animation = Animation_DEPRECATED;
	}

	if (StartOffset_DEPRECATED != 0.f)
	{
		Params.StartOffset_DEPRECATED = StartOffset_DEPRECATED;
	}

	if (EndOffset_DEPRECATED != 0.f)
	{
		Params.EndOffset_DEPRECATED = EndOffset_DEPRECATED;
	}

	if (PlayRate_DEPRECATED != 1.f)
	{
		Params.PlayRate = PlayRate_DEPRECATED;
	}

	if (bReverse_DEPRECATED != false)
	{
		Params.bReverse = bReverse_DEPRECATED;
	}

	if (SlotName_DEPRECATED != DefaultSlotName)
	{
		Params.SlotName = SlotName_DEPRECATED;
	}

	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();

	if (MovieScene)
	{
		FFrameRate DisplayRate = MovieScene->GetDisplayRate();
		FFrameRate TickResolution = MovieScene->GetTickResolution();

		if (Params.StartOffset_DEPRECATED != SkeletalDeprecatedMagicNumber)
		{
			Params.StartFrameOffset = ConvertFrameTime(FFrameTime::FromDecimal(DisplayRate.AsDecimal() * Params.StartOffset_DEPRECATED), DisplayRate, TickResolution).FrameNumber;

			Params.StartOffset_DEPRECATED = SkeletalDeprecatedMagicNumber;
		}

		if (Params.EndOffset_DEPRECATED != SkeletalDeprecatedMagicNumber)
		{
			Params.EndFrameOffset = ConvertFrameTime(FFrameTime::FromDecimal(DisplayRate.AsDecimal() * Params.EndOffset_DEPRECATED), DisplayRate, TickResolution).FrameNumber;

			Params.EndOffset_DEPRECATED = SkeletalDeprecatedMagicNumber;
		}
	}

	// if version is less than this
	if (GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::ConvertEnableRootMotionToForceRootLock)
	{
		UAnimSequence* AnimSeq = Cast<UAnimSequence>(Params.Animation);
		if (AnimSeq && AnimSeq->bEnableRootMotion && !AnimSeq->bForceRootLock)
		{
			// this is not ideal, but previously single player node was using this flag to whether or not to extract root motion
			// with new anim sequencer instance, this would break because we use the instance flag to extract root motion or not
			// so instead of setting that flag, we use bForceRootLock flag to asset
			// this can have side effect, where users didn't want that to be on to start with
			// So we'll notify users to let them know this has to be saved
			AnimSeq->bForceRootLock = true;
			AnimSeq->MarkPackageDirty();
			// warning to users
#if WITH_EDITOR			
			if (!IsRunningGame())
			{
				static FName NAME_LoadErrors("LoadErrors");
				FMessageLog LoadErrors(NAME_LoadErrors);

				TSharedRef<FTokenizedMessage> Message = LoadErrors.Warning();
				Message->AddToken(FTextToken::Create(LOCTEXT("RootMotionFixUp1", "The Animation ")));
				Message->AddToken(FAssetNameToken::Create(AnimSeq->GetPathName(), FText::FromString(GetNameSafe(AnimSeq))));
				Message->AddToken(FTextToken::Create(LOCTEXT("RootMotionFixUp2", "will be set to ForceRootLock on. Please save the animation if you want to keep this change.")));
				Message->SetSeverity(EMessageSeverity::Warning);
				LoadErrors.Notify();
			}
#endif // WITH_EDITOR

			UE_LOGF(LogMovieScene, Warning, "%ls Animation has set ForceRootLock to be used in Sequencer. If this animation is used in anywhere else using root motion, that will cause conflict.", *AnimSeq->GetName());
		}
	}

	Super::PostLoad();
}

TOptional<TRange<FFrameNumber> > UMovieSceneSkeletalAnimationSection::GetAutoSizeRange() const
{
	if (UMovieScene* MovieScene = GetTypedOuter<UMovieScene>())
	{
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FMovieSceneInverseSequenceTransform InnerToOuterTransform = Params.MakeTransform(TickResolution, GetRange()).Inverse();
		
		FFrameTime AnimationLength = Params.GetSequenceLength() * TickResolution;
		int32 IFrameNumber = AnimationLength.FrameNumber.Value + static_cast<int32>(AnimationLength.GetSubFrame() + 0.5f) + 1;

		FFrameTime InnerStartTime = Params.StartFrameOffset + Params.FirstLoopStartFrameOffset;
		FFrameTime InnerEndTime = IFrameNumber - Params.EndFrameOffset;

		TOptional<FFrameTime> OuterStartTime = InnerToOuterTransform.TryTransformTime(InnerStartTime);
		TOptional<FFrameTime> OuterEndTime = InnerToOuterTransform.TryTransformTime(InnerEndTime);
		if (OuterStartTime.IsSet() && OuterEndTime.IsSet())
		{
			return TRange<FFrameNumber>(GetInclusiveStartFrame(), GetInclusiveStartFrame() + (OuterEndTime.GetValue() - OuterStartTime.GetValue()).FrameNumber);
		}
	}
	return TOptional<TRange<FFrameNumber>>();
}

void UMovieSceneSkeletalAnimationSection::TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys)
{
	SetFlags(RF_Transactional);

	if (TryModify())
	{
		if (bTrimLeft)
		{
			FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();
			FMovieSceneSequenceTransform OuterToInnerTransform = Params.MakeTransform(FrameRate, GetRange());

			FFrameTime AnimationTimeInSeconds = OuterToInnerTransform.TransformTime(TrimTime.Time);
			Params.FirstLoopStartFrameOffset = (AnimationTimeInSeconds.AsDecimal() * FrameRate).FrameNumber;
		}

		Super::TrimSection(TrimTime, bTrimLeft, bDeleteKeys);
	}
}

UMovieSceneSection* UMovieSceneSkeletalAnimationSection::SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys)
{
	//handle root motion, only in editor
#if WITH_EDITOR

	TOptional<FTransform> RootTransform;
	FName BoneName;
	UMovieSceneCommonAnimationTrack* Track = GetTypedOuter<UMovieSceneCommonAnimationTrack>();
	//if we are doing root motion then get the BoneName 
	if (Track)
	{
		int32 BoneIndex = SetBoneIndexForRootMotionCalculations(Track->bBlendFirstChildOfRoot);
		FMovieSceneSkeletalAnimRootMotionTrackParams* RootMotionParams = GetRootMotionParams();
		if (RootMotionParams->bHaveRootMotion)
		{
			if (UAnimSequence* AnimSequence = Cast<UAnimSequence>(Params.Animation))
			{
				const FReferenceSkeleton& RefSkeleton = AnimSequence->GetSkeleton()->GetReferenceSkeleton();
				RootTransform = Track->GetRootMotion(SplitTime.Time);
				if (RootTransform.IsSet())
				{
					BoneName = RefSkeleton.GetBoneName(BoneIndex);
				}
			}
		}
	}
#endif

	//handle FirstLoopStartFrameOffset
	const FFrameNumber InitialFirstLoopStartFrameOffset = Params.FirstLoopStartFrameOffset;

	FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();
	FMovieSceneSequenceTransform OuterToInnerTransform = Params.MakeTransform(FrameRate, GetRange());

	FFrameTime AnimationTimeInSeconds = OuterToInnerTransform.TransformTime(SplitTime.Time);
	const FFrameNumber NewOffset = (AnimationTimeInSeconds.AsDecimal() * FrameRate).FrameNumber;

	UMovieSceneSkeletalAnimationSection* NewSection = Cast<UMovieSceneSkeletalAnimationSection>(Super::SplitSection(SplitTime, bDeleteKeys));
	if (NewSection != nullptr)
	{
		NewSection->Params.FirstLoopStartFrameOffset = NewOffset;

#if WITH_EDITOR

		if (RootTransform.IsSet())
		{
			NewSection->bMatchTranslation = NewSection->bMatchIncludeZHeight = true;
			NewSection->bMatchRotationYaw = NewSection->bMatchRotationPitch = NewSection->bMatchRotationRoll = true;
			NewSection->bMatchWithPrevious = true;
			NewSection->MatchedLocationOffset = FVector(0.0, 0.0, 0.0);
			NewSection->MatchedRotationOffset = FRotator(0.0, 0.0, 0.0);
			if (NewSection->GetRootMotionParams())
			{
				NewSection->GetRootMotionParams()->bRootMotionsDirty = true;
			}
		}
#endif
	}

	// Restore original offset modified by splitting
	Params.FirstLoopStartFrameOffset = InitialFirstLoopStartFrameOffset;
	return NewSection;
}


void UMovieSceneSkeletalAnimationSection::GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const
{
	using namespace UE::MovieScene;

	Super::GetSnapTimes(OutSnapTimes, bGetSectionBorders);

	const UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	const FFrameNumber StartFrame = GetInclusiveStartFrame();
	const FFrameNumber EndFrame   = GetExclusiveEndFrame();

	if (!MovieScene)
	{
		return;
	}

	auto VisitBoundary = [&OutSnapTimes](FFrameTime InTime)
	{
		OutSnapTimes.Add(InTime.RoundToFrame());
		return true;
	};

	FMovieSceneSequenceTransform OuterToInnerTransform = Params.MakeTransform(MovieScene->GetTickResolution(), GetRange());

	if (!OuterToInnerTransform.ExtractBoundariesWithinRange(StartFrame, EndFrame, VisitBoundary))
	{
		FMovieSceneInverseSequenceTransform InnerToOuterTransform = OuterToInnerTransform.Inverse();

		TOptional<FFrameTime> AnimEnd   = InnerToOuterTransform.TryTransformTime(FFrameTime::FromDecimal(Params.GetSequenceLength()));

		if (AnimEnd && AnimEnd.GetValue() < EndFrame)
		{
			VisitBoundary(AnimEnd.GetValue());
		}
	}
}

double UMovieSceneSkeletalAnimationSection::MapTimeToAnimation(FFrameTime InPosition, FFrameRate InTickResolution) const
{
	return Params.MapTimeToAnimation(this, InPosition, InTickResolution);
}

float UMovieSceneSkeletalAnimationSection::GetTotalWeightValue(FFrameTime InTime) const
{
	float ManualWeight = 1.f;
	Params.Weight.Evaluate(InTime, ManualWeight);
	return ManualWeight *  EvaluateEasing(InTime);
}

void UMovieSceneSkeletalAnimationSection::SetRange(const TRange<FFrameNumber>& NewRange)
{
	UMovieSceneSection::SetRange(NewRange);
	if (GetRootMotionParams())
	{
		GetRootMotionParams()->bRootMotionsDirty = true;
	}
}
void UMovieSceneSkeletalAnimationSection::SetStartFrame(TRangeBound<FFrameNumber> NewStartFrame)
{
	UMovieSceneSection::SetStartFrame(NewStartFrame);
	if (GetRootMotionParams())
	{
		GetRootMotionParams()->bRootMotionsDirty = true;
	}

}
void UMovieSceneSkeletalAnimationSection::SetEndFrame(TRangeBound<FFrameNumber> NewEndFrame)
{
	UMovieSceneSection::SetEndFrame(NewEndFrame);
	if (GetRootMotionParams())
	{
		GetRootMotionParams()->bRootMotionsDirty = true;
	}
}

FMovieSceneTimeWarpVariant* UMovieSceneSkeletalAnimationSection::GetTimeWarp()
{
	return &Params.PlayRate;
}

UObject* UMovieSceneSkeletalAnimationSection::GetSourceObject() const
{
	return Params.Animation;
}

void UMovieSceneSkeletalAnimationSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& InParams, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	const FGuid ObjectBindingID = InParams.GetObjectBindingID();
	const bool bIsAdditive = Params.Animation ? Params.Animation->IsValidAdditive() : false;

	if (!ObjectBindingID.IsValid())
	{
		return;
	}

	FMovieSceneSkeletalAnimationComponentData ComponentData { this };

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(TrackComponents->SkeletalAnimation, ComponentData)
		.Add(BuiltInComponents->GenericObjectBinding, ObjectBindingID)
		.Add(BuiltInComponents->BoundObjectResolver, UMovieSceneSkeletalAnimationSystem::ResolveSkeletalMeshComponentBinding)
		.AddConditional(BuiltInComponents->WeightChannel, &Params.Weight, Params.Weight.HasAnyData())
		.AddTagConditional(BuiltInComponents->Tags.AdditiveAnimation, bIsAdditive)
	);
}

bool UMovieSceneSkeletalAnimationSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	if (!Params.Animation)
	{
		return true;
	}

	return false;
}

#if WITH_EDITOR
bool UMovieSceneSkeletalAnimationSection::Modify(bool bAlwaysMarkDirty)
{
	if (GetRootMotionParams())
	{
		GetRootMotionParams()->bRootMotionsDirty = true;
	}
	return Super::Modify(bAlwaysMarkDirty);
}

void UMovieSceneSkeletalAnimationSection::PreEditChange(FProperty* PropertyAboutToChange)
{
	// Store the current play rate so that we can compute the amount to compensate the section end time when the play rate changes
	if (Params.PlayRate.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
	{
		PreviousPlayRate = Params.PlayRate.AsFixedPlayRateFloat();
	}

	Super::PreEditChange(PropertyAboutToChange);
}

void UMovieSceneSkeletalAnimationSection::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	// Adjust the duration automatically if the play rate changes
	if (PropertyChangedEvent.Property != nullptr &&
		PropertyChangedEvent.Property->GetFName() == TEXT("PlayRate"))
	{
		if (Params.PlayRate.GetType() == EMovieSceneTimeWarpType::FixedPlayRate)
		{
			float NewPlayRate = Params.PlayRate.AsFixedPlayRateFloat();

			if (NewPlayRate > KINDA_SMALL_NUMBER)
			{
				float CurrentDuration = UE::MovieScene::DiscreteSize(GetRange());
				float NewDuration = CurrentDuration * (PreviousPlayRate / NewPlayRate);
				SetEndFrame( GetInclusiveStartFrame() + FMath::FloorToInt(NewDuration) );

				PreviousPlayRate = NewPlayRate;
			}
		}

		ChannelProxy = nullptr;
	}
	if (GetRootMotionParams())
	{
		GetRootMotionParams()->bRootMotionsDirty = true;
	}
	Super::PostEditChangeProperty(PropertyChangedEvent);
}

void UMovieSceneSkeletalAnimationSection::PostEditImport()
{
	if (GetRootMotionParams())
	{
		GetRootMotionParams()->bRootMotionsDirty = true;
	}
	Super::PostEditImport();
}
void UMovieSceneSkeletalAnimationSection::PostEditUndo()
{
	if (GetRootMotionParams())
	{
		GetRootMotionParams()->bRootMotionsDirty = true;
	}
	Super::PostEditUndo();
}

#endif

void UMovieSceneSkeletalAnimationSection::GetRootMotion(FFrameTime CurrentTime, UMovieSceneSkeletalAnimationSection::FRootMotionParams& OutRootMotionParams) const
{
	if (GetRootMotionParams())
	{
		if (UMovieSceneCommonAnimationTrack* Track = GetTypedOuter<UMovieSceneCommonAnimationTrack>())
		{
			OutRootMotionParams.Transform = Track->GetRootMotion(CurrentTime);
			OutRootMotionParams.ChildBoneIndex = TempRootBoneIndex.IsSet() ? TempRootBoneIndex.GetValue() : INDEX_NONE;
			OutRootMotionParams.bBlendFirstChildOfRoot = Track->bBlendFirstChildOfRoot ;
			OutRootMotionParams.PreviousTransform = PreviousTransform;
		}
	}
}

bool UMovieSceneSkeletalAnimationSection::GetRootMotionVelocity(FFrameTime PreviousTime, FFrameTime CurrentTime, FFrameRate InTickResolution,
	FTransform& OutVelocity, float& OutWeight) const
{
	UAnimSequence* AnimSequence = Cast<UAnimSequence>(Params.Animation);
	if (AnimSequence)
	{
		float ManualWeight = 1.f;
		Params.Weight.Evaluate(CurrentTime, ManualWeight);
		OutWeight = ManualWeight * EvaluateEasing(CurrentTime);
		//mz todo we should be able to cache the PreviousTimeSeconds;
		//mz todo need to get the starting value.
		float PreviousTimeSeconds = static_cast<float>(MapTimeToAnimation(PreviousTime, InTickResolution));
		float CurrentTimeSeconds  = static_cast<float>(MapTimeToAnimation(CurrentTime, InTickResolution));
		OutVelocity = AnimSequence->ExtractRootMotionFromRange(PreviousTimeSeconds, CurrentTimeSeconds, FAnimExtractContext());
		return true;
	}
	return false;
}

FMovieSceneSkeletalAnimRootMotionTrackParams*  UMovieSceneSkeletalAnimationSection::GetRootMotionParams() const
{
	UMovieSceneCommonAnimationTrack* Track = GetTypedOuter<UMovieSceneCommonAnimationTrack>();
	if (Track)
	{
		return &Track->RootMotionParams;
	}
	return nullptr;
}

int32 UMovieSceneSkeletalAnimationSection::SetBoneIndexForRootMotionCalculations(bool bBlendFirstChildOfRoot)
{
	if (!bBlendFirstChildOfRoot)
	{
		TempRootBoneIndex.Reset();
		return 0;
	}
	else if (const UAnimSequence* AnimSequence = Cast<UAnimSequence>(Params.Animation))
	{
		if (TempRootBoneIndex.IsSet() == false || TempRootBoneIndex.GetValue() == INDEX_NONE)
		{
			//but if not first find first
			int32 RootIndex = INDEX_NONE;
#if WITH_EDITOR
			TArray<FName> TrackNames;
			AnimSequence->GetDataModelInterface()->GetBoneTrackNames(TrackNames);
			const FReferenceSkeleton& RefSkeleton = AnimSequence->GetSkeleton()->GetReferenceSkeleton();
			const IAnimationDataModel* DataModel = AnimSequence->GetDataModel();
			TArray<FTransform> OutTransforms;
			for (int32 BoneIndex = 0; BoneIndex < RefSkeleton.GetNum(); ++BoneIndex)
			{
				const FName BoneName = RefSkeleton.GetBoneName(BoneIndex);
				if (DataModel->IsValidBoneTrackName(BoneName))
				{
					DataModel->GetBoneTrackTransforms(BoneName, OutTransforms);
					TOptional<FTransform> LocalPreviousTransform;
					for (const FTransform& Transform : OutTransforms)
					{
						if (Transform.GetLocation().IsNearlyZero() == false
							&& LocalPreviousTransform.IsSet() && LocalPreviousTransform.GetValue().GetLocation() != Transform.GetLocation())
						{
							TempRootBoneIndex = BoneIndex;
							break;
						}
						LocalPreviousTransform = Transform;
					}
					
					OutTransforms.Reset();
					if (TempRootBoneIndex.IsSet())
					{
						break;
					}
					
				}
			}
#else
			UAnimSequence::FScopedCompressedAnimSequence CompressedData = AnimSequence->GetCompressedData();
			const TArray<FTrackToSkeletonMap>& BoneMappings = CompressedData.Get().CompressedTrackToSkeletonMapTable;
			const int32 NumTracks = BoneMappings.Num();
			for (int32 TrackIndex = 0; TrackIndex < NumTracks; ++TrackIndex)
			{
				const FTrackToSkeletonMap& Mapping = BoneMappings[TrackIndex];
				// verify if this bone exists in skeleton
				const int32 BoneTreeIndex = Mapping.BoneTreeIndex;
				if (BoneTreeIndex != INDEX_NONE)
				{
					int32 ParentIndex = AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetParentIndex(BoneTreeIndex);
					if (ParentIndex == INDEX_NONE)
					{
						RootIndex = TrackIndex;
					}
					else if (ParentIndex == RootIndex)
					{
						FTransform Transform;
						const int32 NumFrames = AnimSequence->GetNumberOfSampledKeys();
						for (int32 Index = 0; Index < NumFrames; ++Index)
						{
							const double Pos = FMath::Clamp(AnimSequence->GetSamplingFrameRate().AsSeconds(Index), 0.0, AnimSequence->GetPlayLength());
							AnimSequence->GetBoneTransform(Transform, FSkeletonPoseBoneIndex(BoneTreeIndex), FAnimExtractContext(Pos), false);
							if (Transform.Equals(FTransform::Identity) == false)
							{
								TempRootBoneIndex = BoneTreeIndex;
								break;
							}
						}
						if (TempRootBoneIndex.IsSet())
						{
							break;
						}
					}
				}
			}
#endif
		}
	}
	return TempRootBoneIndex.IsSet() ? TempRootBoneIndex.GetValue() : 0;
}

FTransform UMovieSceneSkeletalAnimationSection::GetRootMotionStartOffset() const
{
	UAnimSequence* AnimSequence = Cast<UAnimSequence>(Params.Animation);
	if (AnimSequence && TempRootBoneIndex.IsSet() && TempRootBoneIndex.GetValue() != 0)
	{
		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		const double StartSeconds = MovieScene ? (MapTimeToAnimation(FFrameNumber(0), MovieScene->GetTickResolution())): 0.0;
		return AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(StartSeconds), nullptr);
	}
	return FTransform::Identity;;
}

bool UMovieSceneSkeletalAnimationSection::GetRootMotionTransform(FAnimationPoseData& AnimationPoseData,FRootMotionTransformParam& InOutParams) const
{
	UAnimSequence* AnimSequence = Cast<UAnimSequence>(Params.Animation);
	FTransform OffsetTransform(FQuat(StartRotationOffset), StartLocationOffset);
	FTransform MatchedTransform(FQuat(MatchedRotationOffset), MatchedLocationOffset);

	if (AnimSequence)
	{
		float ManualWeight = 1.f;
		Params.Weight.Evaluate(InOutParams.CurrentTime, ManualWeight);
		InOutParams.OutWeight = ManualWeight * EvaluateEasing(InOutParams.CurrentTime);
		InOutParams.bOutIsAdditive = false;
		const double CurrentTimeSeconds = MapTimeToAnimation(InOutParams.CurrentTime, InOutParams.FrameRate);
		const double StartSeconds = (MapTimeToAnimation(FFrameNumber(0), InOutParams.FrameRate));

		InOutParams.bOutIsAdditive = AnimSequence->GetAdditiveAnimType() != EAdditiveAnimationType::AAT_None;
		FTransform StartBoneTransform;
		InOutParams.OutRootStartTransform = GetRootMotionStartOffset();

		if (TempRootBoneIndex.IsSet() && TempRootBoneIndex.GetValue() != 0)
		{
			//get the start pose first since we pass out the pose and need the current
			FCompactPoseBoneIndex PoseIndex = AnimationPoseData.GetPose().GetBoneContainer().GetCompactPoseIndexFromSkeletonIndex(TempRootBoneIndex.GetValue());
			FAnimExtractContext ExtractionContext(StartSeconds);
			AnimSequence->GetAnimationPose(AnimationPoseData, ExtractionContext);
			StartBoneTransform = AnimationPoseData.GetPose()[FCompactPoseBoneIndex(PoseIndex)];

			ExtractionContext.CurrentTime = CurrentTimeSeconds;
			AnimSequence->GetAnimationPose(AnimationPoseData, ExtractionContext);
			InOutParams.OutPoseTransform = AnimationPoseData.GetPose()[FCompactPoseBoneIndex(PoseIndex)];
		}
		else //not set then just use root.
		{
			StartBoneTransform = AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(StartSeconds), nullptr);
			InOutParams.OutPoseTransform = AnimSequence->ExtractRootTrackTransform(FAnimExtractContext(CurrentTimeSeconds), nullptr);
		}
		//note though we don't support mesh space addtive just local additive it will still work the same here for the root so 
		if (!InOutParams.bOutIsAdditive)
		{
			const bool bStartTransformOffsetInBoneSpace = CVarStartTransformOffsetInBoneSpace.GetValueOnGameThread();
			if (bStartTransformOffsetInBoneSpace)
			{
				FTransform StartMatchedInRoot = StartBoneTransform * MatchedTransform;
				FTransform LocalToRoot = (InOutParams.OutPoseTransform * StartBoneTransform.Inverse());
				FTransform OffsetInLocalSpace = LocalToRoot * OffsetTransform;
				InOutParams.OutTransform = OffsetInLocalSpace * StartMatchedInRoot;
			}
			else
			{
				InOutParams.OutTransform = InOutParams.OutPoseTransform * OffsetTransform * MatchedTransform;

			}
			InOutParams.OutParentTransform = OffsetTransform.GetRelativeTransformReverse(InOutParams.OutTransform);

		}
		return true;
	}
	//for safety always return true for now
	InOutParams.OutParentTransform = OffsetTransform * MatchedTransform;
	InOutParams.OutTransform = InOutParams.OutParentTransform;
	InOutParams.OutPoseTransform = FTransform::Identity;
	return true;
}

//this is actually multiply out on previous clips, let's refactor with new anim mixer
void UMovieSceneSkeletalAnimationSection::MultiplyOutInverseOnNextClips(FVector PreviousMatchedLocationOffset, FRotator PreviousMatchedRotationOffset)
{
	UMovieSceneCommonAnimationTrack* Track = GetTypedOuter<UMovieSceneCommonAnimationTrack>();
	if (Track)
	{
		
		bool bMultiplyOutInverse = false;
		//calculate the diff here....
		FTransform Previous(PreviousMatchedRotationOffset.Quaternion(), PreviousMatchedLocationOffset);
		FTransform Matched(MatchedRotationOffset.Quaternion(), MatchedLocationOffset);
		FTransform Inverse = Previous.GetRelativeTransformReverse(Matched);
		for(int32 Index = Track->AnimationSections.Num() - 1; Index >=0; -- Index)
		{ 
			UMovieSceneSection* Section  = Track->AnimationSections[Index];
			if (bMultiplyOutInverse)
			{
				UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section);
				if (AnimSection)
				{
					//then for all of the next sections we need to multiply that diff through.
					FTransform CurrentMatched(AnimSection->MatchedRotationOffset, AnimSection->MatchedLocationOffset);
					FTransform NewMatched = Inverse.GetRelativeTransformReverse(CurrentMatched);
					AnimSection->MatchedLocationOffset = NewMatched.GetTranslation();
					AnimSection->MatchedRotationOffset = NewMatched.GetRotation().Rotator();
				}
				break;
			}
			if (Section == this) //next ones we multiply out
			{
				bMultiplyOutInverse = true;
			}
		}
	}
}
void UMovieSceneSkeletalAnimationSection::ClearMatchedOffsetTransforms()
{
	//need to store the previous since we may need to apply the change we made to the next clips so they don't move
	FVector PreviousMatchedLocationOffset = MatchedLocationOffset;
	FRotator PreviousMatchedRotationOffset = MatchedRotationOffset;
	MatchedLocationOffset = FVector::ZeroVector;
	MatchedRotationOffset = FRotator::ZeroRotator;
	if (bMatchWithPrevious == false)
	{
		MultiplyOutInverseOnNextClips(PreviousMatchedLocationOffset, PreviousMatchedRotationOffset);
	}
	bMatchWithPrevious = true;
	MatchedBoneName = NAME_None;
	if (GetRootMotionParams())
	{
		GetRootMotionParams()->bRootMotionsDirty = true;
	}
}


void UMovieSceneSkeletalAnimationSection::MatchSectionByBoneTransform(USkeletalMeshComponent* SkelMeshComp, FFrameTime CurrentFrame, FFrameRate InTickResolution,
	const FName& BoneName)
{
	MatchedBoneName = BoneName;
	UMovieSceneCommonAnimationTrack* Track = GetTypedOuter<UMovieSceneCommonAnimationTrack>();
	if (Track)
	{
		FTransform DiffTransform;
		FVector DiffTranslate;
		FQuat DiffRotate; 
		//need to store the previous since we may need to apply the change we made to the next clips so they don't move
		FVector PreviousMatchedLocationOffset = MatchedLocationOffset;
		FRotator PreviousMatchedRotationOffset = MatchedRotationOffset;

		Track->MatchSectionByBoneTransform(bMatchWithPrevious,SkelMeshComp, this, CurrentFrame, InTickResolution, BoneName, DiffTransform,DiffTranslate,DiffRotate);

		MatchedLocationOffset = bMatchTranslation ? DiffTranslate : FVector::ZeroVector;
		MatchedRotationOffset = DiffRotate.Rotator();
		
		if (bMatchWithPrevious == false)
		{
			MultiplyOutInverseOnNextClips(PreviousMatchedLocationOffset, PreviousMatchedRotationOffset);
		}
		
		if (GetRootMotionParams())
		{
			GetRootMotionParams()->bRootMotionsDirty = true;
		}
	}
}


void UMovieSceneSkeletalAnimationSection::ToggleMatchTranslation()
{
	bMatchTranslation = bMatchTranslation ? false : true;
	GetRootMotionParams()->bRootMotionsDirty = true;
}

void UMovieSceneSkeletalAnimationSection::ToggleMatchIncludeZHeight()
{
	bMatchIncludeZHeight = bMatchIncludeZHeight ? false : true;
	GetRootMotionParams()->bRootMotionsDirty = true;
}

void UMovieSceneSkeletalAnimationSection::ToggleMatchIncludeYawRotation()
{
	bMatchRotationYaw = bMatchRotationYaw ? false : true;
	GetRootMotionParams()->bRootMotionsDirty = true;
}

void UMovieSceneSkeletalAnimationSection::ToggleMatchIncludePitchRotation()
{
	bMatchRotationPitch = bMatchRotationPitch ? false : true;
	GetRootMotionParams()->bRootMotionsDirty = true;
}

void UMovieSceneSkeletalAnimationSection::ToggleMatchIncludeRollRotation()
{
	bMatchRotationRoll = bMatchRotationRoll ? false : true;
	GetRootMotionParams()->bRootMotionsDirty = true;
}

#if WITH_EDITORONLY_DATA

void UMovieSceneSkeletalAnimationSection::ToggleShowSkeleton()
{
	bShowSkeleton = bShowSkeleton ? false : true;
}

#endif


#undef LOCTEXT_NAMESPACE 
