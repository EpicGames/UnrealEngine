// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneSlomoSection.h"
#include "UObject/SequencerObjectVersion.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSlomoSection)


/* UMovieSceneSlomoSection structors
 *****************************************************************************/

UMovieSceneSlomoSection::UMovieSceneSlomoSection()
{
#if WITH_EDITORONLY_DATA
	bIsInfinite_DEPRECATED = true;
#endif

	bSupportsInfiniteRange = true;
	SetRange(TRange<FFrameNumber>::All());
	FloatCurve.SetDefault(1.f);
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);

#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(FloatCurve, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<float>::Make());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(FloatCurve);

#endif
}

void UMovieSceneSlomoSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!FloatCurve.HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddTag(BuiltInComponents->Tags.Root)
		.AddTag(TrackComponents->Tags.Slomo)
		.Add(BuiltInComponents->FloatChannel[0], &FloatCurve)
	);
}

