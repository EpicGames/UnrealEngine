// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneStringSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "UObject/SequencerObjectVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneStringSection)

UMovieSceneStringSection::UMovieSceneStringSection( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	bSupportsInfiniteRange = true;
	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);

#if WITH_EDITOR

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(StringCurve, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<FString>::Make());

#else

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(StringCurve);

#endif
}

bool UMovieSceneStringSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneStringSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!StringCurve.HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->String)
		.Add(Components->StringChannel, &StringCurve)
		.Commit(this, Params, OutImportedEntity);
}

