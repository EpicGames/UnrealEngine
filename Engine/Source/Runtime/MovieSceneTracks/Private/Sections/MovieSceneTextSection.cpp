// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneTextSection.h"
#include "Channels/MovieSceneTextChannel.h"
#include "Tracks/MovieScenePropertyTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTextSection)

UMovieSceneTextSection::UMovieSceneTextSection(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	bSupportsInfiniteRange = true;
	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);

#if WITH_EDITOR
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(TextChannel, FMovieSceneChannelMetaData(), TMovieSceneExternalValue<FText>::Make());
	TextChannel.SetPackage(GetPackage());
#else
	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(TextChannel);
#endif
}

bool UMovieSceneTextSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange
	, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData
	, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneTextSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker
	, const FEntityImportParams& Params
	, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!TextChannel.HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->Text)
		.Add(Components->TextChannel, &TextChannel)
		.Commit(this, Params, OutImportedEntity);
}
