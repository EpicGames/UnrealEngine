// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneVectorSection.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Tracks/MovieSceneVectorTrack.h"
#include "UObject/SequencerObjectVersion.h"
#include "UObject/StructOnScope.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneVectorSection)

/* FMovieSceneVectorKeyStruct interface
 *****************************************************************************/

#if WITH_EDITOR
template<typename ValueType>
struct TVectorSectionEditorDataCompositeValue
{
	ValueType X, Y, Z, W;
	TVectorSectionEditorDataCompositeValue() : X(0), Y(0), Z(0), W(0) {}
	TVectorSectionEditorDataCompositeValue(ValueType InX, ValueType InY, ValueType InZ, ValueType InW)
		: X(InX), Y(InY), Z(InZ), W(InW)
	{}
};

template<typename ValueType>
struct TVectorSectionEditorDataTraits;

template<>
struct TVectorSectionEditorDataTraits<float>
{
	using FCompositeValue = TVectorSectionEditorDataCompositeValue<float>;

	static FCompositeValue GetPropertyValue(UObject& InObject, FTrackInstancePropertyBindings& Bindings, int32 NumChannels)
	{
		if (NumChannels == 2)
		{
			FVector2f Vector = Bindings.GetCurrentValue<FVector2f>(InObject);
			return FCompositeValue(Vector.X, Vector.Y, 0.f, 0.f);
		}
		else if (NumChannels == 3)
		{
			FVector3f Vector = Bindings.GetCurrentValue<FVector3f>(InObject);
			return FCompositeValue(Vector.X, Vector.Y, Vector.Z, 0.f);
		}
		else
		{
			ensure(NumChannels == 4);
			FVector4f Vector = Bindings.GetCurrentValue<FVector4f>(InObject);
			return FCompositeValue(Vector.X, Vector.Y, Vector.Z, Vector.W);
		}
	}
};

template<>
struct TVectorSectionEditorDataTraits<double>
{
	using FCompositeValue = TVectorSectionEditorDataCompositeValue<double>;

	static FCompositeValue GetPropertyValue(UObject& InObject, FTrackInstancePropertyBindings& Bindings, int32 NumChannels)
	{
		if (NumChannels == 2)
		{
			FVector2D Vector = Bindings.GetCurrentValue<FVector2D>(InObject);
			return FCompositeValue(Vector.X, Vector.Y, 0.f, 0.f);
		}
		else if (NumChannels == 3)
		{
			FVector3d Vector = Bindings.GetCurrentValue<FVector3d>(InObject);
			return FCompositeValue(Vector.X, Vector.Y, Vector.Z, 0.f);
		}
		else
		{
			ensure(NumChannels == 4);
			FVector4d Vector = Bindings.GetCurrentValue<FVector4d>(InObject);
			return FCompositeValue(Vector.X, Vector.Y, Vector.Z, Vector.W);
		}
	}
};

template<typename ValueType>
struct TVectorSectionEditorData
{
	TVectorSectionEditorData(int32 NumChannels)
	{
		MetaData[0].SetIdentifiers("Vector.X", FCommonChannelData::ChannelX);
		MetaData[0].SubPropertyPath = TEXT("X");
		MetaData[0].SortOrder = 0;
		MetaData[0].Color = FCommonChannelData::RedChannelColor;
		MetaData[0].bCanCollapseToTrack = false;

		MetaData[1].SetIdentifiers("Vector.Y", FCommonChannelData::ChannelY);
		MetaData[1].SubPropertyPath = TEXT("Y");
		MetaData[1].SortOrder = 1;
		MetaData[1].Color = FCommonChannelData::GreenChannelColor;
		MetaData[1].bCanCollapseToTrack = false;

		MetaData[2].SetIdentifiers("Vector.Z", FCommonChannelData::ChannelZ);
		MetaData[2].SubPropertyPath = TEXT("Z");
		MetaData[2].SortOrder = 2;
		MetaData[2].Color = FCommonChannelData::BlueChannelColor;
		MetaData[2].bCanCollapseToTrack = false;

		MetaData[3].SetIdentifiers("Vector.W", FCommonChannelData::ChannelW);
		MetaData[3].SubPropertyPath = TEXT("W");
		MetaData[3].SortOrder = 3;
		MetaData[3].bCanCollapseToTrack = false;

		ExternalValues[0].OnGetExternalValue = [NumChannels](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return ExtractChannelX(InObject, Bindings, NumChannels); };
		ExternalValues[1].OnGetExternalValue = [NumChannels](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return ExtractChannelY(InObject, Bindings, NumChannels); };
		ExternalValues[2].OnGetExternalValue = [NumChannels](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return ExtractChannelZ(InObject, Bindings, NumChannels); };
		ExternalValues[3].OnGetExternalValue = [NumChannels](UObject& InObject, FTrackInstancePropertyBindings* Bindings) { return ExtractChannelW(InObject, Bindings, NumChannels); };
	}
	
	static TOptional<ValueType> ExtractChannelX(UObject& InObject, FTrackInstancePropertyBindings* Bindings, int32 NumChannels)
	{
		return Bindings ? TVectorSectionEditorDataTraits<ValueType>::GetPropertyValue(InObject, *Bindings, NumChannels).X : TOptional<ValueType>();
	}
	static TOptional<ValueType> ExtractChannelY(UObject& InObject, FTrackInstancePropertyBindings* Bindings, int32 NumChannels)
	{
		return Bindings ? TVectorSectionEditorDataTraits<ValueType>::GetPropertyValue(InObject, *Bindings, NumChannels).Y : TOptional<ValueType>();
	}
	static TOptional<ValueType> ExtractChannelZ(UObject& InObject, FTrackInstancePropertyBindings* Bindings, int32 NumChannels)
	{
		return Bindings ? TVectorSectionEditorDataTraits<ValueType>::GetPropertyValue(InObject, *Bindings, NumChannels).Z : TOptional<ValueType>();
	}
	static TOptional<ValueType> ExtractChannelW(UObject& InObject, FTrackInstancePropertyBindings* Bindings, int32 NumChannels)
	{
		return Bindings ? TVectorSectionEditorDataTraits<ValueType>::GetPropertyValue(InObject, *Bindings, NumChannels).W : TOptional<ValueType>();
	}

	FMovieSceneChannelMetaData			MetaData[4];
	TMovieSceneExternalValue<ValueType> ExternalValues[4];
};
#endif

void FMovieSceneFloatVectorKeyStructBase::PropagateChanges(const FPropertyChangedEvent& ChangeEvent)
{
	KeyStructInterop.Apply(Time);
}

void FMovieSceneDoubleVectorKeyStructBase::PropagateChanges(const FPropertyChangedEvent& ChangeEvent)
{
	KeyStructInterop.Apply(Time);
}


/* UMovieSceneFloatVectorSection structors
 *****************************************************************************/

UMovieSceneFloatVectorSection::UMovieSceneFloatVectorSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ChannelsUsed = 0;
	bSupportsInfiniteRange = true;

	EvalOptions.EnableAndSetCompletionMode
		(GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToRestoreState ? 
			EMovieSceneCompletionMode::KeepState : 
			GetLinkerCustomVersion(FSequencerObjectVersion::GUID) < FSequencerObjectVersion::WhenFinishedDefaultsToProjectDefault ? 
			EMovieSceneCompletionMode::RestoreState : 
			EMovieSceneCompletionMode::ProjectDefault);
	BlendType = EMovieSceneBlendType::Absolute;
}

void UMovieSceneFloatVectorSection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		RecreateChannelProxy();
	}
}

void UMovieSceneFloatVectorSection::PostEditImport()
{
	Super::PostEditImport();

	RecreateChannelProxy();
}

void UMovieSceneFloatVectorSection::RecreateChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

	check(ChannelsUsed <= UE_ARRAY_COUNT(Curves));

#if WITH_EDITOR

	const TVectorSectionEditorData<float> EditorData(ChannelsUsed);
	for (int32 Index = 0; Index < ChannelsUsed; ++Index)
	{
		Channels.Add(Curves[Index], EditorData.MetaData[Index], EditorData.ExternalValues[Index]);
	}

#else

	for (int32 Index = 0; Index < ChannelsUsed; ++Index)
	{
		Channels.Add(Curves[Index]);
	}

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

TSharedPtr<FStructOnScope> UMovieSceneFloatVectorSection::GetKeyStruct(TArrayView<const FKeyHandle> KeyHandles)
{
	if (!ChannelProxy)
	{
		GetChannelProxy();
	}

	TSharedPtr<FStructOnScope> KeyStruct;
	if (ChannelsUsed == 2)
	{
		KeyStruct = MakeShareable(new FStructOnScope(FMovieSceneVector2fKeyStruct::StaticStruct()));
	}
	else if (ChannelsUsed == 3)
	{
		KeyStruct = MakeShareable(new FStructOnScope(FMovieSceneVector3fKeyStruct::StaticStruct()));
	}
	else if (ChannelsUsed == 4)
	{
		KeyStruct = MakeShareable(new FStructOnScope(FMovieSceneVector4fKeyStruct::StaticStruct()));
	}

	if (KeyStruct.IsValid())
	{
		FMovieSceneFloatVectorKeyStructBase* Struct = (FMovieSceneFloatVectorKeyStructBase*)KeyStruct->GetStructMemory();
		for (int32 Index = 0; Index < ChannelsUsed; ++Index)
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneFloatChannel>(Index), Struct->GetPropertyChannelByIndex(Index), KeyHandles));
		}

		Struct->KeyStructInterop.SetStartingValues();
		Struct->Time = Struct->KeyStructInterop.GetUnifiedKeyTime().Get(0);
	}

	return KeyStruct;
}

bool UMovieSceneFloatVectorSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneFloatVectorSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (ChannelsUsed <= 0)
	{
		return;
	}
	else if (!Curves[0].HasAnyData() && !Curves[1].HasAnyData() && !Curves[2].HasAnyData() && !Curves[3].HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->FloatVector)
		.AddConditional(Components->FloatChannel[0], &Curves[0], ChannelsUsed > 0 && Curves[0].HasAnyData())
		.AddConditional(Components->FloatChannel[1], &Curves[1], ChannelsUsed > 1 && Curves[1].HasAnyData())
		.AddConditional(Components->FloatChannel[2], &Curves[2], ChannelsUsed > 2 && Curves[2].HasAnyData())
		.AddConditional(Components->FloatChannel[3], &Curves[3], ChannelsUsed > 3 && Curves[3].HasAnyData())
		.Commit(this, Params, OutImportedEntity);
}

/* UMovieSceneDoubleVectorSection structors
 *****************************************************************************/

UMovieSceneDoubleVectorSection::UMovieSceneDoubleVectorSection(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	ChannelsUsed = 0;
	bSupportsInfiniteRange = true;

	EvalOptions.EnableAndSetCompletionMode(EMovieSceneCompletionMode::ProjectDefault);
	BlendType = EMovieSceneBlendType::Absolute;
}

void UMovieSceneDoubleVectorSection::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.IsLoading())
	{
		RecreateChannelProxy();
	}
}

void UMovieSceneDoubleVectorSection::PostEditImport()
{
	Super::PostEditImport();

	RecreateChannelProxy();
}

void UMovieSceneDoubleVectorSection::RecreateChannelProxy()
{
	FMovieSceneChannelProxyData Channels;

	check(ChannelsUsed <= UE_ARRAY_COUNT(Curves));

#if WITH_EDITOR

	const TVectorSectionEditorData<double> EditorData(ChannelsUsed);
	for (int32 Index = 0; Index < ChannelsUsed; ++Index)
	{
		Channels.Add(Curves[Index], EditorData.MetaData[Index], EditorData.ExternalValues[Index]);
	}

#else

	for (int32 Index = 0; Index < ChannelsUsed; ++Index)
	{
		Channels.Add(Curves[Index]);
	}

#endif

	ChannelProxy = MakeShared<FMovieSceneChannelProxy>(MoveTemp(Channels));
}

TSharedPtr<FStructOnScope> UMovieSceneDoubleVectorSection::GetKeyStruct(TArrayView<const FKeyHandle> KeyHandles)
{
	if (!ChannelProxy)
	{
		GetChannelProxy();
	}

	TSharedPtr<FStructOnScope> KeyStruct;
	if (ChannelsUsed == 2)
	{
		KeyStruct = MakeShareable(new FStructOnScope(FMovieSceneVector2DKeyStruct::StaticStruct()));
	}
	else if (ChannelsUsed == 3)
	{
		KeyStruct = MakeShareable(new FStructOnScope(FMovieSceneVector3dKeyStruct::StaticStruct()));
	}
	else if (ChannelsUsed == 4)
	{
		KeyStruct = MakeShareable(new FStructOnScope(FMovieSceneVector4dKeyStruct::StaticStruct()));
	}

	if (KeyStruct.IsValid())
	{
		FMovieSceneDoubleVectorKeyStructBase* Struct = (FMovieSceneDoubleVectorKeyStructBase*)KeyStruct->GetStructMemory();
		for (int32 Index = 0; Index < ChannelsUsed; ++Index)
		{
			Struct->KeyStructInterop.Add(FMovieSceneChannelValueHelper(ChannelProxy->MakeHandle<FMovieSceneDoubleChannel>(Index), Struct->GetPropertyChannelByIndex(Index), KeyHandles));
		}

		Struct->KeyStructInterop.SetStartingValues();
		Struct->Time = Struct->KeyStructInterop.GetUnifiedKeyTime().Get(0);
	}

	return KeyStruct;
}

bool UMovieSceneDoubleVectorSection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(*this, EffectiveRange, InMetaData, OutFieldBuilder);
	return true;
}

void UMovieSceneDoubleVectorSection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (ChannelsUsed <= 0)
	{
		return;
	}
	else if (!Curves[0].HasAnyData() && !Curves[1].HasAnyData() && !Curves[2].HasAnyData() && !Curves[3].HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FPropertyTrackEntityImportHelper(TracksComponents->DoubleVector)
		.AddConditional(Components->DoubleChannel[0], &Curves[0], ChannelsUsed > 0 && Curves[0].HasAnyData())
		.AddConditional(Components->DoubleChannel[1], &Curves[1], ChannelsUsed > 1 && Curves[1].HasAnyData())
		.AddConditional(Components->DoubleChannel[2], &Curves[2], ChannelsUsed > 2 && Curves[2].HasAnyData())
		.AddConditional(Components->DoubleChannel[3], &Curves[3], ChannelsUsed > 3 && Curves[3].HasAnyData())
		.Commit(this, Params, OutImportedEntity);
}

