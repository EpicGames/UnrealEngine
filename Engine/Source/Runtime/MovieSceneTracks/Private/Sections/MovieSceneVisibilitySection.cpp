// Copyright Epic Games, Inc. All Rights Reserved.

#include "Sections/MovieSceneVisibilitySection.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "MovieScene.h"
#include "MovieSceneTimeHelpers.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Tracks/MovieScenePropertyTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneVisibilitySection)

namespace UE::MovieScene
{

int32 VisibilityToEntityID(bool bVisible)
{
	return bVisible ? 1 : 0;
}

bool EntityIDToVisibility(int32 EntityID)
{
	return EntityID > 0;
}

}  // namespace UE::MovieScene

UMovieSceneVisibilitySection::UMovieSceneVisibilitySection(const FObjectInitializer& Init)
	: Super(Init)
{
	BoolCurve.SetDefault(true);

#if WITH_EDITORONLY_DATA
	SetIsExternallyInverted(true);
#endif
}

void UMovieSceneVisibilitySection::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	UMovieScenePropertyTrack* OuterPropertyTrack = GetTypedOuter<UMovieScenePropertyTrack>();

	if (OuterPropertyTrack)
	{
		OutImportedEntity->AddBuilder(
			FEntityBuilder()
			.AddConditional(BuiltInComponents->GenericObjectBinding, Params.GetObjectBindingID(), Params.GetObjectBindingID().IsValid())
			.AddTag(TrackComponents->Tags.Visibility)
			.Add(BuiltInComponents->BoolResult, EntityIDToVisibility(Params.EntityID))
			.Add(BuiltInComponents->PropertyBinding, OuterPropertyTrack->GetPropertyBinding())
		);
	}
}

bool UMovieSceneVisibilitySection::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	using namespace UE::MovieScene;

	const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(InMetaData);

	// Add ranges when visibility is ON or OFF to the evaluation tree, skipping keys that don't actually
	// change the value. We will encode the ON/OFF value inside the entity ID.
	TArrayView<const FFrameNumber> Times  = BoolCurve.GetTimes();
	TArrayView<const bool>         Values = BoolCurve.GetValues();

	const TRange<FFrameNumber> TrueSectionRange = GetTrueRange();
	TRangeBound<FFrameNumber> StartBound = TrueSectionRange.GetLowerBound();

	if (Times.Num() == 0)
	{
		const bool bDefaultVisible = BoolCurve.GetDefault().Get(true);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, this, VisibilityToEntityID(bDefaultVisible), MetaDataIndex);
		return true;
	}

	// The first keyframe determines both the value before it and after it.
	bool bIsVisible = Values[0];

	for (int32 Index = 0; Index < Times.Num(); ++Index)
	{
		FFrameNumber CurrentTime = Times[Index];
		TRange<FFrameNumber> Range(StartBound, TRangeBound<FFrameNumber>::Exclusive(CurrentTime));
		if (!Range.IsEmpty())
		{
			OutFieldBuilder->AddPersistentEntity(Range, this, VisibilityToEntityID(bIsVisible), MetaDataIndex);
		}

		StartBound = TRangeBound<FFrameNumber>::Inclusive(CurrentTime);
		bIsVisible = Values[Index];
	}

	TRange<FFrameNumber> TailRange(StartBound, TrueSectionRange.GetUpperBound());
	if (!TailRange.IsEmpty())
	{
		OutFieldBuilder->AddPersistentEntity(TailRange, this, VisibilityToEntityID(bIsVisible), MetaDataIndex);
	}

	return true;
}

