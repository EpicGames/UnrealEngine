// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/ActorReferenceChannelEvaluatorSystem.h"

#include "Channels/MovieSceneActorReferenceChannel.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "GameFramework/Actor.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ActorReferenceChannelEvaluatorSystem)

namespace UE::MovieScene
{
	DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate actor reference channels"), MovieSceneEval_EvaluateActorReferenceChannelTask, STATGROUP_MovieSceneECS);

	struct FEvaluateActorReferenceChannels
	{
		const FInstanceRegistry* InstanceRegistry;

		FEvaluateActorReferenceChannels(const FInstanceRegistry* InInstanceRegistry)
			: InstanceRegistry(InInstanceRegistry)
		{}

		void ForEachEntity(FSourceActorReferenceChannel ActorReferenceChannel, FFrameTime FrameTime, FInstanceHandle InstanceHandle, FActorReferenceComponent& OutResult) const
		{
			FMovieSceneActorReferenceKey ObjectBinding;
			const bool bGotKeyValue = ActorReferenceChannel.Source->Evaluate(FrameTime, ObjectBinding);
			if (bGotKeyValue)
			{
				// (Re-)resolve the actor reference if we have a new valid actor to reference.
				if (ObjectBinding.Object.IsValid() && ObjectBinding.Object != OutResult.ObjectBindingID)
				{
					OutResult.WeakActor.Reset();
					ResolveActorReference(ObjectBinding, InstanceHandle, OutResult);
				}
				// Clear the actor pointer if we're supposed to not reference an actor anymore.
				else if (!ObjectBinding.Object.IsValid() && ObjectBinding.Object != OutResult.ObjectBindingID)
				{
					OutResult.ObjectBindingID = ObjectBinding.Object;
					OutResult.WeakActor.Reset();
				}
				// The actor reference hasn't changed but maybe the pointer was valid and became invalid (e.g. a spawnable went away),
				// or was invalid and became valid (e.g. a spawnable was spawned, unspawned, and has spawned again).
				// We don't need to handle the first case, since the weak pointer will naturally become null, but we need to handle
				// the second case.
				else if (ObjectBinding.Object.IsValid() && !OutResult.WeakActor.IsValid())
				{
					ensure(ObjectBinding.Object == OutResult.ObjectBindingID);
					ResolveActorReference(ObjectBinding, InstanceHandle, OutResult);
				}
			}
			else if (OutResult.ObjectBindingID.IsValid())
			{
				// We have no data anymore.
				OutResult.ObjectBindingID = ObjectBinding.Object;
				OutResult.WeakActor.Reset();
			}
		}

		void ResolveActorReference(const FMovieSceneActorReferenceKey ObjectBinding, FInstanceHandle InstanceHandle, FActorReferenceComponent& OutResult) const
		{
			const FSequenceInstance& TargetInstance = InstanceRegistry->GetInstance(InstanceHandle);

			for (TWeakObjectPtr<> WeakObject : ObjectBinding.Object.ResolveBoundObjects(TargetInstance))
			{
				if (AActor* Actor = Cast<AActor>(WeakObject.Get()))
				{
					OutResult.ObjectBindingID = ObjectBinding.Object;
					OutResult.WeakActor = Actor;

					// Can only ever reference one actor.
					break;
				}
			}
		}
	};
}


UActorReferenceChannelEvaluatorSystem::UActorReferenceChannelEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	RelevantComponent = TracksComponents->ActorReferenceChannel;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), BuiltInComponents->EvalTime);
		DefineComponentProducer(GetClass(), TracksComponents->ActorReferenceResult);
	}
}

void UActorReferenceChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	FEntityTaskBuilder()
	.Read(TracksComponents->ActorReferenceChannel)
	.Read(BuiltInComponents->EvalTime)
	.Read(BuiltInComponents->InstanceHandle)
	.Write(TracksComponents->ActorReferenceResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateActorReferenceChannelTask))
	.Fork_PerEntity<FEvaluateActorReferenceChannels>(&Linker->EntityManager, TaskScheduler, InstanceRegistry);
}

void UActorReferenceChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	FEntityTaskBuilder()
	.Read(TracksComponents->ActorReferenceChannel)
	.Read(BuiltInComponents->EvalTime)
	.Read(BuiltInComponents->InstanceHandle)
	.Write(TracksComponents->ActorReferenceResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateActorReferenceChannelTask))
	.Dispatch_PerEntity<FEvaluateActorReferenceChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents, InstanceRegistry);
}

