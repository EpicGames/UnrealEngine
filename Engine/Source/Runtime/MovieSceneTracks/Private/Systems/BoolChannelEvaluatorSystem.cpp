// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/BoolChannelEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "Channels/MovieSceneBoolChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(BoolChannelEvaluatorSystem)

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate bool channels"), MovieSceneEval_EvaluateBoolChannelTask, STATGROUP_MovieSceneECS);

namespace UE::MovieScene
{
	struct FEvaluateBoolChannels
	{
		static void ForEachEntity(FSourceBoolChannel BoolChannel, FFrameTime FrameTime, bool& OutResult)
		{
			if (!BoolChannel.Source->Evaluate(FrameTime, OutResult))
			{
				OutResult = false;
			}
		}
	};
} // namespace UE::MovieScene

UBoolChannelEvaluatorSystem::UBoolChannelEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;
	RelevantComponent = FBuiltInComponentTypes::Get()->BoolChannel;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
	}
}

void UBoolChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoolChannel)
	.Read(BuiltInComponents->EvalTime)
	.Write(BuiltInComponents->BoolResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateBoolChannelTask))
	.Fork_PerEntity<FEvaluateBoolChannels>(&Linker->EntityManager, TaskScheduler);
}

void UBoolChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	// Backwards compat Evaluation phase
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoolChannel)
	.Read(BuiltInComponents->EvalTime)
	.Write(BuiltInComponents->BoolResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateBoolChannelTask))
	.Dispatch_PerEntity<FEvaluateBoolChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents);
}

