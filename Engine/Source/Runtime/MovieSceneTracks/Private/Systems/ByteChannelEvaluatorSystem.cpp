// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/ByteChannelEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Math/NumericLimits.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ByteChannelEvaluatorSystem)

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate byte channels"), MovieSceneEval_EvaluateByteChannelTask, STATGROUP_MovieSceneECS);

namespace UE
{
namespace MovieScene
{

// @todo: for multi-bindings we currently re-evaluate the byte channel for each binding, even though the time is the same.
// Do we need to optimize for this case using something like the code below, while pessimizing the common (non-multi-bind) codepath??
struct FEvaluateByteChannels
{
	static void ForEachEntity(FSourceByteChannel ByteChannel, FFrameTime FrameTime, uint8& OutResult)
	{
		if (!ByteChannel.Source->Evaluate(FrameTime, OutResult))
		{
			OutResult = MIN_uint8;
		}
	}
};


} // namespace MovieScene
} // namespace UE


UByteChannelEvaluatorSystem::UByteChannelEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	RelevantComponent = BuiltInComponents->ByteChannel;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentProducer(UByteChannelEvaluatorSystem::StaticClass(), BuiltInComponents->ByteResult);

		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
	}
}

void UByteChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// Evaluate byte channels per instance and write the evaluated value into the output
	FEntityTaskBuilder()
	.Read(BuiltInComponents->ByteChannel)
	.Read(BuiltInComponents->EvalTime)
	.Write(BuiltInComponents->ByteResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateByteChannelTask))
	.Fork_PerEntity<FEvaluateByteChannels>(&Linker->EntityManager, TaskScheduler);
}

void UByteChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// Evaluate byte channels per instance and write the evaluated value into the output
	FEntityTaskBuilder()
	.Read(BuiltInComponents->ByteChannel)
	.Read(BuiltInComponents->EvalTime)
	.Write(BuiltInComponents->ByteResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateByteChannelTask))
	.Dispatch_PerEntity<FEvaluateByteChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents);
}


