// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"

#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneInterpolation.h"

#include "Algo/Find.h"
#include "Math/NumericLimits.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DoubleChannelEvaluatorSystem)

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate double channels"), MovieSceneEval_EvaluateDoubleChannelTask, STATGROUP_MovieSceneECS);

namespace UE::MovieScene
{

struct FDoubleChannelTypeAssociation
{
	TComponentTypeID<FSourceDoubleChannel> ChannelType;
	TComponentTypeID<Interpolation::FCachedInterpolation> CachedInterpolationType;
	TComponentTypeID<double> ResultType;
};

TArray<FDoubleChannelTypeAssociation, TInlineAllocator<4>> GDoubleChannelTypeAssociations;

// @todo: for multi-bindings we currently re-evaluate the double channel for each binding, even though the time is the same.
// Do we need to optimize for this case using something like the code below, while pessimizing the common (non-multi-bind) codepath??

/** Entity-component task that evaluates using a cached interpolation if possible */
struct FEvaluateDoubleChannels_Cached
{
	void ForEachEntity(FSourceDoubleChannel DoubleChannel, FFrameTime FrameTime, Interpolation::FCachedInterpolation& Cache, double& OutResult) const
	{
		if (!Cache.IsCacheValidForTime(FrameTime.GetFrame()))
		{
			Cache = DoubleChannel.Source->GetInterpolationForTime(FrameTime);
		}

		if (!Cache.Evaluate(FrameTime, OutResult))
		{
			OutResult = MIN_dbl;
		}
	}
};

/**
 * Mutation that removes FCachedInterpolation components from any channels that have a constant value
 * This effectively prevents the channel from ever being updated again
 */
struct FInitializeConstantDoubleChannelMutation : IMovieSceneConditionalEntityMutation
{
	FDoubleChannelTypeAssociation Association;

	FInitializeConstantDoubleChannelMutation(const FDoubleChannelTypeAssociation& InAssociation)
		: Association(InAssociation)
	{}

	virtual void MarkAllocation(FEntityAllocation* Allocation, TBitArray<>& OutEntitiesToMutate) const
	{
		TComponentReader<FSourceDoubleChannel> Channels = Allocation->ReadComponents(Association.ChannelType);

		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			if (Channels[Index].Source->GetTimes().Num() <= 1)
			{
				OutEntitiesToMutate.PadToNum(Index + 1, false);
				OutEntitiesToMutate[Index] = true;
			}
		}
	}

	virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const
	{
		InOutEntityComponentTypes->Remove(Association.CachedInterpolationType);
	}

	virtual void InitializeEntities(const FEntityRange& EntityRange, const FComponentMask& AllocationType) const
	{
		TComponentReader<FSourceDoubleChannel> Channels = EntityRange.Allocation->ReadComponents(Association.ChannelType);
		TComponentWriter<double>               Results = EntityRange.Allocation->WriteComponents(Association.ResultType, FEntityAllocationWriteContext::NewAllocation());

		for (int32 Index = EntityRange.ComponentStartOffset; Index < EntityRange.ComponentStartOffset + EntityRange.Num; ++Index)
		{
			// Eval time shouldn't even matter if it's constant
			if (!Channels[Index].Source->Evaluate(0, Results[Index]))
			{
				Results[Index] = MIN_dbl;
			}
		}
	}
};

} // namespace UE::MovieScene


UDoubleChannelEvaluatorSystem::UDoubleChannelEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());

		FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
		DefineComponentConsumer(GetClass(), Components->SymbolicTags.CreatesEntities);

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(Components->DoubleChannel); ++Index)
		{
			RegisterChannelType(Components->DoubleChannel[Index], Components->CachedInterpolation[Index], Components->DoubleResult[Index]);
		}
	}
}

void UDoubleChannelEvaluatorSystem::RegisterChannelType(TComponentTypeID<UE::MovieScene::FSourceDoubleChannel> SourceChannelType, TComponentTypeID<UE::MovieScene::Interpolation::FCachedInterpolation> CachedInterpolationType, TComponentTypeID<double> ResultType)
{
	using namespace UE::MovieScene;

	FDoubleChannelTypeAssociation ChannelType;
	ChannelType.ChannelType = SourceChannelType;
	ChannelType.CachedInterpolationType = CachedInterpolationType;
	ChannelType.ResultType  = ResultType;

	DefineComponentProducer(UDoubleChannelEvaluatorSystem::StaticClass(), ResultType);

	GDoubleChannelTypeAssociations.Add(ChannelType);
}

bool UDoubleChannelEvaluatorSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	for (const FDoubleChannelTypeAssociation& ChannelType : GDoubleChannelTypeAssociations)
	{
		if (InLinker->EntityManager.ContainsComponent(ChannelType.ChannelType))
		{
			return true;
		}
	}

	return false;
}

void UDoubleChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	for (const FDoubleChannelTypeAssociation& ChannelType : GDoubleChannelTypeAssociations)
	{
		// Evaluate double channels per instance and write the evaluated value into the output
		FEntityTaskBuilder()
		.Read(ChannelType.ChannelType)
		.Read(BuiltInComponents->EvalTime)
		.Write(ChannelType.CachedInterpolationType)
		.Write(ChannelType.ResultType)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(GET_STATID(MovieSceneEval_EvaluateDoubleChannelTask))
		.Fork_PerEntity<FEvaluateDoubleChannels_Cached>(&Linker->EntityManager, TaskScheduler);
	}
}

void UDoubleChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		// Remove cache components for anything that is constant
		for (const FDoubleChannelTypeAssociation& Association : GDoubleChannelTypeAssociations)
		{
			FInitializeConstantDoubleChannelMutation Mutation{ Association };

			FEntityComponentFilter Filter;
			Filter.All({ Association.ChannelType, Association.ResultType, BuiltInComponents->Tags.NeedsLink });
			Filter.None({ BuiltInComponents->Tags.DontOptimizeConstants });

			Linker->EntityManager.MutateConditional(Filter, Mutation);
		}
	}
	else if (Runner->GetCurrentPhase() == ESystemPhase::Evaluation)
	{
		for (const FDoubleChannelTypeAssociation& ChannelType : GDoubleChannelTypeAssociations)
		{
			// Evaluate double channels per instance and write the evaluated value into the output
			FEntityTaskBuilder()
			.Read(ChannelType.ChannelType)
			.Read(BuiltInComponents->EvalTime)
			.Write(ChannelType.CachedInterpolationType)
			.Write(ChannelType.ResultType)
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.SetStat(GET_STATID(MovieSceneEval_EvaluateDoubleChannelTask))
			.Dispatch_PerEntity<FEvaluateDoubleChannels_Cached>(&Linker->EntityManager, InPrerequisites, &Subsequents);
		}
	}
}
