// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/DoublePerlinNoiseChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieSceneBaseValueEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "Channels/MovieSceneDoublePerlinNoiseChannel.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Math/NumericLimits.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(DoublePerlinNoiseChannelEvaluatorSystem)

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate Double Perlin Noise channels"), MovieSceneEval_EvaluateDoublePerlinNoiseChannelTask, STATGROUP_MovieSceneECS);

namespace UE
{
	namespace MovieScene
	{
		struct FEvaluateDoublePerlinNoiseChannels
		{
			static void ForEachEntity(double EvalSeconds, const FPerlinNoiseParams& PerlinNoiseParams, double& OutResult)
			{
				OutResult = FMovieSceneDoublePerlinNoiseChannel::Evaluate(PerlinNoiseParams, EvalSeconds);
			}
		};
	} // namespace MovieScene
} // namespace UE

UDoublePerlinNoiseChannelEvaluatorSystem::UDoublePerlinNoiseChannelEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	RelevantComponent = TrackComponents->DoublePerlinNoiseChannel;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Allow writing to all the possible channels
		for (int32 i = 0; i < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++i)
		{
			DefineComponentProducer(UDoublePerlinNoiseChannelEvaluatorSystem::StaticClass(), BuiltInComponents->DoubleResult[i]);
		}

		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBaseValueEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieScenePiecewiseDoubleBlenderSystem::StaticClass());
	}
}

void UDoublePerlinNoiseChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	for (int32 i = 0; i < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++i)
	{
		FEntityTaskBuilder()
		.Read(BuiltInComponents->EvalSeconds)
		.Read(TrackComponents->DoublePerlinNoiseChannel)
		.Write(BuiltInComponents->DoubleResult[i])
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(GET_STATID(MovieSceneEval_EvaluateDoublePerlinNoiseChannelTask))
		.Fork_PerEntity<FEvaluateDoublePerlinNoiseChannels>(&Linker->EntityManager, TaskScheduler);
	}
}

void UDoublePerlinNoiseChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		for (int32 i = 0; i < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++i)
		{
			FEntityTaskBuilder()
				.Read(BuiltInComponents->BaseValueEvalSeconds)
				.Read(TrackComponents->DoublePerlinNoiseChannel)
				.Write(BuiltInComponents->BaseDouble[i])
				.FilterAll({ BuiltInComponents->Tags.NeedsLink })
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.RunInline_PerEntity(&Linker->EntityManager, FEvaluateDoublePerlinNoiseChannels());
		}
	}
	else if (Runner->GetCurrentPhase() == ESystemPhase::Evaluation)
	{
		for (int32 i = 0; i < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++i)
		{
			FEntityTaskBuilder()
				.Read(BuiltInComponents->EvalSeconds)
				.Read(TrackComponents->DoublePerlinNoiseChannel)
				.Write(BuiltInComponents->DoubleResult[i])
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.SetStat(GET_STATID(MovieSceneEval_EvaluateDoublePerlinNoiseChannelTask))
				.Dispatch_PerEntity<FEvaluateDoublePerlinNoiseChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents);
		}
	}
}

