// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/FloatChannelEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"

#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneInterpolation.h"
#include "MovieSceneTracksComponentTypes.h"

#include "Algo/Find.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloatChannelEvaluatorSystem)

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate float channels"), MovieSceneEval_EvaluateFloatChannelTask, STATGROUP_MovieSceneECS);

namespace UE::MovieScene
{

struct FFloatChannelTypeAssociation
{
	TComponentTypeID<FSourceFloatChannel> ChannelType;
	TComponentTypeID<Interpolation::FCachedInterpolation> CachedInterpolationType;
	TComponentTypeID<double> ResultType;
};

TArray<FFloatChannelTypeAssociation, TInlineAllocator<4>> GFloatChannelTypeAssociations;

// @todo: for multi-bindings we currently re-evaluate the float channel for each binding, even though the time is the same.
// Do we need to optimize for this case using something like the code below, while pessimizing the common (non-multi-bind) codepath??

/** Entity-component task that evaluates using a cached interpolation if possible */
struct FEvaluateFloatChannels_Cached
{
	static void ForEachEntity(FSourceFloatChannel FloatChannel, FFrameTime FrameTime, Interpolation::FCachedInterpolation& Cache, double& OutResult)
	{
		if (!Cache.IsCacheValidForTime(FrameTime.GetFrame()))
		{
			Cache = FloatChannel.Source->GetInterpolationForTime(FrameTime);
		}

		if (!Cache.Evaluate(FrameTime, OutResult))
		{
			OutResult = MIN_dbl;
		}
	}
};

/**
 * Mutation that removes FCachedInterpolation components from any channels that have a constant value
 * This effectively prevents the channel from ever being updated again
 */
struct FInitializeConstantFloatChannelMutation : IMovieSceneConditionalEntityMutation
{
	FFloatChannelTypeAssociation Association;

	FInitializeConstantFloatChannelMutation(const FFloatChannelTypeAssociation& InAssociation)
		: Association(InAssociation)
	{}

	virtual void MarkAllocation(FEntityAllocation* Allocation, TBitArray<>& OutEntitiesToMutate) const
	{
		TComponentReader<FSourceFloatChannel> Channels = Allocation->ReadComponents(Association.ChannelType);

		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			if (Channels[Index].Source->GetTimes().Num() <= 1)
			{
				OutEntitiesToMutate.PadToNum(Index + 1, false);
				OutEntitiesToMutate[Index] = true;
			}
		}
	}

	virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const
	{
		InOutEntityComponentTypes->Remove(Association.CachedInterpolationType);
	}

	virtual void InitializeEntities(const FEntityRange& EntityRange, const FComponentMask& AllocationType) const
	{
		TComponentReader<FSourceFloatChannel> Channels = EntityRange.Allocation->ReadComponents(Association.ChannelType);
		TComponentWriter<double>               Results = EntityRange.Allocation->WriteComponents(Association.ResultType, FEntityAllocationWriteContext::NewAllocation());

		for (int32 Index = EntityRange.ComponentStartOffset; Index < EntityRange.ComponentStartOffset + EntityRange.Num; ++Index)
		{
			// Eval time shouldn't even matter if it's constant
			float Result;
			if (!Channels[Index].Source->Evaluate(0, Result))
			{
				Results[Index] = MIN_dbl;
			}
			else
			{
				Results[Index] = Result;
			}
		}
	}
};

} // namespace UE::MovieScene


UFloatChannelEvaluatorSystem::UFloatChannelEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());

		FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
		DefineComponentConsumer(GetClass(), Components->SymbolicTags.CreatesEntities);

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(Components->FloatChannel); ++Index)
		{
			RegisterChannelType(Components->FloatChannel[Index], Components->CachedInterpolation[Index], Components->DoubleResult[Index]);
		}
		RegisterChannelType(Components->WeightChannel, Components->CachedWeightChannelInterpolation, Components->WeightResult);
	}
}

void UFloatChannelEvaluatorSystem::RegisterChannelType(TComponentTypeID<UE::MovieScene::FSourceFloatChannel> SourceChannelType, TComponentTypeID<UE::MovieScene::Interpolation::FCachedInterpolation> CachedInterpolationType, TComponentTypeID<double> ResultType)
{
	using namespace UE::MovieScene;

	FFloatChannelTypeAssociation ChannelType;
	ChannelType.ChannelType = SourceChannelType;
	ChannelType.CachedInterpolationType = CachedInterpolationType;
	ChannelType.ResultType  = ResultType;

	DefineComponentProducer(UFloatChannelEvaluatorSystem::StaticClass(), ResultType);

	GFloatChannelTypeAssociations.Add(ChannelType);
}

bool UFloatChannelEvaluatorSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	for (const FFloatChannelTypeAssociation& ChannelType : GFloatChannelTypeAssociations)
	{
		if (InLinker->EntityManager.ContainsComponent(ChannelType.ChannelType))
		{
			return true;
		}
	}

	return false;
}

void UFloatChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	for (const FFloatChannelTypeAssociation& ChannelType : GFloatChannelTypeAssociations)
	{
		// Evaluate float channels per instance and write the evaluated value into the output
		FEntityTaskBuilder()
		.Read(ChannelType.ChannelType)
		.Read(BuiltInComponents->EvalTime)
		.Write(ChannelType.CachedInterpolationType)
		.Write(ChannelType.ResultType)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(GET_STATID(MovieSceneEval_EvaluateFloatChannelTask))
		.Fork_PerEntity<FEvaluateFloatChannels_Cached>(&Linker->EntityManager, TaskScheduler);
	}
}

void UFloatChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		// Remove cache components for anything that is constant
		for (const FFloatChannelTypeAssociation& Association : GFloatChannelTypeAssociations)
		{
			FInitializeConstantFloatChannelMutation Mutation{ Association };

			FEntityComponentFilter Filter;
			Filter.All({ Association.ChannelType, Association.ResultType, BuiltInComponents->Tags.NeedsLink });
			Filter.None({ BuiltInComponents->Tags.DontOptimizeConstants });

			Linker->EntityManager.MutateConditional(Filter, Mutation);
		}
	}
	else if (Runner->GetCurrentPhase() == ESystemPhase::Evaluation)
	{
		for (const FFloatChannelTypeAssociation& ChannelType : GFloatChannelTypeAssociations)
		{
			// Evaluate float channels per instance and write the evaluated value into the output
			FEntityTaskBuilder()
			.Read(ChannelType.ChannelType)
			.Read(BuiltInComponents->EvalTime)
			.Write(ChannelType.CachedInterpolationType)
			.Write(ChannelType.ResultType)
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.SetStat(GET_STATID(MovieSceneEval_EvaluateFloatChannelTask))
			.Dispatch_PerEntity<FEvaluateFloatChannels_Cached>(&Linker->EntityManager, InPrerequisites, &Subsequents);
		}
	}
}
