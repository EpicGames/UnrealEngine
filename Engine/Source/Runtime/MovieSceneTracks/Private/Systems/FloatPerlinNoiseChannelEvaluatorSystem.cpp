// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/FloatPerlinNoiseChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieSceneBaseValueEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "Channels/MovieSceneFloatPerlinNoiseChannel.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Math/NumericLimits.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(FloatPerlinNoiseChannelEvaluatorSystem)

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate Float Perlin Noise channels"), MovieSceneEval_EvaluateFloatPerlinNoiseChannelTask, STATGROUP_MovieSceneECS);

namespace UE
{
	namespace MovieScene
	{
		struct FEvaluateFloatPerlinNoiseChannels
		{
			static void ForEachEntity(double EvalSeconds, const FPerlinNoiseParams& PerlinNoiseParams, double& OutResult)
			{
				const float NoiseResult = FMovieSceneFloatPerlinNoiseChannel::Evaluate(PerlinNoiseParams, EvalSeconds);
				OutResult = (double)NoiseResult;
			}
		};
	} // namespace MovieScene
} // namespace UE

UFloatPerlinNoiseChannelEvaluatorSystem::UFloatPerlinNoiseChannelEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	RelevantComponent = TrackComponents->FloatPerlinNoiseChannel;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Allow writing to all the possible channels
		for (int32 i = 0; i < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++i)
		{
			DefineComponentProducer(UFloatPerlinNoiseChannelEvaluatorSystem::StaticClass(), BuiltInComponents->DoubleResult[i]);
		}

		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBaseValueEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieScenePiecewiseDoubleBlenderSystem::StaticClass());
	}
}

void UFloatPerlinNoiseChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	for (int32 i = 0; i < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++i)
	{
		FEntityTaskBuilder()
		.Read(BuiltInComponents->EvalSeconds)
		.Read(TrackComponents->FloatPerlinNoiseChannel)
		.Write(BuiltInComponents->DoubleResult[i])
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(GET_STATID(MovieSceneEval_EvaluateFloatPerlinNoiseChannelTask))
		.Fork_PerEntity<FEvaluateFloatPerlinNoiseChannels>(&Linker->EntityManager, TaskScheduler);
	}
}

void UFloatPerlinNoiseChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		for (int32 i = 0; i < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++i)
		{
			FEntityTaskBuilder()
				.Read(BuiltInComponents->BaseValueEvalSeconds)
				.Read(TrackComponents->FloatPerlinNoiseChannel)
				.Write(BuiltInComponents->BaseDouble[i])
				.FilterAll({ BuiltInComponents->Tags.NeedsLink })
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.RunInline_PerEntity(&Linker->EntityManager, FEvaluateFloatPerlinNoiseChannels());
		}
	}
	else if (Runner->GetCurrentPhase() == ESystemPhase::Evaluation)
	{
		for (int32 i = 0; i < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++i)
		{
			FEntityTaskBuilder()
				.Read(BuiltInComponents->EvalSeconds)
				.Read(TrackComponents->FloatPerlinNoiseChannel)
				.Write(BuiltInComponents->DoubleResult[i])
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.SetStat(GET_STATID(MovieSceneEval_EvaluateFloatPerlinNoiseChannelTask))
				.Dispatch_PerEntity<FEvaluateFloatPerlinNoiseChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents);
		}
	}
}

