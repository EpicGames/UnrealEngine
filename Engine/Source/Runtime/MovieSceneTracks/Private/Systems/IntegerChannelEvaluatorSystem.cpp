// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/IntegerChannelEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Math/NumericLimits.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(IntegerChannelEvaluatorSystem)

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate integer channels"), MovieSceneEval_EvaluateIntegerChannelTask, STATGROUP_MovieSceneECS);

namespace UE
{
namespace MovieScene
{

// @todo: for multi-bindings we currently re-evaluate the integer channel for each binding, even though the time is the same.
// Do we need to optimize for this case using something like the code below, while pessimizing the common (non-multi-bind) codepath??
struct FEvaluateIntegerChannels
{
	static void ForEachEntity(FSourceIntegerChannel IntegerChannel, FFrameTime FrameTime, int64& OutResult)
	{
		if (!IntegerChannel.Source->Evaluate(FrameTime, OutResult))
		{
			OutResult = MIN_int64;
		}
	}
};


} // namespace MovieScene
} // namespace UE


UIntegerChannelEvaluatorSystem::UIntegerChannelEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	RelevantComponent = BuiltInComponents->IntegerChannel;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentProducer(UIntegerChannelEvaluatorSystem::StaticClass(), BuiltInComponents->IntegerResult);

		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
	}
}

void UIntegerChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// Evaluate integer channels per instance and write the evaluated value into the output
	FEntityTaskBuilder()
	.Read(BuiltInComponents->IntegerChannel)
	.Read(BuiltInComponents->EvalTime)
	.Write(BuiltInComponents->IntegerResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateIntegerChannelTask))
	.Fork_PerEntity<FEvaluateIntegerChannels>(&Linker->EntityManager, TaskScheduler);
}

void UIntegerChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// Evaluate integer channels per instance and write the evaluated value into the output
	FEntityTaskBuilder()
	.Read(BuiltInComponents->IntegerChannel)
	.Read(BuiltInComponents->EvalTime)
	.Write(BuiltInComponents->IntegerResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateIntegerChannelTask))
	.Dispatch_PerEntity<FEvaluateIntegerChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents);
}


