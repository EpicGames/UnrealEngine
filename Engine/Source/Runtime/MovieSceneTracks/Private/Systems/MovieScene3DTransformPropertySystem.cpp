// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieScene3DTransformPropertySystem.h"

#include "Systems/MovieScenePropertyInstantiator.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DTransformPropertySystem)

UMovieScene3DTransformPropertySystem::UMovieScene3DTransformPropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Transform);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());
	}
}

void UMovieScene3DTransformPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}

