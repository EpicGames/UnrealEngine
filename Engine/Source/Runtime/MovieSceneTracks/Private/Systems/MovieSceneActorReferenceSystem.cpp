// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneActorReferencePropertySystem.h"
#include "Systems/ActorReferenceChannelEvaluatorSystem.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneActorReferencePropertySystem)

UMovieSceneActorReferencePropertySystem::UMovieSceneActorReferencePropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->ActorReference);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UActorReferenceChannelEvaluatorSystem::StaticClass(), GetClass());

		DefineComponentConsumer(GetClass(), UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->ActorReference.PropertyTag);
	}
}

void UMovieSceneActorReferencePropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}

