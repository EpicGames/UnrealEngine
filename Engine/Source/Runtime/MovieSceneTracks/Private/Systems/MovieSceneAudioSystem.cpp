// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneAudioSystem.h"

#include "AudioDevice.h"
#include "Components/AudioComponent.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "GameFramework/WorldSettings.h"
#include "IMovieScenePlayer.h"
#include "MovieScene.h"
#include "Decorations/MovieSceneScalingAnchors.h"
#include "Decorations/MovieSceneSectionAnchorsDecoration.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Sound/SoundAttenuation.h"
#include "Kismet/GameplayStatics.h"
#include "Sound/SoundCue.h"
#include "Sound/SoundNodeWavePlayer.h"
#include "Sound/SoundWave.h"
#include "Tracks/MovieSceneAudioTrack.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAudioSystem)

DECLARE_CYCLE_STAT(TEXT("Audio System Evaluate"), MovieSceneEval_AudioTasks, STATGROUP_MovieSceneECS);

static bool bPrimeStreamingDuringPrerollCVar = true;
FAutoConsoleVariableRef CVarPrimeStreamingDuringPreroll(
	TEXT("Sequencer.Audio.PrimeStreamingDuringPreroll"),
	bPrimeStreamingDuringPrerollCVar,
	TEXT("When enabled, streaming sound chunks are requested during Sequencer preroll so they are resident when playback reaches the section start.\n"),
	ECVF_Default);

static float MaxSequenceAudioDesyncToleranceCVar = 0.5f;
FAutoConsoleVariableRef CVarMaxSequenceAudioDesyncTolerance(
	TEXT("Sequencer.Audio.MaxDesyncTolerance"),
	MaxSequenceAudioDesyncToleranceCVar,
	TEXT("Controls how many seconds an audio track can be out of sync in a Sequence before we attempt a time correction.\n"),
	ECVF_Default);

static bool bIgnoreAudioSyncDuringWorldTimeDilationCVar = true;
FAutoConsoleVariableRef CVarIgnoreAudioSyncDuringWorldTimeDilation(
	TEXT("Sequencer.Audio.IgnoreAudioSyncDuringWorldTimeDilation"),
	bIgnoreAudioSyncDuringWorldTimeDilationCVar,
	TEXT("Ignore correcting audio if there is world time dilation.\n"),
	ECVF_Default);

static int32 UseAudioClockForSequencerDesyncCVar = 0;
FAutoConsoleVariableRef CVaUseAudioClockForSequencerDesync(
	TEXT("Sequencer.Audio.UseAudioClockForAudioDesync"),
	UseAudioClockForSequencerDesyncCVar,
	TEXT("When set to 1, we will use the audio render thread directly to query whether audio has went out of sync with the sequence.\n"),
	ECVF_Default);

static bool bPlayAudioWhenPlaybackJumps = false;
FAutoConsoleVariableRef CVarPlayAudioWhenPlaybackJumps(
	TEXT("Sequencer.Audio.PlayAudioWhenPlaybackJumps"),
	bPlayAudioWhenPlaybackJumps,
	TEXT("Play audio when playback jumps.\n"),
	ECVF_Default);

static bool bUseTimeDilationToAdjustPlayDurationCVar = true;
FAutoConsoleVariableRef CVarUseTimeDilationToAdjustPlayDuration(
	TEXT("Sequencer.Audio.UseTimeDilationToAdjustPlayDuration"),
	bUseTimeDilationToAdjustPlayDurationCVar,
	TEXT("Use the effective time dilation to scale the current time of audio.\n"),
	ECVF_Default);


static int32 ScrubWidthMillisecondsCVar = 80;
FAutoConsoleVariableRef CVarScrubWidthMilliseconds(
	TEXT("Sequencer.Audio.ScrubWidthMilliseconds"),
	ScrubWidthMillisecondsCVar,
	TEXT("The time-width of grains (in milliseconds) while scrubbing an audio track.\n"),
	ECVF_Default);


static bool bEnableGranularScrubbingCVar = true;
FAutoConsoleVariableRef CVarEnableGranularScrubbing(
	TEXT("Sequencer.Audio.EnableGranularScrubbing"),
	bEnableGranularScrubbingCVar,
	TEXT("Whether or not to use granular scrubbing.\n"),
	ECVF_Default);

static bool bEnableGranularScrubbingWhileStationaryCVar = true;
FAutoConsoleVariableRef CVarEnableGranularScrubbingWhileStationary(
	TEXT("Sequencer.Audio.EnableGranularScrubbingWhileStationary"),
	bEnableGranularScrubbingWhileStationaryCVar,
	TEXT("Whether or not to use granular scrubbing while holding the playhead still.\n"),
	ECVF_Default);

namespace UE::MovieScene
{

static bool IsSeekingStatus(EMovieScenePlayerStatus::Type Status)
{
    return Status == EMovieScenePlayerStatus::Scrubbing
        || Status == EMovieScenePlayerStatus::Stepping
        || Status == EMovieScenePlayerStatus::Jumping;
};

enum class EPreAnimatedAudioStateType
{
	/** Pre-animated state manages the lifespan of the audio component */
	AudioComponentLifespan,
	/** Pre-animated state manages the lifespan of the audio component.
	* When MRQ is active, it will process UE time outside of recording to let secondary animations settle etc.
	* For this case only, if the section extends beyond the current camera cut, the lifespan of the AudioComponent also spans the camera cut.
	* This ensures audio capture has no gaps between shots as pause/resume (rather than stop/play) enables sample-accurate continuity across camera cuts.
	* On capturing the next camera, playback will resume using the same AudioComponent.
	**/
	AudioComponentLifespanBeyondCurrentCameraCut,
	/** Pre-animated state manages whether the audio component is playing */
	AudioPlaying
};

template<typename BaseTraits>
struct FPreAnimatedAudioStateTraits : BaseTraits
{
	using KeyType = FObjectKey;
	using StorageType = EPreAnimatedAudioStateType;

	EPreAnimatedAudioStateType CachePreAnimatedValue(FObjectKey InKey)
	{
		check(false);
		return EPreAnimatedAudioStateType::AudioComponentLifespan;
	}

	void RestorePreAnimatedValue(FObjectKey InKey, EPreAnimatedAudioStateType InStateType, const FRestoreStateParams& Params)
	{
		if (UAudioComponent* AudioComponent = Cast<UAudioComponent>(InKey.ResolveObjectPtr()))
		{
			check( InStateType == EPreAnimatedAudioStateType::AudioPlaying 
				|| InStateType == EPreAnimatedAudioStateType::AudioComponentLifespan
				|| InStateType == EPreAnimatedAudioStateType::AudioComponentLifespanBeyondCurrentCameraCut );
			switch (InStateType)
			{
				case EPreAnimatedAudioStateType::AudioComponentLifespanBeyondCurrentCameraCut:
					AudioComponent->SetPaused(true);
					break;
				case EPreAnimatedAudioStateType::AudioPlaying:
					AudioComponent->Stop();
					break;
				case EPreAnimatedAudioStateType::AudioComponentLifespan:
					AudioComponent->DestroyComponent();
					// NOTE: if not destroyed here, it will be via ULevel::ClearLevelComponents.
					break;
			}			
		}
	}
};

using FPreAnimatedBoundObjectAudioStateTraits = FPreAnimatedAudioStateTraits<FBoundObjectPreAnimatedStateTraits>;

struct FPreAnimatedAudioStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedBoundObjectAudioStateTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedAudioStorage> StorageID;
};
TAutoRegisterPreAnimatedStorageID<FPreAnimatedAudioStorage> FPreAnimatedAudioStorage::StorageID;

/**
 * Types of audio evaluation we should run for a given sequence.
 */
enum class EAudioEvaluationType
{
	Skip,
	Play,
	StopAndPlay,
	Stop,
	Prime
};

struct FGatherAudioInputs
{
	using FInstanceObjectKey = UMovieSceneAudioSystem::FInstanceObjectKey;
	using FAudioInputsBySectionKey = UMovieSceneAudioSystem::FAudioInputsBySectionKey;
	using FAudioComponentInputEvaluationData = UMovieSceneAudioSystem::FAudioComponentInputEvaluationData;

	UMovieSceneAudioSystem* AudioSystem;

	FGatherAudioInputs(UMovieSceneAudioSystem* InAudioSystem)
		: AudioSystem(InAudioSystem)
	{
	}

	void ForEachAllocation(
		const FEntityAllocation* Allocation,
		TRead<FInstanceHandle> InstanceHandles,
		TRead<FMovieSceneAudioComponentData> AudioDatas,
		TRead<FMovieSceneAudioInputData> AudioInputDatas,
		TReadOneOrMoreOf<
				double, double, double,
				double, double, double,
				double, double, double,
				FString, int64, bool>
			AudioInputResults) const
	{
		FAudioInputsBySectionKey& AudioInputsBySectionKey = AudioSystem->AudioInputsBySectionKey;

		const double* DoubleResults[FMovieSceneAudioInputData::NumFloatChannels];
		{
			DoubleResults[0] = AudioInputResults.Get<0>();
			DoubleResults[1] = AudioInputResults.Get<1>();
			DoubleResults[2] = AudioInputResults.Get<2>();
			DoubleResults[3] = AudioInputResults.Get<3>();
			DoubleResults[4] = AudioInputResults.Get<4>();
			DoubleResults[5] = AudioInputResults.Get<5>();
			DoubleResults[6] = AudioInputResults.Get<6>();
			DoubleResults[7] = AudioInputResults.Get<7>();
			DoubleResults[8] = AudioInputResults.Get<8>();
		}
		const FString* StringResults = AudioInputResults.Get<9>();
		const int64* IntegerResults = AudioInputResults.Get<10>();
		const bool* BoolResults = AudioInputResults.Get<11>();

		for (int32 Index = 0; Index < Allocation->Num(); ++Index)
		{
			const FMovieSceneAudioComponentData& AudioData = AudioDatas[Index];
			const FMovieSceneAudioInputData& AudioInputNames = AudioInputDatas[Index];

			FInstanceObjectKey SectionKey(InstanceHandles[Index], FObjectKey(AudioData.Section));
			FAudioComponentInputEvaluationData& AudioInputValues = AudioInputsBySectionKey.FindOrAdd(SectionKey);
			
			// Gather float inputs.
			for (int32 FloatIndex = 0; FloatIndex < FMovieSceneAudioInputData::NumFloatChannels; ++FloatIndex)
			{
				if (!AudioInputNames.FloatInputs[FloatIndex].IsNone() && ensure(DoubleResults[FloatIndex]))
				{
					AudioInputValues.Inputs_Float.Add(AudioInputNames.FloatInputs[FloatIndex], DoubleResults[FloatIndex][Index]);
				}
			}

			// Gather string inputs.
			if (!AudioInputNames.StringInput.IsNone() && ensure(StringResults))
			{
				AudioInputValues.Inputs_String.Add(AudioInputNames.StringInput, StringResults[Index]);
			}

			// Gather integer inputs.
			if (!AudioInputNames.IntInput.IsNone() && ensure(IntegerResults))
			{
				AudioInputValues.Inputs_Int.Add(AudioInputNames.IntInput, IntegerResults[Index]);
			}

			// Gather boolean inputs.
			if (!AudioInputNames.BoolInput.IsNone() && ensure(BoolResults))
			{
				AudioInputValues.Inputs_Bool.Add(AudioInputNames.BoolInput, BoolResults[Index]);
			}
		}
	}
};

struct FGatherAudioTriggers
{
	using FInstanceObjectKey = UMovieSceneAudioSystem::FInstanceObjectKey;
	using FAudioInputsBySectionKey = UMovieSceneAudioSystem::FAudioInputsBySectionKey;
	using FAudioComponentInputEvaluationData = UMovieSceneAudioSystem::FAudioComponentInputEvaluationData;

	UMovieSceneAudioSystem* AudioSystem;

	FGatherAudioTriggers(UMovieSceneAudioSystem* InAudioSystem)
		: AudioSystem(InAudioSystem)
	{
	}

	void ForEachAllocation(
		const FEntityAllocation* Allocation,
		TRead<FInstanceHandle> InstanceHandles,
		TRead<FMovieSceneAudioComponentData> AudioDatas,
		TRead<FName> AudioTriggerNames) const
	{
		FAudioInputsBySectionKey& AudioInputsBySectionKey = AudioSystem->AudioInputsBySectionKey;

		for (int32 Index = 0; Index < Allocation->Num(); ++Index)
		{
			const FMovieSceneAudioComponentData& AudioData = AudioDatas[Index];
			const FName& AudioTriggerName = AudioTriggerNames[Index];

			FInstanceObjectKey SectionKey(InstanceHandles[Index], FObjectKey(AudioData.Section));
			FAudioComponentInputEvaluationData& AudioInputValues = AudioInputsBySectionKey.FindOrAdd(SectionKey);

			AudioInputValues.Inputs_Trigger.Add(AudioTriggerName);
		}
	}
};

struct FEvaluateAudio
{
	static EAudioEvaluationType GetAudioEvaluationType(const FMovieSceneContext& Context, bool bEntityIsInPreRoll)
	{
		// During preroll (sub-sequence or section-level), request streaming chunks instead of playing.
		if (bPrimeStreamingDuringPrerollCVar && (Context.IsPreRoll() || bEntityIsInPreRoll))
		{
			return EAudioEvaluationType::Prime;
		}

		if (Context.GetStatus() == EMovieScenePlayerStatus::Jumping &&
				!bPlayAudioWhenPlaybackJumps)
		{
			return EAudioEvaluationType::Skip;
		}

		const EMovieScenePlayerStatus::Type Status = Context.GetStatus();
		const bool bIsPlayingForward = Status == EMovieScenePlayerStatus::Playing && Context.GetDirection() == EPlayDirection::Forwards;
		
		if (!bIsPlayingForward && !IsSeekingStatus(Status))
		{
			// stopped, recording, etc
			return EAudioEvaluationType::Stop;
		}

		if (Context.HasJumped())
		{
			// If the status says we jumped, we always stop all sounds, then allow them to be played again 
			// naturally if status == Playing (for example)
			return EAudioEvaluationType::StopAndPlay;
		}

		return EAudioEvaluationType::Play;
	}

	UMovieSceneAudioSystem* AudioSystem;
	const FInstanceRegistry* InstanceRegistry;

	FEvaluateAudio(UMovieSceneAudioSystem* InAudioSystem)
		: AudioSystem(InAudioSystem)
	{
		InstanceRegistry = AudioSystem->GetLinker()->GetInstanceRegistry();
	}

	void ForEachAllocation(
			const FEntityAllocation* Allocation, 
			TRead<FMovieSceneEntityID> EntityIDs, 
			TRead<FRootInstanceHandle> RootInstanceHandles,
			TRead<FInstanceHandle> InstanceHandles,
			TRead<FMovieSceneAudioComponentData> AudioDatas,
			TRead<double> VolumeMultipliers,
			TRead<double> PitchMultipliers,
			TReadOptional<UObject*> BoundObjects) const
	{
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		check(Allocation != nullptr);
		const int32 Num = Allocation->Num();
		const bool bWantsRestoreState = Allocation->HasComponent(BuiltInComponents->Tags.RestoreState);

		// All entities in this allocation share the same component set, so one check covers them all.
		const bool bIsPreRollAllocation = Allocation->HasComponent(BuiltInComponents->Tags.PreRoll);

		for (int32 Index = 0; Index < Num; ++Index)
		{
			const FMovieSceneEntityID& EntityID = EntityIDs[Index];
			const FRootInstanceHandle& RootInstanceHandle = RootInstanceHandles[Index];
			const FInstanceHandle& InstanceHandle = InstanceHandles[Index];
			const FMovieSceneAudioComponentData& AudioData = AudioDatas[Index];

			const FSequenceInstance& Instance = InstanceRegistry->GetInstance(InstanceHandle);

			double VolumeMultiplier = VolumeMultipliers[Index];
			double PitchMultiplier = PitchMultipliers[Index];
			UObject* BoundObject = (BoundObjects.IsValid() ? BoundObjects[Index] : nullptr);

			Evaluate(EntityID, AudioData, Instance, RootInstanceHandle, VolumeMultiplier, PitchMultiplier, BoundObject, bWantsRestoreState, bIsPreRollAllocation);
		}
	}

private:
	void UpdatePreAnimatedStorage(	const FMovieSceneEntityID& EntityID,
									const FSequenceInstance& Instance,
									const FRootInstanceHandle& RootInstanceHandle,
									bool bWantsRestoreState,
									const FMovieSceneContext& Context,
									FAudioComponentEvaluationData* EvaluationData,
									UMovieSceneAudioSection* AudioSection,
									bool bRootMovieSceneIsRecordingAudio,
									UAudioComponent* AudioComponent ) const
	{
		check(EvaluationData != nullptr);
		check(AudioSection != nullptr);
		check(AudioComponent != nullptr);

		// When bRootMovieSceneIsRecordingAudio (e.g. MRQ is active) and the audio section doesn't end in this cinematic shot or camera cut,
		// use EPreAnimatedAudioStateType::AudioComponentLifespanBeyondCurrentCameraCut.
		// This is applied on the first frame of each cinematic shot and camera cut, flagged by setting EvaluationData->AudioComponentLifespanControlSection
		// to the current cinematic shot or camera cut.
		// When not bRootMovieSceneIsRecordingAudio, EvaluationData->AudioComponentLifespanControlSection is set to the AudioSection itself,
		// and is thus applied once for the life of the audio component / evaluation data.
		UMovieSceneSection* CurrentLifespanControlSection = AudioSection;
		EPreAnimatedAudioStateType LifespanType = EPreAnimatedAudioStateType::AudioComponentLifespan;
		if (EvaluationData->AudioComponentLifespanControlSection != AudioSection && AudioSection->HasEndFrame() && bRootMovieSceneIsRecordingAudio)
		{
			check(Instance.GetSharedPlaybackState()->GetRootSequence() != nullptr);
			const UMovieScene* RootMovieScene = Instance.GetSharedPlaybackState()->GetRootSequence()->GetMovieScene();
			const UMovieSceneTrack* CameraCutTrack = RootMovieScene->GetCameraCutTrack();
			const UMovieSceneTrack* MovieSceneCinematicShotTrack = RootMovieScene->FindTrack<UMovieSceneCinematicShotTrack>();
			if (CameraCutTrack || MovieSceneCinematicShotTrack)
			{
				const FFrameNumber CurrentFrame = Context.GetTime().FrameNumber;
				const FFrameNumber AudioSectionEndFrame = AudioSection->GetExclusiveEndFrame();
				if (MovieSceneCinematicShotTrack)
				{
					for (UMovieSceneSection* MovieSceneCinematicShotSection : MovieSceneCinematicShotTrack->GetAllSections())
					{
						if (MovieSceneCinematicShotSection && MovieSceneCinematicShotSection->GetRange().Contains(CurrentFrame)) // determine current cinematic shot
						{
							CurrentLifespanControlSection = MovieSceneCinematicShotSection;
							if (!MovieSceneCinematicShotSection->GetRange().Contains(AudioSectionEndFrame))
							{
								LifespanType = EPreAnimatedAudioStateType::AudioComponentLifespanBeyondCurrentCameraCut;
							}
							break;
						}
					}
				}
				// If no cinematic section contains the audio section's end frame, check if a camera cut section contains it instead.
				// If either contains the audio section's end frame, use EPreAnimatedAudioStateType::AudioComponentLifespanBeyondCurrentCameraCut.
				if (CameraCutTrack && LifespanType != EPreAnimatedAudioStateType::AudioComponentLifespanBeyondCurrentCameraCut)
				{
					for (UMovieSceneSection* CameraCutSection : CameraCutTrack->GetAllSections())
					{
						if (CameraCutSection && CameraCutSection->GetRange().Contains(CurrentFrame)) // determine current camera cut
						{
							CurrentLifespanControlSection = CameraCutSection;
							if (!CameraCutSection->GetRange().Contains(AudioSectionEndFrame))
							{
								LifespanType = EPreAnimatedAudioStateType::AudioComponentLifespanBeyondCurrentCameraCut;
							}
							break;
						}
					}
				}
			}
		}

		// On IsSeekingStatus transition, invalidate every tracked component's lifespan cache
		// so the guard below re-fires BeginTrackingEntity with the current effective bWantsRestoreState.
		// This is what makes the scrubbing override below non-sticky: when the user releases scrub on a
		// KeepState section, the next eval's BeginTrackingEntity flips the entry's flag back so KeepState audio
		// survives natural teardown.
		const EMovieScenePlayerStatus::Type CurrentStatus = Context.GetStatus();

		const bool bScrubStateTransitioned = IsSeekingStatus(EvaluationData->LastSeenStatus) != IsSeekingStatus(CurrentStatus);

		EvaluationData->LastSeenStatus = CurrentStatus;

		if (bScrubStateTransitioned)
		{
			AudioSystem->InvalidateLifespanTracking();
		}

		if (EvaluationData->AudioComponentLifespanControlSection != CurrentLifespanControlSection)
		{
			if (LifespanType == EPreAnimatedAudioStateType::AudioComponentLifespanBeyondCurrentCameraCut)
			{
				ensure(CurrentLifespanControlSection != AudioSection); // AudioComponentLifespanBeyondCurrentCameraCut can only be used when the CurrentLifespanControlSection is a camera cut.
			}
			EvaluationData->AudioComponentLifespanControlSection = CurrentLifespanControlSection;

			// Force RestoreState while IsSeekingStatus so the section dropping out of the eval field
			// (e.g. user scrubs past the section's end) destroys the audio component via the
			// RestoreState entity-teardown path. The transition-driven invalidation above keeps 
			// this override from being sticky once scrubbing ends.
			const bool bEffectiveWantsRestoreState = bWantsRestoreState || IsSeekingStatus(CurrentStatus);

			AudioSystem->PreAnimatedStorage->BeginTrackingEntity(EntityID, bEffectiveWantsRestoreState, RootInstanceHandle, AudioComponent);
			AudioSystem->PreAnimatedStorage->CachePreAnimatedValue(
				FCachePreAnimatedValueParams(), AudioComponent,
				[LifespanType](FObjectKey InKey) { return LifespanType; });
		}
	}

	void Evaluate(
			const FMovieSceneEntityID& EntityID,
			const FMovieSceneAudioComponentData& AudioData,
			const FSequenceInstance& Instance,
			const FRootInstanceHandle& RootInstanceHandle,
			double VolumeMultiplier,
			double PitchMultiplier,
			UObject* BoundObject,
			bool bWantsRestoreState,
			bool bIsPreRollAllocation) const
	{
		const FMovieSceneContext& Context = Instance.GetContext();
		UObject* PlaybackContext = Instance.GetSharedPlaybackState()->GetPlaybackContext();

		UMovieSceneAudioSection* AudioSection = AudioData.Section;
		if (!ensureMsgf(AudioSection, TEXT("No valid audio section found in audio track component data!")))
		{
			return;
		}

		FInstanceHandle InstanceHandle(Instance.GetInstanceHandle());
		FObjectKey ActorKey(BoundObject);
		FObjectKey SectionKey(AudioSection);

		const EAudioEvaluationType EvalType = GetAudioEvaluationType(Context, bIsPreRollAllocation);

		bool bRootMovieSceneIsRecordingAudio = false;
		if (FMovieSceneAudioRecordingCapability* AudioRecordingCapability = Instance.GetSharedPlaybackState()->FindCapability<FMovieSceneAudioRecordingCapability>()) // AudioRecordingCapability will be null if MRQ isn't active, leaving bRootMovieSceneIsRecordingAudio false
		{
			bRootMovieSceneIsRecordingAudio = AudioRecordingCapability->bIsRecordingAudio;
		}

		if (EvalType == EAudioEvaluationType::StopAndPlay)
		{
			AudioSystem->StopSound(InstanceHandle, ActorKey, AudioData.Section, bRootMovieSceneIsRecordingAudio);
		}
		else if (EvalType == EAudioEvaluationType::Stop)
		{
			AudioSystem->StopSound(InstanceHandle, ActorKey, AudioData.Section, bRootMovieSceneIsRecordingAudio);
			return;
		}
		else if (EvalType == EAudioEvaluationType::Skip)
		{
			return;
		}
		else if (EvalType == EAudioEvaluationType::Prime)
		{
			if (USoundBase* Sound = AudioSection->GetPlaybackSound())
			{
				UGameplayStatics::PrimeSound(Sound);
			}
			return;
		}

		// Root audio track
		if (BoundObject == nullptr)
		{
			const FMovieSceneActorReferenceChannel& AttachActorData = AudioSection->GetAttachActorData();

			USceneComponent* AttachComponent = nullptr;
			FMovieSceneActorReferenceKey AttachKey;
			AttachActorData.Evaluate(Context.GetTime(), AttachKey);
			FMovieSceneObjectBindingID AttachBindingID = AttachKey.Object;
			if (AttachBindingID.IsValid())
			{
				// If the transform is set, otherwise use the bound actor's transform
				for (TWeakObjectPtr<> WeakObject : AttachBindingID.ResolveBoundObjects(Instance.GetSequenceID(), Instance.GetSharedPlaybackState()))
				{
					AActor* AttachActor = Cast<AActor>(WeakObject.Get());
					if (AttachActor)
					{
						AttachComponent = AudioSection->GetAttachComponent(AttachActor, AttachKey);
					}
					if (AttachComponent)
					{
						break;
					}
				}
			}

			FAudioComponentEvaluationData* EvaluationData = AudioSystem->GetAudioComponentEvaluationData(InstanceHandle, FObjectKey(), SectionKey, AudioSection);
			if (!EvaluationData)
			{
				// Initialize the sound
				UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
				if (World)
				{
					EvaluationData = AudioSystem->AddRootAudioComponent(InstanceHandle, AudioSection, World);
					ensure(EvaluationData && EvaluationData->AudioComponent.Get() != nullptr);
				}
			}

			if (EvaluationData)
			{
				UAudioComponent* AudioComponent = EvaluationData->AudioComponent.Get();

				if (AudioComponent)
				{
					UpdatePreAnimatedStorage(EntityID,
											Instance,
											RootInstanceHandle,
											bWantsRestoreState,
											Context,
											EvaluationData,
											AudioSection,
											bRootMovieSceneIsRecordingAudio,
											AudioComponent);

					if (AttachComponent && (AudioComponent->GetAttachParent() != AttachComponent || AudioComponent->GetAttachSocketName() != AttachKey.SocketName))
					{
						AudioComponent->AttachToComponent(AttachComponent, FAttachmentTransformRules::KeepRelativeTransform, AttachKey.SocketName);
					}
					else if (!AttachComponent && (AudioComponent->GetAttachParent() != AttachComponent || AudioComponent->GetAttachSocketName() != AttachKey.SocketName))
					{
						AudioComponent->DetachFromComponent(FDetachmentTransformRules::KeepRelativeTransform);
					}

					if (AudioSection->GetOnQueueSubtitles().IsBound())
					{
						AudioComponent->OnQueueSubtitles = AudioSection->GetOnQueueSubtitles();
					}
					else
					{
						AudioComponent->OnQueueSubtitles.Clear();
					}

					if (AudioSection->GetOnAudioFinished().IsBound())
					{
						AudioComponent->OnAudioFinished = AudioSection->GetOnAudioFinished();
					}
					else
					{
						AudioComponent->OnAudioFinished.Clear();
					}

					if (AudioSection->GetOnAudioPlaybackPercent().IsBound())
					{
						AudioComponent->OnAudioPlaybackPercent = AudioSection->GetOnAudioPlaybackPercent();
					}
					else
					{
						AudioComponent->OnAudioPlaybackPercent.Clear();
					}
				}

				EvaluationData->VolumeMultiplier = VolumeMultiplier * AudioSection->EvaluateEasing(Context.GetTime());
				EvaluationData->PitchMultiplier = PitchMultiplier;

				EnsureAudioIsPlaying(nullptr, InstanceHandle, *AudioSection, *EvaluationData, Context, PlaybackContext);
			}
		}

		// Object binding audio track
		else
		{
			FAudioComponentEvaluationData* EvaluationData = AudioSystem->GetAudioComponentEvaluationData(InstanceHandle, ActorKey, SectionKey, AudioSection);
			if (!EvaluationData)
			{
				// Initialize the sound
				EvaluationData = AudioSystem->AddBoundObjectAudioComponent(InstanceHandle, AudioSection, BoundObject);
			}

			if (EvaluationData)
			{
				UAudioComponent* AudioComponent = EvaluationData->AudioComponent.Get();

				if (AudioComponent)
				{
					UpdatePreAnimatedStorage(EntityID,
											Instance,
											RootInstanceHandle,
											bWantsRestoreState,
											Context,
											EvaluationData,
											AudioSection,
											bRootMovieSceneIsRecordingAudio,
											AudioComponent);

					if (AudioSection->GetOnQueueSubtitles().IsBound())
					{
						AudioComponent->OnQueueSubtitles = AudioSection->GetOnQueueSubtitles();
					}
					else
					{
						AudioComponent->OnQueueSubtitles.Clear();
					}

					if (AudioSection->GetOnAudioFinished().IsBound())
					{
						AudioComponent->OnAudioFinished = AudioSection->GetOnAudioFinished();
					}
					else
					{
						AudioComponent->OnAudioFinished.Clear();
					}

					if (AudioSection->GetOnAudioPlaybackPercent().IsBound())
					{
						AudioComponent->OnAudioPlaybackPercent = AudioSection->GetOnAudioPlaybackPercent();
					}
					else
					{
						AudioComponent->OnAudioPlaybackPercent.Clear();
					}
				}

				EvaluationData->VolumeMultiplier = VolumeMultiplier;
				EvaluationData->PitchMultiplier = PitchMultiplier;

				EnsureAudioIsPlaying(BoundObject, InstanceHandle, *AudioSection, *EvaluationData, Context, PlaybackContext);
			}
		}
	}

	void EnsureAudioIsPlaying(
			UObject* BoundObject,
			FInstanceHandle InstanceHandle,
			UMovieSceneAudioSection& AudioSection,
			FAudioComponentEvaluationData& EvaluationData,
			const FMovieSceneContext& Context, 
			UObject* PlaybackContext) const
	{
		using FInstanceObjectKey = UMovieSceneAudioSystem::FInstanceObjectKey;
		using FAudioInputsBySectionKey = UMovieSceneAudioSystem::FAudioInputsBySectionKey;
		using FAudioComponentInputEvaluationData = UMovieSceneAudioSystem::FAudioComponentInputEvaluationData;

		ensureMsgf(EvaluationData.AudioComponent.IsValid(), TEXT("Trying to evaluate audio track on an invalid audio component"));
		UAudioComponent& AudioComponent = *EvaluationData.AudioComponent.Get();
		
		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
		AWorldSettings* WorldSettings = World ? World->GetWorldSettings() : nullptr;

#if WITH_EDITOR
		UScrubbedSound* ScrubbedSound = EvaluationData.ScrubbedSound.Get();
#endif // WITH_EDITOR

		AudioSystem->PreAnimatedStorage->CachePreAnimatedValue(
				FCachePreAnimatedValueParams(), &AudioComponent,
				[](FObjectKey InKey) { return EPreAnimatedAudioStateType::AudioPlaying; });

		if (AudioComponent.VolumeMultiplier != EvaluationData.VolumeMultiplier)
		{
			AudioComponent.SetVolumeMultiplier(EvaluationData.VolumeMultiplier);
		}

		if (AudioComponent.PitchMultiplier != EvaluationData.PitchMultiplier)
		{
			AudioComponent.SetPitchMultiplier(EvaluationData.PitchMultiplier);
		}

		AudioComponent.bSuppressSubtitles = AudioSection.GetSuppressSubtitles();

		// Allow spatialization if we have any object we've been attached to.
		const bool bAllowSpatialization = (BoundObject != nullptr || AudioComponent.GetAttachParent() != nullptr);

		// Apply the input params.
		FAudioInputsBySectionKey& AudioInputsBySectionKey = AudioSystem->AudioInputsBySectionKey;
		FInstanceObjectKey SectionKey(InstanceHandle, FObjectKey(&AudioSection));
		FAudioComponentInputEvaluationData* AudioInputs = AudioInputsBySectionKey.Find(SectionKey);
		if (AudioInputs)
		{
			SetAudioInputParameters(AudioInputs->Inputs_Float, AudioComponent);
			SetAudioInputParameters(AudioInputs->Inputs_String, AudioComponent);
			SetAudioInputParameters(AudioInputs->Inputs_Bool, AudioComponent);
			SetAudioInputParameters(AudioInputs->Inputs_Int, AudioComponent);
		}

		FFrameNumber SectionStartFrame = (AudioSection.HasStartFrame() ? AudioSection.GetInclusiveStartFrame() : 0);

		// If this audio seciton is a scaling driver (ie, it has the section anchors decoration),
		//     we need to 'undo' the scaling from the evaluation time and use the scaled section start time
		FFrameTime EvalTime = Context.GetTime();
		UMovieSceneSectionAnchorsDecoration* AnchorsDecoration = AudioSection.FindDecoration<UMovieSceneSectionAnchorsDecoration>();
		if (AnchorsDecoration)
		{
			UMovieSceneScalingAnchors* ScalingAnchors = AudioSection.GetTypedOuter<UMovieScene>()->FindDecoration<UMovieSceneScalingAnchors>();
			if (const FMovieSceneScalingAnchor* AnchoredStart = ScalingAnchors->GetCurrentAnchors().Find(AnchorsDecoration->StartAnchor))
			{
				SectionStartFrame = AnchoredStart->Position;
				TOptional<FFrameTime> UnwarpedTime = ScalingAnchors->InverseRemapTimeCycled(EvalTime, EvalTime, FInverseTransformTimeParams());
				if (UnwarpedTime.IsSet())
				{
					EvalTime = UnwarpedTime.GetValue();
				}
			}
		}

		float SectionStartTimeSeconds = SectionStartFrame / Context.GetFrameRate();

		float InverseTimeDilation = 1.0f;
		const bool bUseTimeDilationToAdjustment = (bUseTimeDilationToAdjustPlayDurationCVar && WorldSettings);

		if (bUseTimeDilationToAdjustment)
		{
			// Use time dilation to correct the duration so that the sound stops at the correct time.
			// Without this adjustment, time dialations < 1.0 will incorrectly attempt to play beyond the end of the section.
			const float EffectiveTimeDilation = WorldSettings->GetEffectiveTimeDilation();
			if (!FMath::IsNearlyEqual(EffectiveTimeDilation, 0.f))
			{
				InverseTimeDilation = (1.0f / EffectiveTimeDilation);
				SectionStartTimeSeconds *= InverseTimeDilation;
			}
		}

		const FFrameNumber AudioStartOffset = AudioSection.GetStartOffset();
		USoundBase* Sound = AudioSection.GetPlaybackSound();

		float AudioTime = (EvalTime / Context.GetFrameRate()) * InverseTimeDilation
			- SectionStartTimeSeconds 
			+ (float)Context.GetFrameRate().AsSeconds(AudioStartOffset);

		if (AudioTime >= 0.f && Sound)
		{
			if (bUseTimeDilationToAdjustment)
			{
				// Keep track of initial Audio and Context times when sound started playing.
				if (!(EvaluationData.LastAudioTime.IsSet() && EvaluationData.LastContextTime.IsSet()))
				{
					// Store current audio time and current context time
					EvaluationData.LastAudioTime = AudioTime;
					EvaluationData.LastContextTime = EvalTime / Context.GetFrameRate();
				}
				else
				{
					// Get previous AudioTime
					AudioTime = (EvaluationData.LastAudioTime.GetValue() < AudioTime) ? EvaluationData.LastAudioTime.GetValue() : AudioTime;
					float CurrContextTime = (EvalTime / Context.GetFrameRate());
					float PrevContextTime = EvaluationData.LastContextTime.GetValue();
					PrevContextTime = (PrevContextTime < CurrContextTime) ? PrevContextTime : CurrContextTime;

					// Get Time Delta between previous time in sequencer context and current time, not taking into account dilation.
					// Add to previous frame's audio time
					AudioTime += (CurrContextTime - PrevContextTime) * (InverseTimeDilation);

					EvaluationData.LastAudioTime = AudioTime;
					EvaluationData.LastContextTime = CurrContextTime;
				}
			}

			// Procedurally generated sounds don't have a defined duration so when the audio component is done, it's done
			if (Sound->IsProcedurallyGenerated())
			{
				const EMovieScenePlayerStatus::Type Status = Context.GetStatus();

				if (AudioSection.IsPlayUntilFinishedActive() && AudioSection.HasEndFrame() && EvalTime.FrameNumber >= AudioSection.GetExclusiveEndFrame())
				{
					// PlayUntilFinished tail: once eval time is past the section's visual end, suppress all further
					// trigger work so seeking through the silent extension doesn't restart the sound
					// (procedural sounds without a StartTime interface can only play from t=0).
					if (IsSeekingStatus(Status))
					{
						return;
					}

					// Press-Play / resume-from-Pause with the playhead in the PUF tail: don't retrigger a finished
					// procedural sound. Gated on !IsPlaying() so a normal play-through (AC actively rendering the tail)
					// still gets through to the volume/pitch/spatialization updates below.
					if (Status == EMovieScenePlayerStatus::Playing && !AudioComponent.IsPlaying())
					{
						EvaluationData.bAudioComponentHasBeenPlayed = true;
						return;
					}
				}

				if (Status == EMovieScenePlayerStatus::Playing)
				{
					if (EvaluationData.bAudioComponentHasBeenPlayed && !AudioSection.IsRepeating())
					{
						// If we're not a repeating section and the AC is done, return
						// otherwise a repeating section will restart the sound if it's not playing
						if (!AudioComponent.IsPlaying()) // !IsPlaying() == not playing, fading, or paused
						{
							UE_LOGF(LogMovieScene, Verbose, "Procedural sound Audio Component reached end of playback. Component: %ls Sound: %ls", *AudioComponent.GetName(), *GetNameSafe(AudioComponent.Sound));
							return;
						}
					}
				}
				else
				{
					// PUF sections deliberately stay alive past their visual end. Don't reset the
					// "has been played" flag on pause/resume in that case, otherwise a finished
					// MetaSound would be re-triggered on the next Playing tick.
					if (!AudioSection.IsPlayUntilFinishedActive())
					{
						EvaluationData.bAudioComponentHasBeenPlayed = false;
					}
				}
			}
			else
			{
				const float Duration = MovieSceneHelpers::GetSoundDuration(Sound);
				check(Duration >= 0.f);

				if (!AudioSection.IsLoopingOrRepeating() && AudioTime >= Duration && Duration != 0.f)
				{
					// If this is non-procedurally generated audio, and it's not looping or repeating, check if it needs to be stopped
					if (AudioComponent.IsPlaying()) // IsPlaying() == playing, fading, or paused
					{
						UE_LOGF(LogMovieScene, Verbose, "Audio Component reached end of playback. Component: %ls Sound: %ls", *AudioComponent.GetName(), *GetNameSafe(AudioComponent.Sound));
						AudioComponent.Stop();
					}

					return;
				}
				else
				{
					// Wrap AudioTime according to duration for non-procedural audio that is looping or repeating
					AudioTime = Duration > 0.f ? FMath::Fmod(AudioTime, Duration) : AudioTime;
				}
			}
		}

		// If the audio component is not playing we (may) need a state change. If the audio component is playing
		// the wrong sound then we need a state change. If the audio playback time is significantly out of sync 
		// with the desired time then we need a state change.
		const EAudioComponentPlayState AudioComponentPlayState = AudioComponent.GetPlayState();
		const bool bAudioComponentIsPaused = (AudioComponentPlayState == EAudioComponentPlayState::Paused);
		const bool bSoundsNeedPlaying = !AudioComponent.IsPlaying() || bAudioComponentIsPaused;
		const bool bSoundNeedsStateChange =  AudioComponent.Sound != Sound;
		bool bSoundNeedsTimeSync = false;


		// Sync only if:
		// - We are playing, and
		// - There is no time dilation because otherwise the system will constantly resync because audio 
		// playback is not dilated and will never match the expected playback time.
		const bool bDoTimeSync = 
			(
				AudioComponentPlayState == EAudioComponentPlayState::Playing ||
				AudioComponentPlayState == EAudioComponentPlayState::FadingIn ||
				AudioComponentPlayState == EAudioComponentPlayState::FadingOut
			) &&
			World && WorldSettings &&
			(FMath::IsNearlyEqual(WorldSettings->GetEffectiveTimeDilation(), 1.f) ||
			 !bIgnoreAudioSyncDuringWorldTimeDilationCVar);

		if (bDoTimeSync)
		{
			float CurrentGameTime = 0.0f;

			FAudioDevice* AudioDevice = World ? World->GetAudioDeviceRaw() : nullptr;
			if (UseAudioClockForSequencerDesyncCVar && AudioDevice)
			{
				CurrentGameTime = AudioDevice->GetAudioClock();
			}
			else
			{
				CurrentGameTime = World ? World->GetAudioTimeSeconds() : 0.f;
			}

			// This tells us how much time has passed in the game world (and thus, reasonably, the audio playback)
			// so if we calculate that we should be playing say, 15s into the section during evaluation, but
			// we're only 5s since the last Play call, then we know we're out of sync. 
			if (EvaluationData.PartialDesyncComputation.IsSet())
			{
				const float PartialDesyncComputation = EvaluationData.PartialDesyncComputation.GetValue();
				float Desync = PartialDesyncComputation + AudioTime - CurrentGameTime;

				if (!FMath::IsNearlyZero(MaxSequenceAudioDesyncToleranceCVar) && FMath::Abs(Desync) > MaxSequenceAudioDesyncToleranceCVar)
				{
					UE_LOGF(LogMovieScene, Verbose, "Audio Component detected a significant mismatch in (assumed) playback time versus the desired time. Desync: %6.2f(s) Desired Time: %6.2f(s). Component: %ls Sound: %ls", Desync, AudioTime, *AudioComponent.GetName(), *GetNameSafe(AudioComponent.Sound));
					bSoundNeedsTimeSync = true;
				}
			}
		}

		if (bSoundsNeedPlaying || bSoundNeedsStateChange || bSoundNeedsTimeSync)
		{
#if !NO_LOGGING
			FString ReasonMessage;
			if (bSoundsNeedPlaying)
			{
				ReasonMessage += TEXT("playing");
			}
			else if (bSoundNeedsStateChange)
			{
				ReasonMessage += TEXT("state change");
			}
			else
			{
				ReasonMessage += TEXT("time sync");
			}
			UE_LOGF(LogMovieScene, Verbose, "Audio component needs %ls. Component: %ls", *ReasonMessage, *AudioComponent.GetName());
#endif

			AudioComponent.bAllowSpatialization = bAllowSpatialization;

			if (AudioSection.GetOverrideAttenuation())
			{
				AudioComponent.AttenuationSettings = AudioSection.GetAttenuationSettings();
			}

			// If our sound is currently the scrubbed sound, that means we're actively scrubbing
			// So we don't need to stop or set the sound again
			if (Context.GetStatus() != EMovieScenePlayerStatus::Scrubbing)
			{
				// Only call stop on the sound if it is actually playing. This prevents spamming
				// stop calls when a sound cue with a duration of zero is played.
				if ((AudioComponent.IsPlaying() && !bAudioComponentIsPaused) || bSoundNeedsTimeSync)
				{
					UE_LOGF(LogMovieScene, Verbose, "Audio Component stopped due to needing a state change bIsPlaying: %d bNeedsTimeSync: %d. Component: %ls Sound: %ls", AudioComponent.IsPlaying(), bSoundNeedsTimeSync, *AudioComponent.GetName(), *GetNameSafe(AudioComponent.Sound));
					AudioComponent.Stop();

					UE_LOGF(LogMovieScene, Verbose, "AudioComponent.Stop()");
					if (Context.GetStatus() != EMovieScenePlayerStatus::Playing)
					{
						EvaluationData.bAudioComponentHasBeenPlayed = false;
					}
				}
			}

#if WITH_EDITOR
			if (GIsEditor && World != nullptr && !World->IsPlayInEditor())
			{
				// This is needed otherwise the sound doesn't have a position and will not play properly
				AudioComponent.bIsUISound = true;
			}
			else
#endif // WITH_EDITOR
			{
				AudioComponent.bIsUISound = false;
			}

			if (AudioTime >= 0.f)
			{
				UE_LOGF(LogMovieScene, Verbose, "Audio Component Play at Local Time: %6.2f CurrentTime: %6.2f(s) SectionStart: %6.2f(s), SoundDur: %6.2f OffsetIntoClip: %6.2f Component: %ls Sound: %ls", AudioTime, (EvalTime / Context.GetFrameRate()), SectionStartTimeSeconds, AudioComponent.Sound ? AudioComponent.Sound->GetDuration() : 0.0f, (float)Context.GetFrameRate().AsSeconds(AudioStartOffset), *AudioComponent.GetName(), *GetNameSafe(AudioComponent.Sound));
				
#if WITH_EDITOR
				// We only want to perform granular scrubbing in the narrow case of a non-procedural sound wave
				// Otherwise, we fallback to simply restarting the sound at the given audio time.
				USoundWave* SoundWave = Cast<USoundWave>(Sound);
				bool bPerformGranularScrubbing = bEnableGranularScrubbingCVar
					&& (nullptr != SoundWave)
					&& !SoundWave->IsProcedurallyGenerated()
					&& (nullptr != ScrubbedSound)
					&& Context.GetStatus() == EMovieScenePlayerStatus::Scrubbing;

				if (bPerformGranularScrubbing)
				{
					// If we're not playing the audio component, then set up the audio component to use the scrubbed sound wave and play it
					if (!AudioComponent.IsPlaying()) // !IsPlaying() == not playing, fading, or paused
					{
						ScrubbedSound->SetSoundWave(SoundWave);
						ScrubbedSound->ModulationSettings = SoundWave->ModulationSettings;
						ScrubbedSound->SetPlayheadTime(AudioTime);
						float MaxScrubWidthSeconds = 0.001f * (float)ScrubWidthMillisecondsCVar;
						ScrubbedSound->SetGrainDurationRange({ MaxScrubWidthSeconds, 0.05f });

						ScrubbedSound->SetIsScrubbing(true);
						ScrubbedSound->SetIsScrubbingWhileStationary(bEnableGranularScrubbingWhileStationaryCVar);

						AudioComponent.SetSound(ScrubbedSound);
						AudioComponent.Play();
					}
					else
					{
						// If we're already playing, then just update the playhead time on the scrubbed sound
						// This will propagate the playhead time to the rendering ISoundGenerator
						ScrubbedSound->SetPlayheadTime(AudioTime);
					}
				}
				else
#endif // WITH_EDITOR
				{
#if WITH_EDITOR
					if (ScrubbedSound)
					{
						ScrubbedSound->SetIsScrubbing(false);
					}
#endif // WITH_EDITOR

					// Only change the sound clip if it has actually changed. This calls Stop internally if needed.
					if (AudioComponent.Sound != Sound)
					{
						UE_LOGF(LogMovieScene, Verbose, "Audio Component calling SetSound due to new sound. Component: %ls OldSound: %ls NewSound: %ls", *AudioComponent.GetName(), *GetNameSafe(AudioComponent.Sound), *GetNameSafe(Sound));
						AudioComponent.SetSound(Sound);
					}
					
					if (!AudioComponent.IsPlaying() || !bAudioComponentIsPaused) // !IsPlaying() == not playing, fading, or paused
					{
						AudioComponent.Play(AudioTime);
					}
					else
					{
						AudioComponent.SetPaused(false); // no-op if already unpaused
					}
					
					if (Context.GetStatus() == EMovieScenePlayerStatus::Playing)
					{
						// Set that we've played an audio component. This is used by procedural sounds who have undefined duration.
						EvaluationData.bAudioComponentHasBeenPlayed = true;
					}

					// Keep track of when we asked this audio clip to play (in game time) so that we can figure 
					// out if there's a significant desync in the future.
					//
					// The goal is later to compare:
					//   (NewAudioTime - PreviousAudioTime) and 
					//   (NewGameTime - PreviousGameTime)
					//
					// If their difference is larger than some threshold, we have a desync. NewGameTime and 
					// NewAudioTime will be known next update, but PreviousGameTime and PreviousAudioTime
					// are known now. Let's store (-PreviousAudioTime + PreviousGameTime), and we will only 
					// need to add (NewAudioTime - NewGameTime).
					if (World)
					{
						FAudioDevice* AudioDevice = World->GetAudioDeviceRaw();
						if (UseAudioClockForSequencerDesyncCVar && AudioDevice)
						{
							EvaluationData.PartialDesyncComputation = AudioDevice->GetInterpolatedAudioClock() - AudioTime;
						}
						else
						{
							EvaluationData.PartialDesyncComputation = World->GetAudioTimeSeconds() - AudioTime;
						}
					}
				}
			}
		}

		if (IsSeekingStatus(Context.GetStatus()))
		{
			float ScrubDuration = AudioTrackConstants::ScrubDuration;
			if (FAudioDevice* AudioDevice = AudioComponent.GetAudioDevice())
			{
				constexpr float MinScrubFrameRateFactor = 1.5f;
				float DeviceDeltaTime = AudioDevice->GetGameDeltaTime();

				// When operating at very low frame-rates (<20fps), a single frame will be
				// longer than the hard coded scrub duration of 50ms in which case the delayed
				// stop will trigger on the same frame that the sound starts playing and
				// no audio will be heard. Here we increase the scrub duration to be greater than
				// a single frame if necessary.
				ScrubDuration = FMath::Max(ScrubDuration, DeviceDeltaTime * MinScrubFrameRateFactor);
			}

			// While scrubbing, play the sound for a short time and then cut it.
			AudioComponent.StopDelayed(ScrubDuration);
		}

		if ((AudioComponent.IsPlaying() && !bAudioComponentIsPaused) && AudioInputs)
		{
			SetAudioInputTriggers(AudioInputs->Inputs_Trigger, AudioComponent);
		}

		if (bAllowSpatialization)
		{
			if (FAudioDevice* AudioDevice = AudioComponent.GetAudioDevice())
			{
				DECLARE_CYCLE_STAT(TEXT("FAudioThreadTask.MovieSceneUpdateAudioTransform"), STAT_MovieSceneUpdateAudioTransform, STATGROUP_TaskGraphTasks);
				AudioDevice->SendCommandToActiveSounds(AudioComponent.GetAudioComponentID(), [ActorTransform = AudioComponent.GetComponentTransform()](FActiveSound& ActiveSound)
				{
					ActiveSound.bLocationDefined = true;
					ActiveSound.Transform = ActorTransform;
				}, GET_STATID(STAT_MovieSceneUpdateAudioTransform));
			}
		}
	}

	// Helper method for firing all triggered audio triggers.
	void SetAudioInputTriggers(const TArray<FName>& Inputs, IAudioParameterControllerInterface& InParamaterInterface) const
	{
		for (const FName& TriggerName : Inputs)
		{
			InParamaterInterface.SetTriggerParameter(TriggerName);
		};
	}

	// Helper template to set all audio input values previously evaluated.
	template<typename ValueType>
	void SetAudioInputParameters(TMap<FName, ValueType>& Inputs, IAudioParameterControllerInterface& InParamaterInterface) const
	{
		for (TPair<FName, ValueType>& Pair : Inputs)
		{
			InParamaterInterface.SetParameter<ValueType>(Pair.Key, MoveTempIfPossible(Pair.Value));
		};
	}
};

} // namespace UE::MovieScene

UMovieSceneAudioSystem::UMovieSceneAudioSystem(const FObjectInitializer& ObjInit)
	: UMovieSceneEntitySystem(ObjInit)
{
	using namespace UE::MovieScene;

	RelevantComponent = FMovieSceneTracksComponentTypes::Get()->Audio;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

		// We consume the result of all possible audio input channels.
		for (int32 Index = 0; Index < 9; ++Index)
		{
			DefineComponentConsumer(GetClass(), BuiltInComponents->DoubleResult[Index]);
		}
		DefineComponentConsumer(GetClass(), BuiltInComponents->StringResult);
		DefineComponentConsumer(GetClass(), BuiltInComponents->IntegerResult);
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoolResult);
		DefineComponentConsumer(GetClass(), TrackComponents->AudioTriggerName);
	}
}

void UMovieSceneAudioSystem::OnLink()
{
	using namespace UE::MovieScene;
	check(Linker);

	PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedAudioStorage>();
}

void UMovieSceneAudioSystem::OnUnlink()
{
	using namespace UE::MovieScene;

	for (const TPair<FObjectKey, FAudioComponentBySectionKey>& AudioComponentsForActor : AudioComponentsByActorKey)
	{
		for (const TPair<FInstanceObjectKey, FAudioComponentEvaluationData>& AudioComponentForSection : AudioComponentsForActor.Value)
		{
			UAudioComponent* AudioComponent = AudioComponentForSection.Value.AudioComponent.Get();
			if (AudioComponent)
			{
				UObject* Actor = AudioComponentsForActor.Key.ResolveObjectPtr();
				UObject* Section = AudioComponentForSection.Key.Value.ResolveObjectPtr();
				bool bIsCurrentlyLooping = AudioComponent->GetSound() && AudioComponent->GetSound()->IsLooping() && AudioComponent->IsPlaying();
				UE_LOGF(LogMovieScene, Warning, "%lsequencer sound playing past section end - Audio component '%ls' for section '%ls' on actor '%ls'.",
						bIsCurrentlyLooping ? TEXT("Looping s") : TEXT("S"),
						*AudioComponent->GetPathName(),
						Section ? *Section->GetPathName() : TEXT("<null>"),
						Actor ? *Actor->GetPathName() : TEXT("<null>"));

				// Looping components never reach natural completion, so OnAudioFinishedNative will not fire and the destroy-on-completion path below will 
				// leak the component. Stop them explicitly here. Non-looping components are left alone and allowed to finish naturally.
				if (bIsCurrentlyLooping)
				{
					AudioComponent->Stop();
					AudioComponent->DestroyComponent();
				}
				else if (AudioComponent->IsPlaying()) // If unlinking a playing audio component, set it up to destroy on completion so it can continue to play after the movie scene is complete
				{
					AudioComponent->OnAudioFinishedNative.AddLambda([](UAudioComponent* FinishingComponent)
						{
							UE_LOGF(LogMovieScene, Display, "Destroying audio component '%ls' upon completion",
								*FinishingComponent->GetPathName());
							FinishingComponent->DestroyComponent();
						});
				}
				// otherwise if it is not playing, just destroy the audio component now
				else
				{
					UE_LOGF(LogMovieScene, Display, "Destroying stopped audio component '%ls' on MovieSceneAudioSystem::OnUnlink",
						*AudioComponent->GetPathName());
					AudioComponent->DestroyComponent();
				}
			}
		}
	}

	AudioComponentsByActorKey.Reset();
	AudioInputsBySectionKey.Reset();

}

void UMovieSceneAudioSystem::ResetSharedData()
{
	bLifespanTrackingInvalidatedThisFrame = false;
	AudioInputsBySectionKey.Reset();
	for (TPair<FObjectKey, FAudioComponentBySectionKey>& AudioComponentsForActor : AudioComponentsByActorKey)
	{
		for (TPair<FInstanceObjectKey, FAudioComponentEvaluationData>& AudioComponentForSection : AudioComponentsForActor.Value)
		{
			AudioComponentForSection.Value.bEvaluatedThisFrame = false;
		}
	}
}

void UMovieSceneAudioSystem::AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector)
{
	Super::AddReferencedObjects(InThis, Collector);
#if WITH_EDITOR
	UMovieSceneAudioSystem* This = CastChecked<UMovieSceneAudioSystem>(InThis);
	for (auto& ActorPair : This->AudioComponentsByActorKey)
	{
		for (auto& SectionPair : ActorPair.Value)
		{
			FAudioComponentEvaluationData& EvalData = SectionPair.Value;
			if (EvalData.ScrubbedSound)
			{
				Collector.AddReferencedObject(EvalData.ScrubbedSound);
			}
		}
	}
#endif
}

void UMovieSceneAudioSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	if (!GEngine || !GEngine->UseSound())
	{
		return;
	}

	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	// Reset shared data.
	FTaskID ResetSharedDataTask = TaskScheduler->AddMemberFunctionTask(FTaskParams(TEXT("Reset Audio Data")), this, &UMovieSceneAudioSystem::ResetSharedData);

	// Gather audio input values computed by the channel evaluators.
	FTaskID GatherInputsTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(TrackComponents->Audio)
	.Read(TrackComponents->AudioInputs)
	.ReadOneOrMoreOf(
			BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2], 
			BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5], 
			BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8], 
			BuiltInComponents->StringResult,
			BuiltInComponents->IntegerResult,
			BuiltInComponents->BoolResult)
	.Schedule_PerAllocation<FGatherAudioInputs>(&Linker->EntityManager, TaskScheduler, this);

	TaskScheduler->AddPrerequisite(ResetSharedDataTask, GatherInputsTask);

	// Gather up audio triggers
	FTaskID GatherTriggersTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(TrackComponents->Audio)
	.Read(TrackComponents->AudioTriggerName)
	.Schedule_PerAllocation<FGatherAudioTriggers>(&Linker->EntityManager, TaskScheduler, this);

	TaskScheduler->AddPrerequisite(ResetSharedDataTask, GatherTriggersTask);

	// Next, evaluate audio to play and use the gathered audio input values to set on the audio components.
	FTaskID EvaluateAudioTask = FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->InstanceHandle)
	.Read(TrackComponents->Audio)
	.Read(BuiltInComponents->DoubleResult[0]) // Volume
	.Read(BuiltInComponents->DoubleResult[1]) // Pitch multiplier
	.ReadOptional(BuiltInComponents->BoundObject)
	.SetDesiredThread(Linker->EntityManager.GetGatherThread())
	.Schedule_PerAllocation<FEvaluateAudio>(&Linker->EntityManager, TaskScheduler, this);

	TaskScheduler->AddPrerequisite(GatherInputsTask, EvaluateAudioTask);
	TaskScheduler->AddPrerequisite(GatherTriggersTask, EvaluateAudioTask);
	TaskScheduler->AddPrerequisite(ResetSharedDataTask, EvaluateAudioTask);
}

void UMovieSceneAudioSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_AudioTrack_Evaluate)

	if (!GEngine || !GEngine->UseSound())
	{
		return;
	}

	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	// Reset shared data.
	ResetSharedData();

	// Gather audio input values computed by the channel evaluators.
	FSystemTaskPrerequisites Prereqs;

	FGraphEventRef Task = FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(TrackComponents->Audio)
	.Read(TrackComponents->AudioInputs)
	.ReadOneOrMoreOf(
			BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2], 
			BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5], 
			BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8], 
			BuiltInComponents->StringResult,
			BuiltInComponents->IntegerResult,
			BuiltInComponents->BoolResult)
	.template Dispatch_PerAllocation<FGatherAudioInputs>(&Linker->EntityManager, InPrerequisites, nullptr, this);
	if (Task)
	{
		Prereqs.AddRootTask(Task);
	}

	Task = FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(TrackComponents->Audio)
	.Read(TrackComponents->AudioTriggerName)
	.template Dispatch_PerAllocation<FGatherAudioTriggers>(&Linker->EntityManager, InPrerequisites, nullptr, this);
	if (Task)
	{
		Prereqs.AddRootTask(Task);
	}

	// Next, evaluate audio to play and use the gathered audio input values to set on the audio components.
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->InstanceHandle)
	.Read(TrackComponents->Audio)
	.Read(BuiltInComponents->DoubleResult[0]) // Volume
	.Read(BuiltInComponents->DoubleResult[1]) // Pitch multiplier
	.ReadOptional(BuiltInComponents->BoundObject)
	.SetDesiredThread(Linker->EntityManager.GetGatherThread())
	.template Dispatch_PerAllocation<FEvaluateAudio>(&Linker->EntityManager, Prereqs, &Subsequents, this);
}

UMovieSceneAudioSystem::FAudioComponentEvaluationData* UMovieSceneAudioSystem::GetAudioComponentEvaluationData(FInstanceHandle InstanceHandle, FObjectKey ActorKey, FObjectKey SectionKey, UMovieSceneAudioSection* Section)
{
	FAudioComponentBySectionKey* Map = AudioComponentsByActorKey.Find(ActorKey);
	if (Map != nullptr)
	{
		// First, check for an exact match for this entity
		FInstanceObjectKey DataKey{ InstanceHandle, SectionKey };
		FAudioComponentEvaluationData* ExistingData = Map->Find(DataKey);
		if (ExistingData != nullptr)
		{
			if (ExistingData->AudioComponent.IsValid())
			{
				return ExistingData;
			}
		}

		// If no exact match, check for any AudioComponent that isn't busy and doesn't contain a procedurally generated sound
		for (FAudioComponentBySectionKey::ElementType& Pair : *Map)
		{
			UAudioComponent* ExistingComponent = Pair.Value.AudioComponent.Get();
			if (ExistingComponent && !ExistingComponent->IsPlaying() &&
				(ExistingComponent->Sound == nullptr || !ExistingComponent->Sound->IsProcedurallyGenerated()))
			{
				// Skip ACs still owned by a PUF audio section sitting in its tail - stealing them leaves the old section's
				// pre-animated contributor holding the AC alive past the new section's end.
				if (TStrongObjectPtr<UMovieSceneAudioSection> OwningSection = Pair.Value.OwningSection.Pin())
				{
					if (OwningSection->IsPlayUntilFinishedActive())
					{
						continue;
					}
				}

				// Replace this entry with the new entity ID to claim it
				FAudioComponentEvaluationData MovedData(Pair.Value);
				Map->Remove(Pair.Key);
				MovedData.PartialDesyncComputation.Reset();

				MovedData.LastAudioTime.Reset();
				MovedData.LastContextTime.Reset();
				MovedData.OwningSection = Section;

				return &Map->Add(DataKey, MovedData);
			}
		}
	}

	return nullptr;
}

UMovieSceneAudioSystem::FAudioComponentEvaluationData* UMovieSceneAudioSystem::AddBoundObjectAudioComponent(FInstanceHandle InstanceHandle, UMovieSceneAudioSection* Section, UObject* PrincipalObject)
{
	using namespace UE::MovieScene;

	FObjectKey ObjectKey(PrincipalObject);
	FObjectKey SectionKey(Section);

	FAudioComponentBySectionKey& ActorAudioComponentMap = AudioComponentsByActorKey.FindOrAdd(ObjectKey);

	FAudioComponentEvaluationData* ExistingData = GetAudioComponentEvaluationData(InstanceHandle, ObjectKey, SectionKey, Section);
	if (!ExistingData)
	{
		USoundCue* TempPlaybackAudioCue = NewObject<USoundCue>();

		AActor* Actor = nullptr;
		USceneComponent* SceneComponent = nullptr;
		FString ObjectName;

		if (PrincipalObject->IsA<AActor>())
		{
			Actor = Cast<AActor>(PrincipalObject);
			SceneComponent = Actor->GetRootComponent();
			ObjectName =
#if WITH_EDITOR
				Actor->GetActorLabel();
#else
				Actor->GetName();
#endif
		}
		else if (PrincipalObject->IsA<UActorComponent>())
		{
			UActorComponent* ActorComponent = Cast<UActorComponent>(PrincipalObject);
			Actor = ActorComponent->GetOwner();
			SceneComponent = Cast<USceneComponent>(ActorComponent);
			ObjectName = ActorComponent->GetName();
		}

		if (!Actor || !SceneComponent)
		{
			const int32 RowIndex = Section->GetRowIndex();
			UE_LOGF(LogMovieScene, Warning, "Failed to find scene component for spatialized audio track (row %d).", RowIndex);
			return nullptr;
		}

		FAudioDevice::FCreateComponentParams Params(Actor->GetWorld(), Actor);
		UAudioComponent* NewComponent = FAudioDevice::CreateComponent(TempPlaybackAudioCue, Params);

		if (!NewComponent)
		{
			const int32 RowIndex = Section->GetRowIndex();
			UE_LOGF(LogMovieScene, Warning, "Failed to create audio component for spatialized audio track (row %d on %ls).", RowIndex, *ObjectName);
			return nullptr;
		}

		NewComponent->SetFlags(RF_Transient);
		NewComponent->AttachToComponent(SceneComponent, FAttachmentTransformRules::KeepRelativeTransform);

		FInstanceObjectKey DataKey{ InstanceHandle, SectionKey };
		ExistingData = &ActorAudioComponentMap.Add(DataKey);
		ExistingData->AudioComponent = NewComponent;
		ExistingData->OwningSection = Section;
		ExistingData->bAudioComponentHasBeenPlayed = false;

		if (Section->GetOnQueueSubtitles().IsBound())
		{
			NewComponent->OnQueueSubtitles = Section->GetOnQueueSubtitles();
		}
		if (Section->GetOnAudioFinished().IsBound())
		{
			NewComponent->OnAudioFinished = Section->GetOnAudioFinished();
		}
		if (Section->GetOnAudioPlaybackPercent().IsBound())
		{
			NewComponent->OnAudioPlaybackPercent = Section->GetOnAudioPlaybackPercent();
		}

#if WITH_EDITOR
		static int32 ScrubSoundCounter = 0;
		UScrubbedSound* ScrubbedSound = NewObject<UScrubbedSound>(Actor, FName(*FString::Printf(TEXT("ScrubbedSound_Bound_%i"), ScrubSoundCounter++)));
		if (!ScrubbedSound)
		{
			const int32 RowIndex = Section->GetRowIndex();
			UE_LOGF(LogMovieScene, Warning, "Failed to create scrubbed sound audio track (row %d on %ls).", RowIndex, *ObjectName);
		}
		else
		{
			ScrubbedSound->SetFlags(RF_Transient);
			ExistingData->ScrubbedSound = TObjectPtr<UScrubbedSound>(ScrubbedSound);
		}
#endif // WITH_EDITOR

	}

	return ExistingData;
}

UMovieSceneAudioSystem::FAudioComponentEvaluationData* UMovieSceneAudioSystem::AddRootAudioComponent(FInstanceHandle InstanceHandle, UMovieSceneAudioSection* Section, UWorld* World)
{
	using namespace UE::MovieScene;

	FObjectKey NullKey;
	FObjectKey SectionKey(Section);

	FAudioComponentBySectionKey& RootAudioComponentMap = AudioComponentsByActorKey.FindOrAdd(NullKey);

	FAudioComponentEvaluationData* ExistingData = GetAudioComponentEvaluationData(InstanceHandle, NullKey, SectionKey, Section);
	if (!ExistingData)
	{
		USoundCue* TempPlaybackAudioCue = NewObject<USoundCue>();

		FAudioDevice::FCreateComponentParams Params(World);

		UAudioComponent* NewComponent = FAudioDevice::CreateComponent(TempPlaybackAudioCue, Params);

		if (!NewComponent)
		{
			const int32 RowIndex = Section->GetRowIndex();
			UE_LOGF(LogMovieScene, Warning, "Failed to create audio component for root audio track (row %d).", RowIndex);
			return nullptr;
		}

		NewComponent->SetFlags(RF_Transient);

		FInstanceObjectKey DataKey{ InstanceHandle, SectionKey };
		ExistingData = &RootAudioComponentMap.Add(DataKey);
		ExistingData->AudioComponent = NewComponent;
		ExistingData->OwningSection = Section;
		ExistingData->bAudioComponentHasBeenPlayed = false;

		if (Section->GetOnQueueSubtitles().IsBound())
		{
			NewComponent->OnQueueSubtitles = Section->GetOnQueueSubtitles();
		}
		if (Section->GetOnAudioFinished().IsBound())
		{
			NewComponent->OnAudioFinished = Section->GetOnAudioFinished();
		}
		if (Section->GetOnAudioPlaybackPercent().IsBound())
		{
			NewComponent->OnAudioPlaybackPercent = Section->GetOnAudioPlaybackPercent();
		}

#if WITH_EDITOR
		static int32 ScrubSoundCounter = 0;
		UObject* OuterObject = GetTransientPackage();
		if (World && World->GetCurrentLevel())
		{
			OuterObject = World->GetCurrentLevel();
		}
		UScrubbedSound* ScrubbedSound = NewObject<UScrubbedSound>(OuterObject, FName(*FString::Printf(TEXT("ScrubbedSound_Root_%i"), ScrubSoundCounter++)));
		if (!ScrubbedSound)
		{
			const int32 RowIndex = Section->GetRowIndex();
			UE_LOGF(LogMovieScene, Warning, "Failed to create scrubbed sound for root audio track (row %d).", RowIndex);
		}
		else
		{
			ScrubbedSound->SetFlags(RF_Transient);
			ExistingData->ScrubbedSound = TObjectPtr<UScrubbedSound>(ScrubbedSound);
		}
#endif // WITH_EDITOR
	}

	return ExistingData;
}

void UMovieSceneAudioSystem::StopSound(FInstanceHandle InstanceHandle, FObjectKey ActorKey, FObjectKey SectionKey, bool bIsRecordingAudio)
{
	if (FAudioComponentBySectionKey* Map = AudioComponentsByActorKey.Find(ActorKey))
	{
		FInstanceObjectKey DataKey{ InstanceHandle, SectionKey };
		if (FAudioComponentEvaluationData* Data = Map->Find(DataKey))
		{
			if (UAudioComponent* AudioComponent = Data->AudioComponent.Get())
			{
				if (!bIsRecordingAudio)
				{
					AudioComponent->Stop();
				}
				else
				{
					AudioComponent->SetPaused(true);
				}
			}

			Data->bAudioComponentHasBeenPlayed = false;
		}
	}
}

void UMovieSceneAudioSystem::InvalidateLifespanTracking()
{
	if (bLifespanTrackingInvalidatedThisFrame)
	{
		return;
	}
	bLifespanTrackingInvalidatedThisFrame = true;

	for (TPair<FObjectKey, FAudioComponentBySectionKey>& AudioComponentsForActor : AudioComponentsByActorKey)
	{
		for (TPair<FInstanceObjectKey, FAudioComponentEvaluationData>& AudioComponentForSection : AudioComponentsForActor.Value)
		{
			AudioComponentForSection.Value.AudioComponentLifespanControlSection = nullptr;
		}
	}
}

