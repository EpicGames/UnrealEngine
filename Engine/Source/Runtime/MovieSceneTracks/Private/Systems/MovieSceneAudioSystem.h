// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "MovieSceneTracksComponentTypes.h"
#include "UObject/ObjectMacros.h"
#include "Channels/MovieSceneAudioTriggerChannel.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Generators/SoundWaveScrubber.h"

#include "MovieSceneAudioSystem.generated.h"

class IAudioParameterControllerInterface;
class UAudioComponent;
class UMovieSceneSection;
class UMovieSceneAudioSection;
class USoundBase;
struct FMoveSceneAudioTriggerState;

namespace UE::MovieScene
{
	struct FGatherAudioInputs;
	struct FGatherAudioTriggers;
	struct FEvaluateAudio;
	struct FPreAnimatedAudioStorage;

	struct FAudioComponentInputEvaluationData
	{
		TMap<FName, float> Inputs_Float;
		TMap<FName, FString> Inputs_String;
		TMap<FName, bool> Inputs_Bool;
		TMap<FName, int32> Inputs_Int;
		TArray<FName> Inputs_Trigger;
	};

	struct FAudioComponentEvaluationData
	{
		/** The audio component that was created to play audio */
		TWeakObjectPtr<UAudioComponent> AudioComponent;

		/** The audio section that currently owns this entry. Stamped when the entry is created or stolen so the reuse loop can ask the section directly (e.g. PUF state). */
		TWeakObjectPtr<UMovieSceneAudioSection> OwningSection;

		/** The movie section managing the lifespan of the audio component */
		TObjectPtr<UMovieSceneSection> AudioComponentLifespanControlSection = nullptr;

		/** 
		 * Last player status seen during UpdatePreAnimatedStorage. Drives invalidation of 
		 * AudioComponentLifespanControlSection on Scrubbing transitions so re-tracking picks up the new effective bWantsRestoreState. 
		 */
		EMovieScenePlayerStatus::Type LastSeenStatus = EMovieScenePlayerStatus::Stopped;

#if WITH_EDITOR
		/** While in editor, we can scrub the audio in the audio component. */
		TObjectPtr<UScrubbedSound> ScrubbedSound;
#endif

		/** Volume multiplier to use this frame */
		double VolumeMultiplier = 1.0;

		/** Pitch multiplier to use this frame */
		double PitchMultiplier = 1.0;

		/**
		 * Set whenever we ask the Audio component to start playing a sound.
		 * Used to detect desyncs caused when Sequencer evaluates at more-than-real-time.
		 */
		TOptional<float> PartialDesyncComputation;

		/** Previous audio time taking into account any time dilation */
		TOptional<float> LastAudioTime;
		/** The context time from the previous evaluation pass */
		TOptional<float> LastContextTime;

		/** Flag to keep track of audio components evaluated on a given frame */
		bool bEvaluatedThisFrame = false;

		/** Flag to keep track of if the audio component was played in a previous frame. */
		bool bAudioComponentHasBeenPlayed = false;
	};
}

/**
 * System for evaluating audio tracks
 */
UCLASS()
class UMovieSceneAudioSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:

	using FInstanceHandle = UE::MovieScene::FInstanceHandle;
	using FMovieSceneEntityID = UE::MovieScene::FMovieSceneEntityID;
	using FAudioComponentEvaluationData = UE::MovieScene::FAudioComponentEvaluationData;
	using FAudioComponentInputEvaluationData = UE::MovieScene::FAudioComponentInputEvaluationData;

	UMovieSceneAudioSystem(const FObjectInitializer& ObjInit);

	//~ UMovieSceneEntitySystem members
	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	/**
	 * Get the evaluation data for the given actor and section. Pass a null actor key for root (world) audio.
	 * Section is the audio section requesting the entry; when the fallback steal path adopts an existing entry,
	 * the moved data has its OwningSection re-stamped to Section so subsequent reuse-loop checks see the new owner.
	 */
	FAudioComponentEvaluationData* GetAudioComponentEvaluationData(FInstanceHandle InstanceHandle, FObjectKey ActorKey, FObjectKey SectionKey, UMovieSceneAudioSection* Section);

	/**
	 * Adds an audio component to the given bound sequencer object.
	 * WARNING: Only to be called on the game thread.
	 */
	FAudioComponentEvaluationData* AddBoundObjectAudioComponent(FInstanceHandle InstanceHandle, UMovieSceneAudioSection* Section, UObject* PrincipalObject);

	/**
	 * Adds an audio component to the world, for playing root audio tracks.
	 * WARNING: Only to be called on the game thread.
	 */
	FAudioComponentEvaluationData* AddRootAudioComponent(FInstanceHandle InstanceHandle, UMovieSceneAudioSection* Section, UWorld* World);

	/**
	 * Stop (or just pause if bIsRecordingAudio) the audio on the audio component associated with the given audio section.
	 */
	void StopSound(FInstanceHandle InstanceHandle, FObjectKey ActorKey, FObjectKey SectionKey, bool bIsRecordingAudio);

	/**
	 * Reset shared accumulation data required every evaluation frame
	 */
	void ResetSharedData();

	// To expose the class to GC //
	static void AddReferencedObjects(UObject* InThis, FReferenceCollector& Collector);

private:

	using FInstanceObjectKey = TTuple<FInstanceHandle, FObjectKey>;

	/** Map of all created audio components */
	using FAudioComponentBySectionKey = TMap<FInstanceObjectKey, FAudioComponentEvaluationData>;
	using FAudioComponentsByActorKey = TMap<FObjectKey, FAudioComponentBySectionKey>;
	FAudioComponentsByActorKey AudioComponentsByActorKey;

	/** Map of audio input values, rebuilt every frame */
	using FAudioInputsBySectionKey = TMap<FInstanceObjectKey, FAudioComponentInputEvaluationData>;
	FAudioInputsBySectionKey AudioInputsBySectionKey;

	/** Pre-animated state */
	TSharedPtr<UE::MovieScene::FPreAnimatedAudioStorage> PreAnimatedStorage;

	/** Set by InvalidateLifespanTracking() to suppress redundant full-map walks within a single frame when many entities transition in or out of Scrubbing simultaneously. Reset in ResetSharedData(). */
	bool bLifespanTrackingInvalidatedThisFrame = false;

	/** Reset the cached AudioComponentLifespanControlSection on every tracked component so the next eval re-issues BeginTrackingEntity with the current effective bWantsRestoreState. Idempotent within a frame. */
	void InvalidateLifespanTracking();

	friend struct UE::MovieScene::FGatherAudioInputs;
	friend struct UE::MovieScene::FGatherAudioTriggers;
	friend struct UE::MovieScene::FEvaluateAudio;
};

