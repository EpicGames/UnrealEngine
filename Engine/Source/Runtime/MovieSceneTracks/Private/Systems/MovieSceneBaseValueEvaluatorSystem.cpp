// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneBaseValueEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBaseValueEvaluatorSystem)

namespace UE
{
namespace MovieScene
{

struct FAssignBaseValueEvalSeconds
{
	FInstanceRegistry* InstanceRegistry;

	explicit FAssignBaseValueEvalSeconds(FInstanceRegistry* InInstanceRegistry)
		: InstanceRegistry(InInstanceRegistry)
	{}

	void ForEachEntity(UE::MovieScene::FInstanceHandle InstanceHandle, const FFrameTime& BaseValueEvalTime, double& BaseValueEvalSeconds)
	{
		const FSequenceInstance& Instance = InstanceRegistry->GetInstance(InstanceHandle);
		const FMovieSceneContext& Context = Instance.GetContext();
		BaseValueEvalSeconds = Context.GetFrameRate().AsSeconds(BaseValueEvalTime);
	}
};

struct FEvaluateBaseByteValues
{
	void ForEachEntity(FSourceByteChannel ByteChannel, FFrameTime FrameTime, uint8& OutResult)
	{
		if (!ByteChannel.Source->Evaluate(FrameTime, OutResult))
		{
			OutResult = MIN_uint8;
		}
	}
};

struct FEvaluateBaseIntegerValues
{
	void ForEachEntity(FSourceIntegerChannel IntegerChannel, FFrameTime FrameTime, int64& OutResult)
	{
		if (!IntegerChannel.Source->Evaluate(FrameTime, OutResult))
		{
			OutResult = MIN_int64;
		}
	}
};

struct FEvaluateBaseFloatValues
{
	void ForEachEntity(FSourceFloatChannel FloatChannel, FFrameTime FrameTime, double& OutResult)
	{
		float Result;
		if (!FloatChannel.Source->Evaluate(FrameTime, Result))
		{
			Result = MIN_flt;
		}
		OutResult = (double)Result;
	}
};

struct FEvaluateBaseDoubleValues
{
	void ForEachEntity(FSourceDoubleChannel DoubleChannel, FFrameTime FrameTime, double& OutResult)
	{
		if (!DoubleChannel.Source->Evaluate(FrameTime, OutResult))
		{
			OutResult = MIN_dbl;
		}
	}
};

} // namespace MovieScene
} // namespace UE

UMovieSceneBaseValueEvaluatorSystem::UMovieSceneBaseValueEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Instantiation;
	RelevantComponent = FBuiltInComponentTypes::Get()->BaseValueEvalTime;
	SystemCategories = EEntitySystemCategory::ChannelEvaluators;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->BoundObject);
	}
}

void UMovieSceneBaseValueEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// First, compute the base eval time in seconds, for those systems and tasks that need it.

	FAssignBaseValueEvalSeconds AssignBaseValueEvalSecondsTask(Linker->GetInstanceRegistry());
	FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(BuiltInComponents->BaseValueEvalTime)
	.Write(BuiltInComponents->BaseValueEvalSeconds)
	.RunInline_PerEntity(&Linker->EntityManager, AssignBaseValueEvalSecondsTask);

	// Second, evaluate the base values for all the core channel types.

	FEntityTaskBuilder()
	.Read(BuiltInComponents->ByteChannel)
	.Read(BuiltInComponents->BaseValueEvalTime)
	.Write(BuiltInComponents->BaseByte)
	.FilterAll({ BuiltInComponents->Tags.NeedsLink })
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.RunInline_PerEntity(&Linker->EntityManager, FEvaluateBaseByteValues());

	FEntityTaskBuilder()
	.Read(BuiltInComponents->IntegerChannel)
	.Read(BuiltInComponents->BaseValueEvalTime)
	.Write(BuiltInComponents->BaseInteger)
	.FilterAll({ BuiltInComponents->Tags.NeedsLink })
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.RunInline_PerEntity(&Linker->EntityManager, FEvaluateBaseIntegerValues());

	static_assert(
			UE_ARRAY_COUNT(BuiltInComponents->BaseDouble) == UE_ARRAY_COUNT(BuiltInComponents->FloatChannel),
			"There should be a matching number of float channels and float base values.");
	for (size_t Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->BaseDouble); ++Index)
	{
		const TComponentTypeID<double> BaseDouble = BuiltInComponents->BaseDouble[Index];
		const TComponentTypeID<FSourceFloatChannel> FloatChannel = BuiltInComponents->FloatChannel[Index];

		FEntityTaskBuilder()
		.Read(FloatChannel)
		.Read(BuiltInComponents->BaseValueEvalTime)
		.Write(BaseDouble)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.RunInline_PerEntity(&Linker->EntityManager, FEvaluateBaseFloatValues());
	}

	static_assert(
			UE_ARRAY_COUNT(BuiltInComponents->BaseDouble) == UE_ARRAY_COUNT(BuiltInComponents->DoubleChannel),
			"There should be a matching number of double channels and double base values.");
	for (size_t Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->BaseDouble); ++Index)
	{
		const TComponentTypeID<double> BaseDouble = BuiltInComponents->BaseDouble[Index];
		const TComponentTypeID<FSourceDoubleChannel> DoubleChannel = BuiltInComponents->DoubleChannel[Index];

		FEntityTaskBuilder()
		.Read(DoubleChannel)
		.Read(BuiltInComponents->BaseValueEvalTime)
		.Write(BaseDouble)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.RunInline_PerEntity(&Linker->EntityManager, FEvaluateBaseDoubleValues());
	}
}

