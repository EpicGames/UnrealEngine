// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneBoolPropertySystem.h"
#include "Systems/MovieScenePiecewiseBoolBlenderSystem.h"
#include "Systems/BoolChannelEvaluatorSystem.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBoolPropertySystem)

UMovieSceneBoolPropertySystem::UMovieSceneBoolPropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Bool);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UBoolChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieScenePiecewiseBoolBlenderSystem::StaticClass(), GetClass());

		DefineComponentConsumer(GetClass(), UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Bool.PropertyTag);
	}
}

void UMovieSceneBoolPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}


