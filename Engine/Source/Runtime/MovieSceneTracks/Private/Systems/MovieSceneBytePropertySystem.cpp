// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneBytePropertySystem.h"
#include "Systems/ByteChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseByteBlenderSystem.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBytePropertySystem)

UMovieSceneBytePropertySystem::UMovieSceneBytePropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Byte);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UByteChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieScenePiecewiseByteBlenderSystem::StaticClass(), GetClass());

		DefineComponentConsumer(GetClass(), UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Byte.PropertyTag);
	}
}

void UMovieSceneBytePropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}


