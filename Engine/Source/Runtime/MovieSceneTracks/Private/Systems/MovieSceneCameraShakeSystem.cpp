// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneCameraShakeSystem.h"

#include "Camera/CameraComponent.h"
#include "Camera/CameraModifier_CameraShake.h"
#include "Camera/CameraShakeSourceComponent.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "Evaluation/MovieSceneCameraShakePreviewer.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "IMovieScenePlayer.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneCameraShakeSection.h"
#include "UObject/Object.h"
#include "UObject/Package.h"

#if WITH_EDITOR
#include "Editor.h"
#include "LevelEditorViewport.h"
#endif  // WITH_EDITOR

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraShakeSystem)

namespace UE::MovieScene
{

TOptional<float> ComputeCameraShakeDurationOverride(const FMovieSceneContext& Context, const FMovieSceneCameraShakeComponentData& ShakeData)
{
	// Get the duration of the shake and compare it to the duration of the section, to know
	// if we need to override it. We only override duration if the section is shorter than
	// the shake's natural duration... i.e, we only make shakes shorter, not longer.
	const FFrameRate& FrameRate = Context.GetFrameRate();
	TSubclassOf<UCameraShakeBase> ShakeClass = ShakeData.SectionData.ShakeClass;

	FCameraShakeDuration ShakeDuration;
	UCameraShakeBase::GetCameraShakeDuration(ShakeClass, ShakeDuration);

	TOptional<float> DurationOverride;
	const FFrameTime SectionDurationFrames = (ShakeData.SectionEndTime - ShakeData.SectionStartTime);
	if (ShakeDuration.IsFixed())
	{
		const FFrameTime ShakeDurationFrames = Context.GetFrameRate().AsFrameTime(ShakeDuration.Get());
		if (ShakeDurationFrames > SectionDurationFrames)
		{
			DurationOverride = FrameRate.AsSeconds(SectionDurationFrames);
		}
	}
	else
	{
		DurationOverride = FrameRate.AsSeconds(SectionDurationFrames);
	}

	return DurationOverride;
}

struct FPreAnimatedCameraShakeTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType = FObjectKey;
	using StorageType = bool;

	bool CachePreAnimatedValue(UObject* InKey)
	{
		return true;
	}

	void RestorePreAnimatedValue(const FObjectKey& InKey, const bool Unused, const FRestoreStateParams& Params)
	{
		if (UCameraShakeBase* CameraShake = Cast<UCameraShakeBase>(InKey.ResolveObjectPtr()))
		{
			if (!CameraShake->IsFinished())
			{
				CameraShake->StopShake(true);
			}
			CameraShake->TeardownShake();
		}
	}
};

struct FPreAnimatedCameraComponentShakeTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType = FObjectKey;
	using StorageType = bool;

	bool CachePreAnimatedValue(UObject* InKey)
	{
		return true;
	}

	void RestorePreAnimatedValue(const FObjectKey& InKey, const bool Unused, const FRestoreStateParams& Params)
	{
		if (UCameraComponent* CameraComponent = Cast<UCameraComponent>(InKey.ResolveObjectPtr()))
		{
			CameraComponent->ClearAdditiveOffset();
			CameraComponent->ClearExtraPostProcessBlends();
		}
	}
};

struct FPreAnimatedCameraSourceShakeTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType = FObjectKey;
	using StorageType = bool;

	bool CachePreAnimatedValue(UObject* InKey)
	{
		return true;
	}

	void RestorePreAnimatedValue(const FObjectKey& InKey, const bool Unused, const FRestoreStateParams& Params)
	{
		if (UCameraShakeSourceComponent* ShakeSourceComponent = Cast<UCameraShakeSourceComponent>(InKey.ResolveObjectPtr()))
		{
			ShakeSourceComponent->StopAllCameraShakes(true);
#if WITH_EDITOR
			FCameraShakePreviewerLinkerExtension* PreviewerExtension = Params.Linker->FindExtension<FCameraShakePreviewerLinkerExtension>();
			if (PreviewerExtension)
			{
				if (TSharedPtr<FCameraShakePreviewer> Previewer = PreviewerExtension->FindPreviewer(Params.TerminalInstanceHandle))
				{
					Previewer->RemoveAllCameraShakesFromSource(ShakeSourceComponent);
				}
			}
#endif  // WITH_EDITOR
		}
	}
};

struct FPreAnimatedCameraShakeStateStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedCameraShakeTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedCameraShakeStateStorage> StorageID;
	FPreAnimatedStorageID GetStorageType() const override { return StorageID; }
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedCameraShakeStateStorage> FPreAnimatedCameraShakeStateStorage::StorageID;

struct FPreAnimatedCameraComponentShakeStateStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedCameraComponentShakeTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedCameraComponentShakeStateStorage> StorageID;
	FPreAnimatedStorageID GetStorageType() const override { return StorageID; }
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedCameraComponentShakeStateStorage> FPreAnimatedCameraComponentShakeStateStorage::StorageID;

struct FPreAnimatedCameraSourceShakeStateStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedCameraSourceShakeTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedCameraSourceShakeStateStorage> StorageID;
	FPreAnimatedStorageID GetStorageType() const override { return StorageID; }
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedCameraSourceShakeStateStorage> FPreAnimatedCameraSourceShakeStateStorage::StorageID;

#if WITH_EDITOR

TEntitySystemLinkerExtensionID<FCameraShakePreviewerLinkerExtension> FCameraShakePreviewerLinkerExtension::GetExtensionID()
{
	static TEntitySystemLinkerExtensionID<FCameraShakePreviewerLinkerExtension> ID = UMovieSceneEntitySystemLinker::RegisterExtension<FCameraShakePreviewerLinkerExtension>();
	return ID;
}

TSharedPtr<FCameraShakePreviewerLinkerExtension> FCameraShakePreviewerLinkerExtension::GetOrCreateExtension(UMovieSceneEntitySystemLinker* Linker)
{
	if (FCameraShakePreviewerLinkerExtension* PreviewerExtension = Linker->FindExtension<FCameraShakePreviewerLinkerExtension>())
	{
		return PreviewerExtension->AsShared();
	}

	TSharedPtr<FCameraShakePreviewerLinkerExtension> NewPreviewerExtension = MakeShared<FCameraShakePreviewerLinkerExtension>(Linker);
	Linker->AddExtension(NewPreviewerExtension.Get());
	return NewPreviewerExtension;
}

FCameraShakePreviewerLinkerExtension::FCameraShakePreviewerLinkerExtension(UMovieSceneEntitySystemLinker* Linker)
	: TSharedEntitySystemLinkerExtension(Linker)
{
}

FCameraShakePreviewerLinkerExtension::~FCameraShakePreviewerLinkerExtension()
{
	if (GEditor != nullptr)
	{
		GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
	}
	
	for (TPair<FInstanceHandle, TSharedRef<FCameraShakePreviewer>>& Pair : Previewers)
	{
		TSharedRef<FCameraShakePreviewer> Previewer = Pair.Value;
		Previewer->UnRegisterViewModifiers();
	}
	Previewers.Reset();
}

TSharedPtr<FCameraShakePreviewer> FCameraShakePreviewerLinkerExtension::FindPreviewer(FInstanceHandle InstanceHandle)
{
	if (TSharedRef<FCameraShakePreviewer>* FoundItem = Previewers.Find(InstanceHandle))
	{
		return FoundItem->ToSharedPtr();
	}
	return TSharedPtr<FCameraShakePreviewer>();
}

TSharedRef<FCameraShakePreviewer> FCameraShakePreviewerLinkerExtension::GetPreviewer(FInstanceHandle InstanceHandle)
{
	if (TSharedRef<FCameraShakePreviewer>* Previewer = Previewers.Find(InstanceHandle))
	{
		return *Previewer;
	}

	if (Previewers.IsEmpty() && GEditor != nullptr)
	{
		// This is our first previewer... let's start listening to viewports changing.
		GEditor->OnLevelViewportClientListChanged().AddSP(this, &FCameraShakePreviewerLinkerExtension::OnLevelViewportClientListChanged);
	}

	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandle);
	UObject* PlaybackContext = SequenceInstance.GetSharedPlaybackState()->GetPlaybackContext();
	UWorld* ContextWorld = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	TSharedRef<FCameraShakePreviewer> NewPreviewer = MakeShared<FCameraShakePreviewer>(ContextWorld);
	NewPreviewer->RegisterViewModifiers([ContextWorld](FLevelEditorViewportClient* LevelVC) -> bool
				{
					return LevelVC->AllowsCinematicControl() && LevelVC->GetWorld() == ContextWorld;
				},
				// Pass false to help the Mac compiler along.
				false);
	Previewers.Add(InstanceHandle, NewPreviewer);
	return NewPreviewer;
}

void FCameraShakePreviewerLinkerExtension::UpdateAllPreviewers()
{
	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	const TSparseArray<FSequenceInstance>& Instances = InstanceRegistry->GetSparseInstances();
	for (auto It = Instances.CreateConstIterator(); It; ++It)
	{
		FInstanceHandle InstanceHandle = It->GetInstanceHandle();
		if (TSharedRef<FCameraShakePreviewer>* FoundItem = Previewers.Find(InstanceHandle))
		{
			TSharedRef<FCameraShakePreviewer> Previewer(*FoundItem);
			const FMovieSceneContext& Context = It->GetContext();
			const float DeltaTime = Context.GetFrameRate().AsSeconds(Context.GetDelta());
			if (DeltaTime > 0.f)
			{
				const bool bIsPlaying = Context.GetStatus() == EMovieScenePlayerStatus::Playing;
				Previewer->Update(DeltaTime, bIsPlaying);
			}
			else
			{
				const float ScrubTime = Context.GetFrameRate().AsSeconds(Context.GetTime());
				Previewer->Scrub(ScrubTime);
			}
		}
	}
}

bool FCameraShakePreviewerLinkerExtension::HasAnyShake() const
{
	TArray<FActiveCameraShakeInfo> TempCameraShakes;
	for (const TPair<FInstanceHandle, TSharedRef<FCameraShakePreviewer>>& Pair : Previewers)
	{
		TSharedRef<FCameraShakePreviewer> Previewer(Pair.Value);
		if (Previewer->NumActiveCameraShakes() > 0)
		{
			return true;
		}
	}
	return false;
}

void FCameraShakePreviewerLinkerExtension::OnLevelViewportClientListChanged()
{
	// When viewports change, our shake previewers already correctly unregister from any removed viewport.
	// However, we need to automatically register any *new* viewport that fits our requirements.
	UMovieSceneEntitySystemLinker* Linker = WeakLinker.Get();
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	for (TPair<FInstanceHandle, TSharedRef<FCameraShakePreviewer>> Pair : Previewers)
	{
		const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(Pair.Key);
		UObject* PlaybackContext = SequenceInstance.GetSharedPlaybackState()->GetPlaybackContext();
		UWorld* ContextWorld = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

		TSharedRef<FCameraShakePreviewer> Previewer(Pair.Value);
		Previewer->RegisterViewModifiers(
				[ContextWorld](FLevelEditorViewportClient* LevelVC)
				{
					return LevelVC->AllowsCinematicControl() && LevelVC->GetWorld() == ContextWorld;
				},
				// Ignore duplicate registrations.
				true);
	}
}

#endif  // WITH_EDITOR

} // namespace UE::MovieScene

UMovieSceneCameraShakeInstantiatorSystem::UMovieSceneCameraShakeInstantiatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	RelevantComponent = FMovieSceneTracksComponentTypes::Get()->CameraShake;
	Phase = ESystemPhase::Instantiation;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);

		// Make sure our shakes aren't stopped/restored before we get a chance to transfer
		// them to a re-imported entity.
		DefineImplicitPrerequisite(GetClass(), UMovieSceneRestorePreAnimatedStateSystem::StaticClass());
	}
}

bool UMovieSceneCameraShakeInstantiatorSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	return TriggersByInstance.Num() > 0;
}

void UMovieSceneCameraShakeInstantiatorSystem::OnLink()
{
	using namespace UE::MovieScene;

	PreAnimatedCameraShakeStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedCameraShakeStateStorage>();
	PreAnimatedCameraComponentShakeStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedCameraComponentShakeStateStorage>();
	PreAnimatedCameraSourceShakeStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedCameraSourceShakeStateStorage>();

#if WITH_EDITOR
	if (GEditor != nullptr)
	{
		// We only need the previewer extension if there's an actual editor.
		PreviewerExtension = FCameraShakePreviewerLinkerExtension::GetOrCreateExtension(Linker);
	}

	// Handle camera shakes being recompiled.
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UMovieSceneCameraShakeInstantiatorSystem::OnObjectsReplaced);
#endif  // WITH_EDITOR
}

void UMovieSceneCameraShakeInstantiatorSystem::OnUnlink()
{
	using namespace UE::MovieScene;

#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);

	// Only the two camera shake systems hold pointers to the extension, so it should delete itself
	// once both systems are unlinked.
	PreviewerExtension = nullptr;
#endif  // WITH_EDITOR

	if (!ensure(TriggersByInstance.Num() == 0))
	{
		TriggersByInstance.Reset();
	}
}

void UMovieSceneCameraShakeInstantiatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityManager& EntityManager = Linker->EntityManager;
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	// Create camera shake instances for new shakes, and start them.
	auto VisitNewShakes = [this, BuiltInComponents, &EntityManager, InstanceRegistry](
		FMovieSceneEntityID EntityID,
		FInstanceHandle InstanceHandle,
		UObject* BoundObject,
		const FMovieSceneCameraShakeComponentData& ShakeData,
		FMovieSceneCameraShakeInstanceData& ShakeInstanceData)
	{
		const FSequenceInstance& Instance = InstanceRegistry->GetInstance(InstanceHandle);
		const FMovieSceneContext& Context = Instance.GetContext();
		UObject* PlaybackContext = Instance.GetSharedPlaybackState()->GetPlaybackContext();

		TSubclassOf<UCameraShakeBase> ShakeClass = ShakeData.SectionData.ShakeClass;
		UCameraShakeSourceComponent* ShakeSourceComponent = Cast<UCameraShakeSourceComponent>(BoundObject);

		if (ShakeClass.Get() == nullptr)
		{
			if (ShakeSourceComponent)
			{
				ShakeClass = ShakeSourceComponent->CameraShake;
			}
		}
		if (ShakeClass.Get() == nullptr)
		{
			return;
		}

		TOptional<float> DurationOverride = ComputeCameraShakeDurationOverride(Context, ShakeData);

		const bool bWantsRestoreState = EntityManager.HasComponent(EntityID, BuiltInComponents->Tags.RestoreState);
		const FRootInstanceHandle RootInstanceHandle = Instance.GetRootInstanceHandle();
		FCachePreAnimatedValueParams CacheParams;

		// Start playing the shake.
		if (ShakeSourceComponent)
		{
			bool bStartShake = true;
			if (ShakeInstanceData.SectionSignature == ShakeData.SectionSignature && ShakeInstanceData.ShakeInstance.Get() != nullptr && ShakeInstanceData.ShakeInstance->IsActive())
			{
				// Don't re-create and restart the shake if it was already running with the same
				// parameters and this instantiation phase is just re-importing other stuff.
				bStartShake = false;
			}

			PreAnimatedCameraSourceShakeStorage->BeginTrackingEntity(EntityID, bWantsRestoreState, RootInstanceHandle, ShakeSourceComponent);
			PreAnimatedCameraSourceShakeStorage->CachePreAnimatedValue(CacheParams, ShakeSourceComponent);

			if (bStartShake)
			{
				FCameraShakeSourceComponentStartParams ComponentParams;
				ComponentParams.ShakeClass = ShakeClass;
				ComponentParams.Scale = ShakeData.SectionData.PlayScale;
				ComponentParams.PlaySpace = ShakeData.SectionData.PlaySpace;
				ComponentParams.UserPlaySpaceRot = ShakeData.SectionData.UserDefinedPlaySpace;
				ComponentParams.DurationOverride = DurationOverride;

				ShakeSourceComponent->StartCameraShake(ComponentParams);
			}

			ShakeInstanceData.SectionSignature = ShakeData.SectionSignature;

#if WITH_EDITOR
			if (PreviewerExtension && bStartShake)
			{
				// Shake source components start shakes in the world, unlike the other shakes
				// (in the `else` clause) who directly affect the bound camera. This means that
				// the shake we have just started won't affect the Sequencer preview unless we
				// add some shaking ourselves. Let's do that here.
				// In fact, in the editor, the above StartCameraShake call generally does nothing
				// since there is no player controller.
				TSharedRef<FCameraShakePreviewer> Previewer = PreviewerExtension->GetPreviewer(InstanceHandle);

				FCameraShakePreviewerAddParams PreviewParams;
				PreviewParams.ShakeClass = ShakeClass;
				PreviewParams.GlobalStartTime = Context.GetFrameRate().AsSeconds(ShakeData.SectionStartTime);
				PreviewParams.SourceComponent = ShakeSourceComponent;
				PreviewParams.Scale = ShakeData.SectionData.PlayScale;
				PreviewParams.PlaySpace = ShakeData.SectionData.PlaySpace;
				PreviewParams.UserPlaySpaceRot = ShakeData.SectionData.UserDefinedPlaySpace;
				PreviewParams.DurationOverride = DurationOverride;

				// Stop any previous version of this shake.
				UCameraShakeBase* OldShakeInstance = ShakeInstanceData.ShakeInstance;
				if (OldShakeInstance)
				{
					Previewer->RemoveCameraShake(OldShakeInstance);
				}

				ShakeInstanceData.ShakeInstance = Previewer->AddCameraShake(PreviewParams);
				ShakeInstanceData.bManagedByPreviewer = true;
			}
#endif  // WITH_EDITOR
		}
		else if (UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromRuntimeObject(BoundObject))
		{
			bool bStartInstance = true;
			UCameraShakeBase* ShakeInstance = nullptr;
			if (ShakeInstanceData.SectionSignature == ShakeData.SectionSignature && ShakeInstanceData.ShakeInstance.Get() != nullptr && ShakeInstanceData.ShakeInstance->IsActive())
			{
				// Don't re-create and restart the shake if it was already running with the same
				// parameters and this instantiation phase is just re-importing other stuff.
				bStartInstance = false;
				ShakeInstance = ShakeInstanceData.ShakeInstance;
			}
			else
			{
				UObject* OuterObject = PlaybackContext ? PlaybackContext : GetTransientPackage();
				ShakeInstance = NewObject<UCameraShakeBase>(OuterObject, ShakeClass);
			}

			PreAnimatedCameraShakeStorage->BeginTrackingEntity(EntityID, bWantsRestoreState, RootInstanceHandle, ShakeInstance);
			PreAnimatedCameraShakeStorage->CachePreAnimatedValue(CacheParams, ShakeInstance);

			PreAnimatedCameraComponentShakeStorage->BeginTrackingEntity(EntityID, bWantsRestoreState, RootInstanceHandle, CameraComponent);
			PreAnimatedCameraComponentShakeStorage->CachePreAnimatedValue(CacheParams, CameraComponent);

			ShakeInstanceData.ShakeInstance = ShakeInstance;
			ShakeInstanceData.SectionSignature = ShakeData.SectionSignature;
			ShakeInstanceData.bManagedByPreviewer = false;

			if (bStartInstance)
			{
				FCameraShakeBaseStartParams ShakeParams;
				ShakeParams.Scale = ShakeData.SectionData.PlayScale;
				ShakeParams.PlaySpace = ShakeData.SectionData.PlaySpace;
				ShakeParams.UserPlaySpaceRot = ShakeData.SectionData.UserDefinedPlaySpace;
				ShakeParams.DurationOverride = DurationOverride;

				ShakeInstance->StartShake(ShakeParams);
			}
		}
	};

	FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(BuiltInComponents->BoundObject)
		.Read(TrackComponents->CameraShake)
		.Write(TrackComponents->CameraShakeInstance)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.Iterate_PerEntity(&Linker->EntityManager, VisitNewShakes);

	// We don't need to go over expired shakes from NeedsUnlink entities. Either these shakes
	// will be stopped be the pre-animated state, and then their component data will be freed
	// when the entities are deleted, or they come from KeepState sections and will therefore
	// continue running inside the camera manager.

	// Now trigger any one-shot shakes.
	if (TriggersByInstance.Num() > 0)
	{
		TriggerOneShotShakes();
	}
}

void UMovieSceneCameraShakeInstantiatorSystem::AddShakeTrigger(UE::MovieScene::FInstanceHandle InInstance, const FGuid& ObjectBindingID, const FFrameTime& InTime, const FMovieSceneCameraShakeSourceTrigger& InTrigger)
{
	TriggersByInstance.FindOrAdd(InInstance).Add(FTimedTrigger{ ObjectBindingID, InTime, InTrigger });
}

void UMovieSceneCameraShakeInstantiatorSystem::TriggerOneShotShakes()
{
	using namespace UE::MovieScene;

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	for (TPair<FInstanceHandle, TArray<FTimedTrigger>>& Pair : TriggersByInstance)
	{
		const FSequenceInstance& Instance = InstanceRegistry->GetInstance(Pair.Key);
		TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();

		const FMovieSceneContext& Context = Instance.GetContext();
		if (Context.GetDirection() != EPlayDirection::Forwards)
		{
			return;
		}

		for (const FTimedTrigger& Trigger : Pair.Value)
		{
			TArray<UCameraShakeSourceComponent*> ShakeSourceComponents;
			if (Trigger.ObjectBindingID.IsValid())
			{
				TArrayView<TWeakObjectPtr<>> BoundObjects = SharedPlaybackState->FindBoundObjects(Trigger.ObjectBindingID, Instance.GetSequenceID());
				for (TWeakObjectPtr<> WeakBoundObject : BoundObjects)
				{
					if (UObject* BoundObject = WeakBoundObject.Get())
					{
						if (UCameraShakeSourceComponent* ShakeSourceComponent = Cast<UCameraShakeSourceComponent>(BoundObject))
						{
							ShakeSourceComponents.Add(ShakeSourceComponent);
						}
					}
				}
			}

			const float TriggerTime = Context.GetFrameRate().AsSeconds(Trigger.Time);

			for (UCameraShakeSourceComponent* ShakeSourceComponent : ShakeSourceComponents)
			{
				TSubclassOf<UCameraShakeBase> ShakeClass = Trigger.Trigger.ShakeClass;
				if (ShakeClass.Get() == nullptr)
				{
					ShakeClass = ShakeSourceComponent->CameraShake;
				}

				if (ShakeClass.Get() != nullptr)
				{
					// Start playing the shake.
					ShakeSourceComponent->StartCameraShake(
						ShakeClass,
						Trigger.Trigger.PlayScale,
						Trigger.Trigger.PlaySpace,
						Trigger.Trigger.UserDefinedPlaySpace);

#if WITH_EDITOR
					if (PreviewerExtension)
					{
						// Also start playing the shake in our editor preview.
						TSharedRef<FCameraShakePreviewer> Previewer = PreviewerExtension->GetPreviewer(Pair.Key);

						FCameraShakePreviewerAddParams PreviewParams;
						PreviewParams.ShakeClass = ShakeClass;
						PreviewParams.GlobalStartTime = TriggerTime;
						PreviewParams.SourceComponent = ShakeSourceComponent;
						PreviewParams.Scale = Trigger.Trigger.PlayScale;
						PreviewParams.PlaySpace = Trigger.Trigger.PlaySpace;
						PreviewParams.UserPlaySpaceRot = Trigger.Trigger.UserDefinedPlaySpace;
						Previewer->AddCameraShake(PreviewParams);
					}
#endif  // WITH_EDITOR
				}
			}
		}
	}

#if WITH_EDITOR
	if (PreviewerExtension && !TriggersByInstance.IsEmpty())
	{
		// If we have just started new shakes, we might need to forcibly link the shake evaluator system
		// so that it can keep the shake previewer ticking every frame (in case that system isn't linked
		// because there are no shake sections active right now).
		// Once linked, this system will stay relevant and alive for as long as there are active shakes
		// in the previewer.
		Linker->LinkSystem<UMovieSceneCameraShakeEvaluatorSystem>();
	}
#endif

	TriggersByInstance.Empty();
}

#if WITH_EDITOR
void UMovieSceneCameraShakeInstantiatorSystem::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	auto ReplaceShakeInstances = [InstanceRegistry, &ReplacementMap](
			FInstanceHandle InstanceHandle,
			const FMovieSceneCameraShakeComponentData& ShakeData,
			FMovieSceneCameraShakeInstanceData& ShakeInstanceData)
	{
		if (UObject* const * NewShakeInstance = ReplacementMap.Find(ShakeInstanceData.ShakeInstance))
		{
			ShakeInstanceData.ShakeInstance = Cast<UCameraShakeBase>(*NewShakeInstance);
			
			// Restart the new shake instance, unless it's already managed by a previewer which
			// will do this already.
			if (ensure(ShakeInstanceData.ShakeInstance) && !ShakeInstanceData.bManagedByPreviewer)
			{
				const FMovieSceneContext& Context = InstanceRegistry->GetInstance(InstanceHandle).GetContext();
				TOptional<float> DurationOverride = ComputeCameraShakeDurationOverride(Context, ShakeData);

				FCameraShakeBaseStartParams ShakeParams;
				ShakeParams.Scale = ShakeData.SectionData.PlayScale;
				ShakeParams.PlaySpace = ShakeData.SectionData.PlaySpace;
				ShakeParams.UserPlaySpaceRot = ShakeData.SectionData.UserDefinedPlaySpace;
				ShakeParams.DurationOverride = DurationOverride;

				ShakeInstanceData.ShakeInstance->StartShake(ShakeParams);
			}
		}
	};

	FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(TrackComponents->CameraShake)
		.Write(TrackComponents->CameraShakeInstance)
		.Iterate_PerEntity(&Linker->EntityManager, ReplaceShakeInstances);
}
#endif

namespace UE::MovieScene
{

struct FAccumulatedShake
{
	void AccumulateOffset(const FTransform& InTransformOffset, float InFOVOffset)
	{
		TotalTransformOffset = TotalTransformOffset * InTransformOffset;
		TotalFOVOffset += InFOVOffset;
		bApplyTransform = true;
	}

	void AccumulatePostProcessing(const FPostProcessSettings& InPostProcessSettings, float InWeight)
	{
		PostProcessSettings.Add({ InPostProcessSettings, InWeight });
		bApplyPostProcessing = true;
	}

	void Apply(UCameraComponent* CameraComponent) const
	{
		if (bApplyTransform)
		{
			CameraComponent->ClearAdditiveOffset();
			CameraComponent->AddAdditiveOffset(TotalTransformOffset, TotalFOVOffset);
		}

		if (bApplyPostProcessing)
		{
			CameraComponent->ClearExtraPostProcessBlends();
			for (const TPair<FPostProcessSettings, float>& Pair : PostProcessSettings)
			{
				CameraComponent->AddExtraPostProcessBlend(Pair.Key, Pair.Value);
			}
		}
	}

private:
	bool bApplyTransform = false;
	bool bApplyPostProcessing = false;

	FTransform TotalTransformOffset;
	float TotalFOVOffset = 0.f;
	TArray<TTuple<FPostProcessSettings, float>, TInlineAllocator<2>> PostProcessSettings;
};

struct FEvaluateCameraShake
{
	const FInstanceRegistry* InstanceRegistry;
	TMap<UCameraComponent*, FAccumulatedShake> AccumulatedShakes;

	FEvaluateCameraShake(UMovieSceneEntitySystemLinker* InLinker)
		: InstanceRegistry(InLinker->GetInstanceRegistry())
	{
	}

	void ForEachAllocation(
			const FEntityAllocation* Allocation,
			TRead<FInstanceHandle> InstanceHandles,
			TRead<UObject*> BoundObjects,
			TRead<FMovieSceneCameraShakeComponentData> ShakeComponents,
			TRead<FMovieSceneCameraShakeInstanceData> ShakeInstances)
	{
		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			FInstanceHandle InstanceHandle = InstanceHandles[Index];
			const FSequenceInstance& Instance = InstanceRegistry->GetInstance(InstanceHandle);

			// Shakes should have been started by the instantiator system.
			//
			// We don't need to evaluate source components' camera shakes here, as they are ticking
			// along by themselves in both the player camera manager (in the game) and the camera shake
			// previewer (in the editor). We do however need to tick the camera shakes running directly
			// onto camera component bindings.
			const FMovieSceneCameraShakeComponentData& ShakeData = ShakeComponents[Index];
			const FMovieSceneCameraShakeInstanceData& ShakeInstance = ShakeInstances[Index];
			if (UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromRuntimeObject(BoundObjects[Index]))
			{
				EvaluateCameraComponentShake(CameraComponent, ShakeData, ShakeInstance, Instance);
			}
		}
	}

	void EvaluateCameraComponentShake(
			UCameraComponent* CameraComponent,
			const FMovieSceneCameraShakeComponentData& ShakeData,
			const FMovieSceneCameraShakeInstanceData& ShakeInstance,
			const FSequenceInstance& Instance)
	{
		FMinimalViewInfo POV;
		POV.Location = CameraComponent->GetComponentLocation();
		POV.Rotation = CameraComponent->GetComponentRotation();
		POV.FOV = CameraComponent->FieldOfView;

		// Update shake to the new time.
		const FMovieSceneContext& Context = Instance.GetContext();
		const FFrameTime NewShakeTime = Context.GetTime() - ShakeData.SectionStartTime;
		ShakeInstance.ShakeInstance->ScrubAndApplyCameraShake(NewShakeTime / Context.GetFrameRate(), 1.f, POV);

		// Grab transform and FOV changes.
		FTransform WorldToBaseCamera = CameraComponent->GetComponentToWorld().Inverse();
		float BaseFOV = CameraComponent->FieldOfView;
		FTransform NewCameraToWorld(POV.Rotation, POV.Location);
		float NewFOV = POV.FOV;

		FTransform NewCameraToBaseCamera = NewCameraToWorld * WorldToBaseCamera;

		float NewFOVToBaseFOV = BaseFOV - NewFOV;

		{
			// Accumumulate the offsets into the track data for application as part of the track execution token
			FAccumulatedShake& AccumulatedShake = AccumulatedShakes.FindOrAdd(CameraComponent);
			AccumulatedShake.AccumulateOffset(NewCameraToBaseCamera, NewFOVToBaseFOV);
		}

		// Grab post process changes.
		if (POV.PostProcessBlendWeight > 0.f)
		{
			FAccumulatedShake& AccumulatedShake = AccumulatedShakes.FindOrAdd(CameraComponent);
			AccumulatedShake.AccumulatePostProcessing(POV.PostProcessSettings, POV.PostProcessBlendWeight);
		}
	}

	void PostTask()
	{
		// Apply accumulated shakes.
		for (const TPair<UCameraComponent*, FAccumulatedShake>& Pair : AccumulatedShakes)
		{
			Pair.Value.Apply(Pair.Key);
		}
	}
};

} // namespace UE::MovieScene

UMovieSceneCameraShakeEvaluatorSystem::UMovieSceneCameraShakeEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	RelevantComponent = FMovieSceneTracksComponentTypes::Get()->CameraShake;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
	}
}

bool UMovieSceneCameraShakeEvaluatorSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

#if WITH_EDITOR
	if (FCameraShakePreviewerLinkerExtension* Extension = InLinker->FindExtension<FCameraShakePreviewerLinkerExtension>())
	{
		return Extension->HasAnyShake();
	}
#endif  // WITH_EDITOR
	return false;
}

void UMovieSceneCameraShakeEvaluatorSystem::OnLink()
{
	using namespace UE::MovieScene;

#if WITH_EDITOR
	PreviewerExtension = FCameraShakePreviewerLinkerExtension::GetOrCreateExtension(Linker);
#endif  // WITH_EDITOR
}

void UMovieSceneCameraShakeEvaluatorSystem::OnUnlink()
{
	using namespace UE::MovieScene;

#if WITH_EDITOR
	// Only the two camera shake systems hold pointers to the extension, so it should delete itself
	// once both systems are unlinked.
	PreviewerExtension = nullptr;
#endif  // WITH_EDITOR
}

void UMovieSceneCameraShakeEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(BuiltInComponents->BoundObject)
		.Read(TrackComponents->CameraShake)
		.Read(TrackComponents->CameraShakeInstance)
		.SetDesiredThread(Linker->EntityManager.GetGatherThread())
		.Dispatch_PerAllocation<FEvaluateCameraShake>(
			&Linker->EntityManager, InPrerequisites, &Subsequents, Linker);

#if WITH_EDITOR
	if (PreviewerExtension)
	{
		// The previewer only stores the delta time, and only computes shake results when the editor
		// later processes viewports. We can therefore safely do this in parallel with the shake
		// evaluation above.
		PreviewerExtension->UpdateAllPreviewers();
	}
#endif  // WITH_EDITOR
}

