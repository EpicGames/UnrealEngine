// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneColorPropertySystem.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneColorPropertySystem)


UMovieSceneColorPropertySystem::UMovieSceneColorPropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Color);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// We need our floats correctly evaluated and blended, so we are downstream from those systems.
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UFloatChannelEvaluatorSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneColorPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}



