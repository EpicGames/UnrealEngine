// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneComponentAttachmentSystem.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "Systems/MovieSceneComponentTransformSystem.h"
#include "Systems/MovieSceneComponentMobilitySystem.h"

#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneBoundSceneComponentInstantiator.h"
#include "EntitySystem/MovieSceneEntityGroupingSystem.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"

#include "PreAnimatedState/MovieScenePreAnimatedComponentTransformStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"

#include "GameFramework/Actor.h"

#include "MovieSceneTracksComponentTypes.h"
#include "MovieSceneObjectBindingID.h"
#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneComponentAttachmentSystem)

namespace UE
{
namespace MovieScene
{

struct FComponentAttachmentGroupingPolicy
{
	using GroupKeyType = FObjectKey;

	void InitializeGroupKeys(
		TEntityGroupingHandlerBase<FComponentAttachmentGroupingPolicy>& Handler,
		FEntityGroupBuilder* Builder,
		FEntityAllocationIteratorItem Item,
		FReadEntityIDs EntityIDs,
		TWrite<FEntityGroupID> GroupIDs,
		TRead<UObject*> BoundObjects,
		TRead<FAttachmentComponent> AttachComponents)
	{
		const FEntityAllocation* Allocation = Item.GetAllocation();
		for (int32 Index = 0; Index < Allocation->Num(); ++Index)
		{
			if (!GroupIDs[Index].IsValid())
			{
				GroupKeyType Key = FObjectKey(BoundObjects[Index]);
				const int32 NewGroupIndex = Handler.GetOrAllocateGroupIndex(Key, Builder);
				FEntityGroupID NewGroupID = Builder->MakeGroupID(NewGroupIndex);

				Builder->AddEntityToGroup(EntityIDs[Index], NewGroupID);

				GroupIDs[Index] = NewGroupID;
			}
		}
	}

#if WITH_EDITOR
	bool OnObjectsReplaced(GroupKeyType& InOutKey, const TMap<UObject*, UObject*>& ReplacementMap)
	{
		if (UObject* KeyObj = InOutKey.ResolveObjectPtr())
		{
			if (UObject* NewKeyObj = ReplacementMap.FindRef(KeyObj))
			{
				InOutKey = FObjectKey(NewKeyObj);
				return true;
			}
		}
		return false;
	}
#endif
};

struct FInitializeAttachParentsTask
{
	FInstanceRegistry* InstanceRegistry;

	void ForEachEntity(UE::MovieScene::FInstanceHandle InstanceHandle, const FMovieSceneObjectBindingID& BindingID, const UE::MovieScene::FAttachmentComponent& AttachComponent, TWeakObjectPtr<USceneComponent>& OutAttachedParent)
	{
		const FSequenceInstance& TargetInstance = InstanceRegistry->GetInstance(InstanceHandle);

		for (TWeakObjectPtr<> WeakObject : BindingID.ResolveBoundObjects(TargetInstance))
		{
			if (AActor* ParentActor = Cast<AActor>(WeakObject.Get()))
			{
				OutAttachedParent = AttachComponent.Destination.ResolveAttachment(ParentActor);

				// Can only ever be attached to one thing
				return;
			}
		}
	}
};

struct FAttachmentHandler
{
	UMovieSceneComponentAttachmentSystem* AttachmentSystem;
	FBuiltInComponentTypes* BuiltInComponents;
	FMovieSceneTracksComponentTypes* TrackComponents;
	FEntityManager* EntityManager;

	FAttachmentHandler(UMovieSceneComponentAttachmentSystem* InAttachmentSystem)
		: AttachmentSystem(InAttachmentSystem)
	{
		EntityManager = &InAttachmentSystem->GetLinker()->EntityManager;
		BuiltInComponents = FBuiltInComponentTypes::Get();
		TrackComponents = FMovieSceneTracksComponentTypes::Get();
	}

	void InitializeOutput(FObjectKey Object, TArrayView<const FMovieSceneEntityID> Inputs, FPreAnimAttachment* Output, FEntityOutputAggregate Aggregate)
	{
		USceneComponent* AttachChild = CastChecked<USceneComponent>(Object.ResolveObjectPtr());

		Output->OldAttachParent = AttachChild->GetAttachParent();
		Output->OldAttachSocket = AttachChild->GetAttachSocketName();

		UpdateOutput(Object, Inputs, Output, Aggregate);
	}

	void UpdateOutput(FObjectKey Object, TArrayView<const FMovieSceneEntityID> Inputs, FPreAnimAttachment* Output, FEntityOutputAggregate Aggregate)
	{
		USceneComponent* AttachChild = Cast<USceneComponent>(Object.ResolveObjectPtr());
		if (AttachChild == nullptr)
		{
			return;
		}

		for (FMovieSceneEntityID Entity : Inputs)
		{
			if (EntityManager->HasComponent(Entity, BuiltInComponents->Tags.Ignored))
			{
				continue;
			}

			TOptionalComponentReader<TWeakObjectPtr<USceneComponent>> AttachParentComponent = EntityManager->ReadComponent(Entity, TrackComponents->AttachParent);
			TOptionalComponentReader<FAttachmentComponent>            AttachmentComponent   = EntityManager->ReadComponent(Entity, TrackComponents->AttachComponent);
			if (AttachParentComponent && AttachmentComponent)
			{
				if (USceneComponent* AttachParent = AttachParentComponent->Get())
				{
					Output->DetachParams = AttachmentComponent->DetachParams;
					AttachmentComponent->AttachParams.ApplyAttach(AttachChild, AttachParent, AttachmentComponent->Destination.SocketName);

					// Can only be attached to one thing
					break;
				}
			}
		}
	}

	void DestroyOutput(FObjectKey ObjectKey, FPreAnimAttachment* Output, FEntityOutputAggregate Aggregate)
	{
		UObject* Object = ObjectKey.ResolveObjectPtr();
		if (Aggregate.bNeedsRestoration && Object != nullptr)
		{
			USceneComponent* AttachChild = CastChecked<USceneComponent>(Object);
			AttachmentSystem->AddPendingDetach(AttachChild, *Output);
		}
	}
};

struct FComponentAttachmentPreAnimatedTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType     = FObjectKey;
	using StorageType = FPreAnimAttachment;

	static FPreAnimAttachment CachePreAnimatedValue(UObject* InObject)
	{
		USceneComponent* SceneComponent = CastChecked<USceneComponent>(InObject);

		FPreAnimAttachment OutCachedAttachment;
		OutCachedAttachment.OldAttachParent = SceneComponent->GetAttachParent();
		OutCachedAttachment.OldAttachSocket = SceneComponent->GetAttachSocketName();
		return OutCachedAttachment;
	}
	static void RestorePreAnimatedValue(const FObjectKey& InKey, FPreAnimAttachment& InOutCachedAttachment, const FRestoreStateParams& Params)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(InKey.ResolveObjectPtr()))
		{
			InOutCachedAttachment.DetachParams.ApplyDetach(SceneComponent, InOutCachedAttachment.OldAttachParent.Get(), InOutCachedAttachment.OldAttachSocket);
		}
	}
};

struct FPreAnimatedComponentAttachmentStorage : TPreAnimatedStateStorage_ObjectTraits<FComponentAttachmentPreAnimatedTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedComponentAttachmentStorage> StorageID;
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedComponentAttachmentStorage> FPreAnimatedComponentAttachmentStorage::StorageID;

} // namespace MovieScene
} // namespace UE



UMovieSceneComponentAttachmentInvalidatorSystem::UMovieSceneComponentAttachmentInvalidatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	RelevantComponent = TrackComponents->AttachParentBinding;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(GetClass(), UMovieSceneGenericBoundObjectInstantiator::StaticClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneBoundSceneComponentInstantiator::StaticClass());
	}
}

void UMovieSceneComponentAttachmentInvalidatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	UnlinkStaleObjectBindings(FMovieSceneTracksComponentTypes::Get()->AttachParentBinding);
}

UMovieSceneComponentAttachmentSystem::UMovieSceneComponentAttachmentSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	RelevantComponent = TrackComponents->AttachParentBinding;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieScenePropertyInstantiatorSystem::StaticClass(), GetClass());

		DefineImplicitPrerequisite(UMovieSceneCachePreAnimatedStateSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneComponentMobilitySystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneComponentAttachmentInvalidatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneRestorePreAnimatedStateSystem::StaticClass());

		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);
		DefineComponentConsumer(GetClass(), BuiltInComponents->Tags.Ignored);
		DefineComponentConsumer(GetClass(), BuiltInComponents->SymbolicTags.CreatesEntities);
	}
}

void UMovieSceneComponentAttachmentSystem::OnLink()
{
	using namespace UE::MovieScene;

	UMovieSceneRestorePreAnimatedStateSystem* RestoreSystem = Linker->LinkSystem<UMovieSceneRestorePreAnimatedStateSystem>();
	Linker->SystemGraph.AddReference(this, RestoreSystem);

	UMovieSceneComponentAttachmentInvalidatorSystem* AttachmentInvalidator = Linker->LinkSystem<UMovieSceneComponentAttachmentInvalidatorSystem>();
	Linker->SystemGraph.AddReference(this, AttachmentInvalidator);

	AttachmentTracker.Initialize(this);

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->LinkSystem<UMovieSceneEntityGroupingSystem>();
	GroupingKey = GroupingSystem->AddGrouping(
			FComponentAttachmentGroupingPolicy(), BuiltInComponents->BoundObject, TracksComponents->AttachComponent);
}

void UMovieSceneComponentAttachmentSystem::OnUnlink()
{
	using namespace UE::MovieScene;

	AttachmentTracker.Destroy(FAttachmentHandler(this));

	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->FindSystem<UMovieSceneEntityGroupingSystem>();
	if (ensure(GroupingSystem))
	{
		GroupingSystem->RemoveGrouping(GroupingKey);
	}
	GroupingKey = FEntityGroupingPolicyKey();
}

void UMovieSceneComponentAttachmentSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	ensureMsgf(PendingAttachmentsToRestore.Num() == 0, TEXT("Pending attachments were not previously restored when they should have been"));

	FBuiltInComponentTypes*          Components      = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	// Step 1: Resolve attach parent bindings that need linking
	FInitializeAttachParentsTask InitAttachParents{ Linker->GetInstanceRegistry() };

	FEntityTaskBuilder()
	.Read(Components->InstanceHandle)
	.Read(TrackComponents->AttachParentBinding)
	.Read(TrackComponents->AttachComponent)
	.Write(TrackComponents->AttachParent)
	.FilterAll({ Components->Tags.NeedsLink })
	.FilterNone({ Components->Tags.Ignored })
	.RunInline_PerEntity(&Linker->EntityManager, InitAttachParents);

	// Step 2: Update all invalidated inputs and outputs for attachments
	FEntityComponentFilter Filter;
	Filter.All({ TrackComponents->AttachComponent });

	AttachmentTracker.UpdateFromComponents(Linker, Filter, FBuiltInComponentTypes::Get()->BoundObject);
	AttachmentTracker.ProcessInvalidatedOutputs(Linker, FAttachmentHandler(this));
}

void UMovieSceneComponentAttachmentSystem::AddPendingDetach(USceneComponent* SceneComponent, const UE::MovieScene::FPreAnimAttachment& Attachment)
{
	PendingAttachmentsToRestore.Add(MakeTuple(SceneComponent, Attachment));
}

void UMovieSceneComponentAttachmentSystem::SavePreAnimatedState(const FPreAnimationParameters& InParameters)
{
	using namespace UE::MovieScene;

	// Normal restore state attachments are tracked seprately due to the complexity of the detachment rules.
	// We track the transform and pre-attached parent using traditional pre-animated state
	if (!InParameters.CacheExtension->IsCapturingGlobalState())
	{
		return;
	}

	FMovieSceneTracksComponentTypes* TrackComponents   = FMovieSceneTracksComponentTypes::Get();
	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();

	FComponentMask FilterMask({ TrackComponents->AttachParent });
	if (!InParameters.CacheExtension->AreEntriesInvalidated())
	{
		FilterMask.Set(BuiltInComponents->Tags.NeedsLink);
	}

	FComponentMask ExcludeMask({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->Tags.Finished, BuiltInComponents->Tags.Ignored });

	// Attachments change transforms
	FPreAnimatedEntityCaptureSource* EntityMetaData = InParameters.CacheExtension->GetOrCreateEntityMetaData();
	TSharedPtr<FPreAnimatedComponentTransformStorage>  ComponentTransformStorage  = InParameters.CacheExtension->GetOrCreateStorage<FPreAnimatedComponentTransformStorage>();
	TSharedPtr<FPreAnimatedComponentAttachmentStorage> ComponentAttachmentStorage = InParameters.CacheExtension->GetOrCreateStorage<FPreAnimatedComponentAttachmentStorage>();

	// Start tracking all attachments to the component transform storage
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->BoundObject)
	.Read(TrackComponents->AttachComponent)
	.FilterAll(FilterMask)
	.FilterNone(ExcludeMask)
	.Iterate_PerAllocation(&Linker->EntityManager,
		[EntityMetaData, ComponentTransformStorage, ComponentAttachmentStorage](FEntityAllocationIteratorItem Item, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> InstanceHandles, TRead<UObject*> BoundObjects, TRead<FAttachmentComponent> AttachComponents)
		{
			TArrayView<UObject* const> BoundObjectArray = BoundObjects.AsArray(Item.GetAllocation()->Num());

			auto ShouldCacheTransform = [AttachComponents](int32 Index)
			{
				FAttachmentComponent Component = AttachComponents[Index];
				// Only cache persistent preanimated transforms if we're not using a keep relative attachment
				return Component.AttachParams.AttachmentLocationRule != EAttachmentRule::KeepRelative
					|| Component.AttachParams.AttachmentRotationRule != EAttachmentRule::KeepRelative
					|| Component.AttachParams.AttachmentScaleRule    != EAttachmentRule::KeepRelative
					|| Component.DetachParams.DetachmentLocationRule != EDetachmentRule::KeepRelative
					|| Component.DetachParams.DetachmentRotationRule != EDetachmentRule::KeepRelative
					|| Component.DetachParams.DetachmentScaleRule    != EDetachmentRule::KeepRelative;
			};

			// Order is important here - always cache the transforms first so that they are restored last

			// If we're capturing global state for an attached parent, we forcibly persist the transform so that it
			// _always_ remains cached. This ensures that the transform definitely gets restored to the pre-animated
			// state regardless of the detach rules used during normal playback.
			FCachePreAnimatedValueParams ForcePersistParams;
			ForcePersistParams.bForcePersist = true;
			ComponentTransformStorage->CachePreAnimatedTransforms(ForcePersistParams, BoundObjectArray, TFunctionRef<bool(int32)>(ShouldCacheTransform));

			FPreAnimatedTrackerParams AttachmentParams(Item);
			AttachmentParams.bWantsRestoreState = false;

			ComponentAttachmentStorage->BeginTrackingEntities(AttachmentParams, EntityIDs, InstanceHandles, BoundObjects);
			ComponentAttachmentStorage->CachePreAnimatedValues(FCachePreAnimatedValueParams(), BoundObjectArray);
		}
	);
}

void UMovieSceneComponentAttachmentSystem::RestorePreAnimatedState(const FPreAnimationParameters& InParameters)
{
	using namespace UE::MovieScene;

	FMovieSceneTracksComponentTypes* TrackComponents   = FMovieSceneTracksComponentTypes::Get();
	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();

	// Apply detachments
	for (TTuple<USceneComponent*, FPreAnimAttachment> Pair : PendingAttachmentsToRestore)
	{
		Pair.Value.DetachParams.ApplyDetach(Pair.Key, Pair.Value.OldAttachParent.Get(), Pair.Value.OldAttachSocket);
	}
	PendingAttachmentsToRestore.Empty();

	FPreAnimatedEntityCaptureSource* EntityMetaData = InParameters.CacheExtension->GetEntityMetaData();
	if (InParameters.CacheExtension->IsCapturingGlobalState() && EntityMetaData)
	{
		// Stop tracking the forcibly 'keep-state' entities that are tracking the attachment parent
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.FilterAll({ TrackComponents->AttachParent, BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerEntity(&Linker->EntityManager,
			[EntityMetaData](FMovieSceneEntityID EntityID)
			{
				EntityMetaData->StopTrackingEntity(EntityID, FPreAnimatedComponentAttachmentStorage::StorageID);
			}
		);
	}
}

