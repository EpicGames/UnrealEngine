// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneComponentMaterialSystem.h"

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

#include "EntitySystem/MovieSceneEntityMutations.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/MovieSceneHierarchicalBiasSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#include "Materials/MaterialInstanceDynamic.h"
#include "Tracks/MovieSceneMaterialTrack.h"

#include "Components/PrimitiveComponent.h"
#include "Components/DecalComponent.h"
#include "Components/MeshComponent.h"
#include "Components/VolumetricCloudComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneComponentMaterialSystem)

namespace UE::MovieScene
{

FComponentMaterialAccessor::FComponentMaterialAccessor(const FComponentMaterialKey& InKey)
	: Object(InKey.Object.ResolveObjectPtr())
	, MaterialInfo(InKey.MaterialInfo)
{}

FComponentMaterialAccessor::FComponentMaterialAccessor(UObject* InObject, const FComponentMaterialInfo& InMaterialInfo)
	: Object(InObject)
	, MaterialInfo(InMaterialInfo)
{}

FComponentMaterialAccessor::operator bool() const
{
	return Object != nullptr;
}

FString FComponentMaterialAccessor::ToString() const
{
	return FString::Printf(TEXT("object %s (material info: %s)"), *Object->GetPathName(), *MaterialInfo.ToString());
}

UMaterialInterface* FComponentMaterialAccessor::GetMaterial() const
{
	switch (MaterialInfo.MaterialType)
	{
	case EComponentMaterialType::Empty:
		break;
	case EComponentMaterialType::IndexedMaterial:
		if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Object))
		{
			UMaterialInterface* Material = nullptr;
			if (!MaterialInfo.MaterialSlotName.IsNone())
			{
				Material = Component->GetMaterialByName(MaterialInfo.MaterialSlotName);
			}
			if (!Material)
			{
				Material = Component->GetMaterial(MaterialInfo.MaterialSlotIndex);
			}
			return Material;
		}
		break;
	case EComponentMaterialType::OverlayMaterial:
		if (UMeshComponent* Component = Cast<UMeshComponent>(Object))
		{
			return Component->GetOverlayMaterial();
		}
		break;
	case EComponentMaterialType::DecalMaterial:
		if (UDecalComponent* DecalComponent = Cast<UDecalComponent>(Object))
		{
			return DecalComponent->GetDecalMaterial();
		}
		break;
	case EComponentMaterialType::VolumetricCloudMaterial:
		if (UVolumetricCloudComponent* CloudComponent = Cast<UVolumetricCloudComponent>(Object))
		{
			return CloudComponent->GetMaterial();
		}
		break;
	case EComponentMaterialType::SlotOverlayMaterial:
		if (UMeshComponent* Component = Cast<UMeshComponent>(Object))
		{
			return Component->GetOverlayMaterial(true, MaterialInfo.MaterialSlotIndex);
		}
		break;
	default:
		break;
	}
	return nullptr;
}

void FComponentMaterialAccessor::SetMaterial(UMaterialInterface* InMaterial) const
{
	switch (MaterialInfo.MaterialType)
	{
	case EComponentMaterialType::Empty:
		break;
	case EComponentMaterialType::IndexedMaterial:
		if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Object))
		{
			TArray<FName> MaterialSlots = Component->GetMaterialSlotNames();
			if (!MaterialInfo.MaterialSlotName.IsNone() && MaterialSlots.Contains(MaterialInfo.MaterialSlotName))
			{
				Component->SetMaterialByName(MaterialInfo.MaterialSlotName, InMaterial);
			}
			else
			{
				Component->SetMaterial(MaterialInfo.MaterialSlotIndex, InMaterial);
			}
		}
		break;
	case EComponentMaterialType::OverlayMaterial:
		if (UMeshComponent* Component = Cast<UMeshComponent>(Object))
		{
			 Component->SetOverlayMaterial(InMaterial);
		}
		break;
	case EComponentMaterialType::DecalMaterial:
		if (UDecalComponent* DecalComponent = Cast<UDecalComponent>(Object))
		{
			DecalComponent->SetDecalMaterial(InMaterial);
		}
		break;
	case EComponentMaterialType::VolumetricCloudMaterial:
		if (UVolumetricCloudComponent* CloudComponent = Cast<UVolumetricCloudComponent>(Object))
		{
			CloudComponent->SetMaterial(InMaterial);
		}
		break;
	case EComponentMaterialType::SlotOverlayMaterial:
		if (UMeshComponent* Component = Cast<UMeshComponent>(Object))
		{
			Component->SetOverlayMaterial(InMaterial, true, MaterialInfo.MaterialSlotIndex);
		}
		break;
	default:
		break;
	}
}

UMaterialInstanceDynamic* FComponentMaterialAccessor::CreateDynamicMaterial(UMaterialInterface* InMaterial)
{
	auto MakeDynamicMaterial = [this, InMaterial]()
	{
		TStringBuilder<128> DynamicName;
		InMaterial->GetFName().ToString(DynamicName);
		DynamicName.Append(TEXT("_Animated"));
		FName UniqueDynamicName = MakeUniqueObjectName(Object, UMaterialInstanceDynamic::StaticClass(), DynamicName.ToString());
		return UMaterialInstanceDynamic::Create(InMaterial, Object, UniqueDynamicName);
	};
	// Need to create a new MID, either because the parent has changed, or because one doesn't already exist
	switch (MaterialInfo.MaterialType)
	{
	case EComponentMaterialType::Empty:
		break;
	case EComponentMaterialType::IndexedMaterial:
		if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(Object))
		{
			TArray<FName> MaterialSlots = Component->GetMaterialSlotNames();
			UMaterialInstanceDynamic* Result = MakeDynamicMaterial();
			if (!MaterialInfo.MaterialSlotName.IsNone() && MaterialSlots.Contains(MaterialInfo.MaterialSlotName))
			{
				Component->SetMaterialByName(MaterialInfo.MaterialSlotName, Result);
			}
			else
			{
				Component->SetMaterial(MaterialInfo.MaterialSlotIndex, Result);
			}
			return Result;
		}
		break;
	case EComponentMaterialType::OverlayMaterial:
		if (UMeshComponent* Component = Cast<UMeshComponent>(Object))
		{
			UMaterialInstanceDynamic* Result = MakeDynamicMaterial();
			Component->SetOverlayMaterial(Result);
			return Result;
		}
		break;
	case EComponentMaterialType::DecalMaterial:
		if (UDecalComponent* DecalComponent = Cast<UDecalComponent>(Object))
		{
			return DecalComponent->CreateDynamicMaterialInstance();
		}
		break;
	case EComponentMaterialType::VolumetricCloudMaterial:
		if (UVolumetricCloudComponent* CloudComponent = Cast<UVolumetricCloudComponent>(Object))
		{
			UMaterialInstanceDynamic* Result = MakeDynamicMaterial();
			CloudComponent->SetMaterial(Result);
			return Result;
		}
		break;
	case EComponentMaterialType::SlotOverlayMaterial:
		if (UMeshComponent* Component = Cast<UMeshComponent>(Object))
		{
			UMaterialInstanceDynamic* Result = MakeDynamicMaterial();
			Component->SetOverlayMaterial(Result, true, MaterialInfo.MaterialSlotIndex);
			return Result;
		}
		break;
	default:
		break;
	}

	return nullptr;
}

TAutoRegisterPreAnimatedStorageID<FPreAnimatedComponentMaterialSwitcherStorage> FPreAnimatedComponentMaterialSwitcherStorage::StorageID;
TAutoRegisterPreAnimatedStorageID<FPreAnimatedComponentMaterialParameterStorage> FPreAnimatedComponentMaterialParameterStorage::StorageID;

} // namespace UE::MovieScene

UMovieSceneComponentMaterialSystem::UMovieSceneComponentMaterialSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	RelevantComponent = TracksComponents->ComponentMaterialInfo;
	Phase = ESystemPhase::Instantiation;


	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), BuiltInComponents->ObjectResult);
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);

		DefineComponentProducer(GetClass(), TracksComponents->BoundMaterial);

		DefineImplicitPrerequisite(UMovieSceneCachePreAnimatedStateSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneHierarchicalBiasSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneComponentMaterialSystem::OnLink()
{
	using namespace UE::MovieScene;

	SystemImpl.MaterialSwitcherStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedComponentMaterialSwitcherStorage>();
	SystemImpl.MaterialParameterStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedComponentMaterialParameterStorage>();

	SystemImpl.OnLink(Linker, FBuiltInComponentTypes::Get()->BoundObject, FMovieSceneTracksComponentTypes::Get()->ComponentMaterialInfo);
}

void UMovieSceneComponentMaterialSystem::OnUnlink()
{
	SystemImpl.OnUnlink(Linker);
}

void UMovieSceneComponentMaterialSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	SystemImpl.OnRun(Linker, FBuiltInComponentTypes::Get()->BoundObject, FMovieSceneTracksComponentTypes::Get()->ComponentMaterialInfo, InPrerequisites, Subsequents);
}

void UMovieSceneComponentMaterialSystem::SavePreAnimatedState(const FPreAnimationParameters& InParameters)
{
	using namespace UE::MovieScene;

	SystemImpl.SavePreAnimatedState(Linker, FBuiltInComponentTypes::Get()->BoundObject, FMovieSceneTracksComponentTypes::Get()->ComponentMaterialInfo, InParameters);
}
