// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneComponentMobilitySystem.h"
#include "MovieSceneTracksComponentTypes.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneExecutionToken.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectGroupManager.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"

#include "Systems/MovieSceneComponentTransformSystem.h"
#include "Systems/MovieScenePropertyInstantiator.h"

#include "Components/SceneComponent.h"
#include "Evaluation/MovieSceneTemplateCommon.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneComponentMobilitySystem)

namespace UE
{
namespace MovieScene
{

TAutoRegisterPreAnimatedStorageID<FPreAnimatedComponentMobilityStorage> FPreAnimatedComponentMobilityStorage::StorageID;

struct FMobilityCacheHandler
{
	UMovieSceneComponentMobilitySystem* System;

	FMobilityCacheHandler(UMovieSceneComponentMobilitySystem* InSystem)
		: System(InSystem)
	{}

	void InitializeOutput(FObjectKey Object, TArrayView<const FMovieSceneEntityID> Inputs, EComponentMobility::Type* OutMobility, FEntityOutputAggregate Aggregate)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Object.ResolveObjectPtr()))
		{
			*OutMobility = SceneComponent->Mobility;
			SceneComponent->SetMobility(EComponentMobility::Movable);
		}
	}

	static void UpdateOutput(FObjectKey Object, TArrayView<const FMovieSceneEntityID> Inputs, EComponentMobility::Type* OutMobility, FEntityOutputAggregate Aggregate)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(Object.ResolveObjectPtr()))
		{
			SceneComponent->SetMobility(EComponentMobility::Movable);
		}
	}

	void DestroyOutput(FObjectKey Object, EComponentMobility::Type* Output, FEntityOutputAggregate Aggregate)
	{
	}
};

/**
 * Gets the flattened depth-first hierarchy of the scene component.
 */
static void GetFlattenedHierarchy(USceneComponent* SceneComponent, TArray<USceneComponent*, TInlineAllocator<4>>& OutFlatHierarchy)
{
	TArray<USceneComponent*, TInlineAllocator<4>> ChildrenStack;
	ChildrenStack.Push(SceneComponent);
	while (ChildrenStack.Num() > 0)
	{
		USceneComponent* Child(ChildrenStack.Pop());

		OutFlatHierarchy.Add(Child);

		const TArray<USceneComponent*>& ChildAttachChildren = Child->GetAttachChildren();
		for (int32 Index = ChildAttachChildren.Num() - 1; Index >= 0; --Index)
		{
			USceneComponent* GrandChild = ChildAttachChildren[Index];
			if (GrandChild)
			{
				ChildrenStack.Add(GrandChild);
			}
		}
	}
}

FPreAnimatedStateEntry FPreAnimatedComponentMobilityStorage::FindEntry(USceneComponent* InSceneComponent)
{
	FPreAnimatedStorageGroupHandle GroupHandle = this->Traits.ObjectGroupManager->FindGroupForKey(InSceneComponent);
	FPreAnimatedStorageIndex       StorageIndex = FindStorageIndex(InSceneComponent);

	return FPreAnimatedStateEntry{ GroupHandle, FPreAnimatedStateCachedValueHandle{ StorageID, StorageIndex } };
}

FPreAnimatedStateEntry FPreAnimatedComponentMobilityStorage::MakeEntry(USceneComponent* InSceneComponent)
{
	FPreAnimatedStorageGroupHandle GroupHandle  = this->Traits.ObjectGroupManager->MakeGroupForKey(InSceneComponent);
	FPreAnimatedStorageIndex       StorageIndex = GetOrCreateStorageIndex(InSceneComponent);

	return FPreAnimatedStateEntry{ GroupHandle, FPreAnimatedStateCachedValueHandle{ StorageID, StorageIndex } };
}

void FPreAnimatedMobilityTraits::RestorePreAnimatedValue(const FObjectKey& InKey, EComponentMobility::Type Mobility, const FRestoreStateParams& Params)
{
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(InKey.ResolveObjectPtr()))
	{
		SceneComponent->SetMobility(Mobility);
	}
}

EComponentMobility::Type FPreAnimatedMobilityTraits::CachePreAnimatedValue(UObject* InObject)
{
	EComponentMobility::Type OutMobility = EComponentMobility::Static;
	if (USceneComponent* SceneComponent = Cast<USceneComponent>(InObject))
	{
		OutMobility = SceneComponent->Mobility;
	}
	return OutMobility;
}

} // namespace MovieScene
} // namespace UE


UMovieSceneComponentMobilitySystem::UMovieSceneComponentMobilitySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents   = FMovieSceneTracksComponentTypes::Get();

	// Anything with a component transform or attach component neds to have its mobility pre-set to Moveable
	Filter.Any({ TrackComponents->ComponentTransform.PropertyTag, TrackComponents->AttachParent });

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneCachePreAnimatedStateSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneRestorePreAnimatedStateSystem::StaticClass());

		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->SymbolicTags.CreatesEntities);
	}
}

bool UMovieSceneComponentMobilitySystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	return InLinker->EntityManager.Contains(Filter);
}

void UMovieSceneComponentMobilitySystem::OnLink()
{
	MobilityTracker.Initialize(this);
}

void UMovieSceneComponentMobilitySystem::OnUnlink()
{
	using namespace UE::MovieScene;

	if (!ensure(PendingMobilitiesToRestore.Num() == 0))
	{
		PendingMobilitiesToRestore.Reset();
	}

	// Destroy everything
	MobilityTracker.Destroy(FMobilityCacheHandler(this));
}

void UMovieSceneComponentMobilitySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	// Update the mobility tracker, caching preanimated mobilities and assigning everything as moveable that needs it
	MobilityTracker.UpdateFromComponents(Linker, Filter, FBuiltInComponentTypes::Get()->BoundObject);
	MobilityTracker.ProcessInvalidatedOutputs(Linker, FMobilityCacheHandler(this));
}

void UMovieSceneComponentMobilitySystem::SavePreAnimatedState(const FPreAnimationParameters& InParameters)
{
	using namespace UE::MovieScene;

	FPreAnimatedStateExtension* Extension = InParameters.CacheExtension;

	auto IterNewObjects = [Extension](FEntityAllocationIteratorItem Item, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> InstanceHandles, TRead<UObject*> BoundObjects)
	{
		TSharedPtr<FPreAnimatedComponentMobilityStorage> ComponentMobilityStorage
			= Extension->GetOrCreateStorage<FPreAnimatedComponentMobilityStorage>();
		FPreAnimatedEntityCaptureSource* EntityMetaData = Extension->GetOrCreateEntityMetaData();

		const FEntityAllocation* Allocation = Item.GetAllocation();
		const bool bWantsRestore = Item.GetAllocationType().Contains(FBuiltInComponentTypes::Get()->Tags.RestoreState);

		FCachePreAnimatedValueParams CacheValueParams;
		CacheValueParams.bForcePersist = Item.GetAllocationType().Contains(FMovieSceneTracksComponentTypes::Get()->AttachParent);

		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			USceneComponent* SceneComponent = Cast<USceneComponent>(BoundObjects[Index]);
			if (!SceneComponent)
			{
				continue;
			}

			FMovieSceneEntityID EntityID       = EntityIDs[Index];
			FRootInstanceHandle InstanceHandle = InstanceHandles[Index];

			TArray<USceneComponent*, TInlineAllocator<4>> FlatHierarchy;
			GetFlattenedHierarchy(SceneComponent, FlatHierarchy);

			for (USceneComponent* CurrentSceneComponent : FlatHierarchy)
			{
				FPreAnimatedStateEntry Entry = ComponentMobilityStorage->MakeEntry(CurrentSceneComponent);
				
				// Always track entities, even KeepState ones, because we don't want to restore the original mobility if there
				// are still contributors animating the transform.
				EntityMetaData->BeginTrackingEntity(Entry, EntityID, InstanceHandle, bWantsRestore);
				ComponentMobilityStorage->CachePreAnimatedValue(CacheValueParams, Entry, CurrentSceneComponent);
			}
		}
	};

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents   = FMovieSceneTracksComponentTypes::Get();

	FComponentMask RestrictiveMask;
	if (!InParameters.CacheExtension->AreEntriesInvalidated())
	{
		RestrictiveMask.Set(BuiltInComponents->Tags.NeedsLink);
	}

	FComponentMask ExcludeMask({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->Tags.Finished, BuiltInComponents->Tags.Ignored });

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->BoundObject)
	.FilterAll(RestrictiveMask)
	.FilterAny({ TrackComponents->ComponentTransform.PropertyTag, TrackComponents->AttachParent })
	.FilterNone(ExcludeMask)
	.Iterate_PerAllocation(&Linker->EntityManager, IterNewObjects);
}

void UMovieSceneComponentMobilitySystem::RestorePreAnimatedState(const FPreAnimationParameters& InParameters)
{

}

