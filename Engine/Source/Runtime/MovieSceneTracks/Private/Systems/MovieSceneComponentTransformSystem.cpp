// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneComponentTransformSystem.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "Systems/MovieSceneQuaternionInterpolationRotationSystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"

#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneComponentTransformSystem)

UMovieSceneComponentTransformSystem::UMovieSceneComponentTransformSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	// This system can be used for interrogation
	SystemCategories &= ~FSystemInterrogator::GetExcludedFromInterrogationCategory();

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	BindToProperty(TrackComponents->ComponentTransform);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());

		DefineComponentConsumer(GetClass(), TrackComponents->ComponentTransform.PropertyTag);

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
		{
			DefineComponentConsumer(GetClass(), BuiltInComponents->DoubleResult[Index]);
		}
	}
}

void UMovieSceneComponentTransformSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}

