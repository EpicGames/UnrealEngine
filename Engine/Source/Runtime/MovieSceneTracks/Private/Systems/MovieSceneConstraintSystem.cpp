// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneConstraintSystem.h"
#include "ConstraintsManager.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneBoundSceneComponentInstantiator.h"

#include "PreAnimatedState/MovieScenePreAnimatedComponentTransformStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/MovieSceneComponentTransformSystem.h"
#include "Transform/TransformConstraint.h"
#include "Transform/TransformableHandle.h"
#include "Transform/TransformableHandleUtils.h"
#include "Sections/MovieSceneConstrainedSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneConstraintSystem)


static UTickableConstraint* CreateConstraintIfNeeded(UWorld* InWorld, const FConstraintAndActiveChannel* InConstraintAndActiveChannel)
{
	UTickableConstraint* Constraint = InConstraintAndActiveChannel->GetConstraint();
	if (!Constraint) //if constraint doesn't exist it probably got unspawned so recreate it and add it
	{
		return nullptr;
	}
	
	// it's possible that we have it but it's not in the manager, due to manager not being saved with it (due to spawning or undo/redo).
	if (!FConstraintsManagerController::Get(InWorld).GetConstraint(Constraint->ConstraintID))
	{
		FConstraintsManagerController::Get(InWorld).AddConstraint(Constraint);
	}
	
	return Constraint;
}



namespace UE::MovieScene
{

struct FPreAnimatedConstraint
{
	FGuid ConstraintID;
	bool bDeleteIt;
	bool bPreviouslyEnabled;
	TWeakObjectPtr<USceneComponent> SceneComponent;
	TWeakObjectPtr<UMovieScene3DTransformSection> Section;
};

struct FPreAnimatedConstraintTraits : FBoundObjectPreAnimatedStateTraits
{
	/** Key type that stores pre-animated state associated with the object and its constraint name */
	struct FKeyType
	{
		FObjectKey Object;
		FGuid ConstraintID;

		/** Constructor that takes a BoundObject and ConstraintChannel component */
		FKeyType(UObject* InObject, const FConstraintComponentData& ComponentData)
			: Object(InObject)
			, ConstraintID(ComponentData.ConstraintID)
		{}

		/** Hashing and equality required for storage within a map */
		friend uint32 GetTypeHash(const FKeyType& InKey)
		{
			return HashCombine(GetTypeHash(InKey.Object), GetTypeHash(InKey.ConstraintID));
		}
		friend bool operator==(const FKeyType& A, const FKeyType& B)
		{
			return A.Object == B.Object && A.ConstraintID == B.ConstraintID;
		}
	};

	using KeyType = FKeyType;
	using StorageType = FPreAnimatedConstraint;

	static FPreAnimatedConstraint CachePreAnimatedValue(UObject* InBoundObject, const FConstraintComponentData& ConstraintData)
	{
		USceneComponent* SceneComponent = CastChecked<USceneComponent>(InBoundObject);
		UWorld* World = SceneComponent->GetWorld();
		if (FConstraintAndActiveChannel* ConstraintAndActiveChannel = ConstraintData.Section->GetConstraintChannel(ConstraintData.ConstraintID))
		{
			UTickableConstraint* Constraint = ConstraintAndActiveChannel->GetConstraint().Get();
			constexpr bool bIsActive = false; //we set static mesh constraints to be inactive to matchup how CR's work in not being valid
			const FGuid ID = Constraint ? Constraint->ConstraintID : ConstraintData.ConstraintID;
			//also need to create it
			Constraint = CreateConstraintIfNeeded(World, ConstraintAndActiveChannel);
			
			return FPreAnimatedConstraint{ ID, (Constraint == nullptr), bIsActive, SceneComponent, ConstraintData.Section };
		}
		return FPreAnimatedConstraint{ ConstraintData.ConstraintID, false,false, SceneComponent, ConstraintData.Section };
	}

	static void RestorePreAnimatedValue(const FKeyType& InKey, const FPreAnimatedConstraint& OldValue, const FRestoreStateParams& Params)
	{
		if (USceneComponent* SceneComponent = OldValue.SceneComponent.Get())
		{
			if (OldValue.Section.IsValid() == false)
			{
				return;
			}
			if (FConstraintAndActiveChannel* ConstraintAndActiveChannel = OldValue.Section->GetConstraintChannel(OldValue.ConstraintID))
			{				
				UTickableConstraint* Constraint = ConstraintAndActiveChannel->GetConstraint().Get();
				if (Constraint)
				{
					Constraint->SetActive(OldValue.bPreviouslyEnabled);
					Constraint->bValid = false; 
					//we don't delete it since for some reason on first creation we get a restore state and so it immediately get's deleted
					if (OldValue.bDeleteIt)
					{
						UMovieScene3DTransformSection* ConstrainedSection = OldValue.Section.Get();
						if (ConstrainedSection)
						{
							ConstrainedSection->SetDoNoRemoveChannel(true);
						}
						FConstraintsManagerController& Controller = FConstraintsManagerController::Get(SceneComponent->GetWorld());
						Controller.RemoveConstraint(Constraint, true);

						if (ConstrainedSection)
						{
							ConstrainedSection->SetDoNoRemoveChannel(false);
						}
					}
				
				}
			}
		}
	}
};


struct FEvaluateConstraintChannels
{
	FEvaluateConstraintChannels(UWorld* InWorld, FInstanceRegistry* InInstanceRegistry, UMovieSceneConstraintSystem* InSystem)
		: World(InWorld), InstanceRegistry(InInstanceRegistry), System(InSystem)
	{
		check(World && InstanceRegistry && System);
	}

	void PreTask()
	{
		System->DynamicOffsets.Reset();
	}

	void ForEachEntity(UObject* BoundObject, FInstanceHandle InstanceHandle, const FConstraintComponentData& ConstraintChannel, FFrameTime FrameTime) const
	{
		FConstraintAndActiveChannel* ConstraintAndActiveChannel = ConstraintChannel.Section->GetConstraintChannel(ConstraintChannel.ConstraintID);
		if (!ConstraintAndActiveChannel)
		{
			return;
		}

		UTickableConstraint* Constraint = CreateConstraintIfNeeded(World, ConstraintAndActiveChannel);
		if (!Constraint)
		{
			return;
		}

		const FSequenceInstance& TargetInstance = InstanceRegistry->GetInstance(InstanceHandle);
		TSharedRef<FSharedPlaybackState> SharedPlaybackState = TargetInstance.GetSharedPlaybackState();

		bool Value = false;
		ConstraintAndActiveChannel->ActiveChannel.Evaluate(FrameTime, Value);
		Constraint->SetActive(Value);
		
		if (UTickableTransformConstraint* TransformConstraint = Cast< UTickableTransformConstraint>(Constraint))
		{
			TransformConstraint->InitConstraint(World);

			// this has to be done once the constraint initialized
			TransformConstraint->ResolveBoundObjects(TargetInstance.GetSequenceID(), SharedPlaybackState);

			if (Value && TransformableHandleUtils::SkipTicking())
			{
				if (UTransformableComponentHandle* ParentHandle = Cast<UTransformableComponentHandle>(TransformConstraint->ParentTRSHandle))
				{
					constexpr bool bRecursive = true;
					USceneComponent* ParentComponent = TransformableHandleUtils::GetTarget<USceneComponent>(TransformConstraint->ParentTRSHandle);
					TransformableHandleUtils::MarkComponentForEvaluation(ParentComponent, bRecursive);
				}
			}
			
			if (UTransformableComponentHandle* ChildHandle = Cast<UTransformableComponentHandle>(TransformConstraint->ChildTRSHandle))
			{
				//bound component may change so need to update constraint
				ChildHandle->Component = Cast<USceneComponent>(BoundObject);
				UMovieSceneConstraintSystem::FUpdateHandleForConstraint UpdateHandle;
				UpdateHandle.Constraint = TransformConstraint;
				UpdateHandle.TransformHandle = ChildHandle;
				System->DynamicOffsets.Add(UpdateHandle);
				TransformConstraint->EnsurePrimaryDependency(World);
			}
		}
		else
		{
			Constraint->ResolveBoundObjects(TargetInstance.GetSequenceID(), SharedPlaybackState);	
		}
	}

	UWorld* World;
	FInstanceRegistry* InstanceRegistry;
	UMovieSceneConstraintSystem* System;
};

struct FPreAnimatedConstraintStorage
	: public TPreAnimatedStateStorage<FPreAnimatedConstraintTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedConstraintStorage> StorageID;
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedConstraintStorage> FPreAnimatedConstraintStorage::StorageID;

} // namespace UE::MovieScene

UMovieSceneConstraintSystem::UMovieSceneConstraintSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	RelevantComponent = TracksComponents->ConstraintChannel;

	// Run constraints during instantiation or evaluation
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling | ESystemPhase::Finalization;

	if (HasAnyFlags(RF_ClassDefaultObject) ) 
	{
		// Constraints must be evaluated before their transforms are evaluated.
		// This is only really necessary if they are in the same phase (which they are not), but I've
		// defined the prerequisite for safety if its phase changes in future
		DefineImplicitPrerequisite(GetClass(), UMovieSceneComponentTransformSystem::StaticClass());
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->EvalTime);
	}
}

void UMovieSceneConstraintSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// Set up new constraints
	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObject)
	.Read(BuiltInComponents->InstanceHandle)
	.Read(TracksComponents->ConstraintChannel)
	.Read(BuiltInComponents->EvalTime)
	.SetDesiredThread(Linker->EntityManager.GetGatherThread())
	.Schedule_PerEntity<FEvaluateConstraintChannels>(&Linker->EntityManager, TaskScheduler, GetWorld(), Linker->GetInstanceRegistry(), this);
}

void UMovieSceneConstraintSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();
	ESystemPhase CurrentPhase = Runner->GetCurrentPhase();

	if (CurrentPhase == ESystemPhase::Instantiation)
	{
		// Save pre-animated state
		TSharedPtr<FPreAnimatedConstraintStorage> PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedConstraintStorage>();

		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		PreAnimatedStorage->BeginTrackingAndCachePreAnimatedValues(Linker, BuiltInComponents->BoundObject, TracksComponents->ConstraintChannel);

	}
	else if (CurrentPhase == ESystemPhase::Evaluation)
	{
		// Backwards compat with legacy non-persistent tasks
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		// Set up new constraints
		FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.Read(BuiltInComponents->InstanceHandle)
		.Read(TracksComponents->ConstraintChannel)
		.Read(BuiltInComponents->EvalTime)
		.SetDesiredThread(Linker->EntityManager.GetGatherThread())
		.Dispatch_PerEntity<FEvaluateConstraintChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents, GetWorld(), Linker->GetInstanceRegistry(), this);
	}
	else if (CurrentPhase == ESystemPhase::Finalization)
	{
		for (FUpdateHandleForConstraint& UpdateHandle : DynamicOffsets)
		{
			if (UpdateHandle.Constraint.IsValid() && UpdateHandle.TransformHandle.IsValid())
			{
				UpdateHandle.Constraint->OnHandleModified(UpdateHandle.TransformHandle.Get(),EHandleEvent::LocalTransformUpdated);
			}
		}
		DynamicOffsets.Reset();
	}
}
