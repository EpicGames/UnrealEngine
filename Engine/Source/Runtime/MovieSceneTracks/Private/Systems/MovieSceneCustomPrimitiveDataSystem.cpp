// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneCustomPrimitiveDataSystem.h"

#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieSceneHierarchicalBiasSystem.h"
#include "Systems/MovieSceneInitialValueSystem.h"
#include "Systems/MovieScenePreAnimatedMaterialParameters.h"
#include "Systems/WeightAndEasingEvaluatorSystem.h"

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "EntitySystem/MovieSceneEntityGroupingSystem.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "SceneTypes.h"
#include "Components/PrimitiveComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCustomPrimitiveDataSystem)

namespace UE::MovieScene
{

	struct FCustomPrimitiveDataGroupingPolicy
	{
		using GroupKeyType = TTuple<FObjectKey, FName>;

		void InitializeGroupKeys(
			TEntityGroupingHandlerBase<FCustomPrimitiveDataGroupingPolicy>& Handler,
			FEntityGroupBuilder* Builder,
			FEntityAllocationIteratorItem Item,
			FReadEntityIDs EntityIDs,
			TWrite<FEntityGroupID> GroupIDs,
			TRead<UObject*> BoundObjects,
			TRead<FName> Names)
		{
			FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

			const FComponentMask& AllocationType = Item.GetAllocationType();
			if (!AllocationType.Contains(TracksComponents->Tags.CustomPrimitiveData))
			{
				return;
			}

			const FEntityAllocation* Allocation = Item.GetAllocation();
			const int32              Num        = Allocation->Num();

			TComponentReader<FName>    ParameterNames = Allocation->ReadComponents(TracksComponents->ScalarParameterName);
			for (int32 Index = 0; Index < Num; ++Index)
			{
				GroupKeyType Key = MakeTuple(BoundObjects[Index], ParameterNames[Index]);

				const int32    NewGroupIndex = Handler.GetOrAllocateGroupIndex(Key, Builder);
				FEntityGroupID NewGroupID    = Builder->MakeGroupID(NewGroupIndex);

				Builder->AddEntityToGroup(EntityIDs[Index], NewGroupID);
				// Write out the group ID component
				GroupIDs[Index] = NewGroupID;
			}
		}

	#if WITH_EDITOR
		bool OnObjectsReplaced(GroupKeyType& InOutKey, const TMap<UObject*, UObject*>& ReplacementMap)
		{
			return false;
		}
	#endif
	};


	void CollectGarbageForOutput(FAnimatedCustomPrimitiveDataInfo* Output)
	{
		// This should only happen during garbage collection
		if (Output->OutputEntityID.IsValid())
		{
			UMovieSceneEntitySystemLinker* Linker = Output->WeakLinker.Get();
			if (Linker)
			{
				Linker->EntityManager.AddComponent(Output->OutputEntityID, FBuiltInComponentTypes::Get()->Tags.NeedsUnlink);
			}

			Output->OutputEntityID = FMovieSceneEntityID();
		}

		if (Output->BlendChannelID.IsValid())
		{
			UMovieSceneBlenderSystem* BlenderSystem = Output->WeakBlenderSystem.Get();
			if (BlenderSystem)
			{
				BlenderSystem->ReleaseBlendChannel(Output->BlendChannelID);
			}
			Output->BlendChannelID = FMovieSceneBlendChannelID();
		}
	}

	FAnimatedCustomPrimitiveDataInfo::~FAnimatedCustomPrimitiveDataInfo()
	{
	}

	/** Apply scalar material parameters */
	struct FApplyCustomPrimitiveDataParameters
	{
		static void ForEachAllocation(FEntityAllocationIteratorItem Item,
			TRead<UObject*> BoundObjects,
			TRead<FName> ParameterNames,
			TRead<double> ScalarValues)
		{
			const int32 Num = Item.GetAllocation()->Num();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				UPrimitiveComponent* PrimitiveComponent = CastChecked<UPrimitiveComponent>(BoundObjects[Index]);
				FString ParameterNameString = ParameterNames[Index].ToString();
				check(ParameterNameString.IsNumeric());
				int32 DataIndex = FCString::Atoi(*ParameterNameString);
				check(DataIndex >= 0 && DataIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats);
				PrimitiveComponent->SetCustomPrimitiveDataFloat(DataIndex, (float)ScalarValues[Index]);
			}
		}
	};

	struct FCustomPrimitiveDataMixin
	{
		void CreateEntity(UMovieSceneEntitySystemLinker* Linker, FObjectKey BoundObject, FName ParameterName, FComponentTypeID BlenderTypeTag, TArrayView<const FMovieSceneEntityID> Inputs, FAnimatedCustomPrimitiveDataInfo* Output)
		{
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
			FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

			bool bHasInitialValue = false;
			float InitialValue = 0.f;

			for (FMovieSceneEntityID Input : Inputs)
			{
				if (TOptionalComponentReader<double> ExistingInitialValue = Linker->EntityManager.ReadComponent(Input, TracksComponents->FloatParameter.InitialValue))
				{
					InitialValue = static_cast<float>(*ExistingInitialValue);
					bHasInitialValue = true;
					break;
				}
			}

			if (!bHasInitialValue)
			{
				if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(BoundObject.ResolveObjectPtr()))
				{
					FString ParameterNameString = ParameterName.ToString();
					check(ParameterNameString.IsNumeric());
					int32 DataIndex = FCString::Atoi(*ParameterNameString);
					check(DataIndex >= 0 && DataIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats);
					const FCustomPrimitiveData& CustomPrimitiveData = PrimitiveComponent->GetCustomPrimitiveData();
					if (CustomPrimitiveData.Data.IsValidIndex(DataIndex))
					{
						bHasInitialValue = true;
						InitialValue = CustomPrimitiveData.Data[DataIndex];
					}
				}
			}

			Output->OutputEntityID = FEntityBuilder()
				.Add(BuiltInComponents->BoundObject, BoundObject.ResolveObjectPtr())
				.Add(BuiltInComponents->BlendChannelOutput, Output->BlendChannelID)
				.Add(BuiltInComponents->DoubleResult[0], 0.0)
				.AddConditional(TracksComponents->FloatParameter.InitialValue, InitialValue, bHasInitialValue)
				.AddTag(TracksComponents->FloatParameter.PropertyTag)
				.AddTag(TracksComponents->Tags.CustomPrimitiveData)
				.AddTag(BlenderTypeTag)
				.AddTag(BuiltInComponents->Tags.NeedsLink)
				.AddMutualComponents()
				.CreateEntity(&Linker->EntityManager);

			Linker->EntityManager.CopyComponents(Inputs[0], Output->OutputEntityID, Linker->EntityManager.GetComponents()->GetCopyAndMigrationMask());
		}

		void InitializeSoleInput(UMovieSceneEntitySystemLinker* Linker, FObjectKey BoundObject, FName ParameterName, FMovieSceneEntityID SoleContributor, FAnimatedCustomPrimitiveDataInfo* Output)
		{
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
			FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

			const FComponentMask& SoleContributorType = Linker->EntityManager.GetEntityType(SoleContributor);
			const bool bNeedsInitialValue = SoleContributorType.ContainsAny({ BuiltInComponents->Tags.RestoreState, BuiltInComponents->Tags.AlwaysCacheInitialValue });
			const bool bAlreadyHasInitialValue = SoleContributorType.Contains(TracksComponents->FloatParameter.InitialValue);

			if (bNeedsInitialValue && !bAlreadyHasInitialValue)
			{
				float InitialValue = 0.0;
				if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(BoundObject.ResolveObjectPtr()))
				{
					FString ParameterNameString = ParameterName.ToString();
					check(ParameterNameString.IsNumeric());
					int32 DataIndex = FCString::Atoi(*ParameterNameString);
					check(DataIndex >= 0 && DataIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats);
					const FCustomPrimitiveData& CustomPrimitiveData = PrimitiveComponent->GetCustomPrimitiveData();
					if (CustomPrimitiveData.Data.IsValidIndex(DataIndex))
					{
						InitialValue = CustomPrimitiveData.Data[DataIndex];
						Linker->EntityManager.AddComponent(SoleContributor, TracksComponents->FloatParameter.InitialValue, InitialValue);
					}
				}
			}

			Linker->EntityManager.RemoveComponent(SoleContributor, FBuiltInComponentTypes::Get()->HierarchicalBlendTarget);
		}
	};

	/** Handler that manages creation of blend outputs where there are multiple contributors for the same material parameter */
	template<typename Mixin>
	struct TOverlappingCustomPrimitiveDataHandler : Mixin
	{
		UMovieSceneEntitySystemLinker* Linker;
		UMovieSceneCustomPrimitiveDataSystem* System;

		TOverlappingCustomPrimitiveDataHandler(UMovieSceneCustomPrimitiveDataSystem* InSystem)
			: Linker(InSystem->GetLinker())
			, System(InSystem)
		{}

		void InitializeOutput(FObjectKey BoundObject, FName ParameterName, TArrayView<const FMovieSceneEntityID> Inputs, FAnimatedCustomPrimitiveDataInfo* Output, FEntityOutputAggregate Aggregate)
		{
			UpdateOutput(BoundObject, ParameterName, Inputs, Output, Aggregate);
		}

		void UpdateOutput(FObjectKey BoundObject, FName ParameterName, TArrayView<const FMovieSceneEntityID> Inputs, FAnimatedCustomPrimitiveDataInfo* Output, FEntityOutputAggregate Aggregate)
		{
			using namespace UE::MovieScene;

			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
			FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

			const int32 NumContributors = Inputs.Num();
			if (!ensure(NumContributors != 0))
			{
				return;
			}

			const bool bUseBlending = NumContributors > 1 || !Linker->EntityManager.HasComponent(Inputs[0], BuiltInComponents->Tags.AbsoluteBlend) || Linker->EntityManager.HasComponent(Inputs[0], BuiltInComponents->WeightAndEasingResult);
			if (bUseBlending)
			{
				if (!Output->OutputEntityID)
				{
					if (!System->DoubleBlenderSystem)
					{
						System->DoubleBlenderSystem = Linker->LinkSystem<UMovieScenePiecewiseDoubleBlenderSystem>();
						Linker->SystemGraph.AddReference(System, System->DoubleBlenderSystem);
					}

					Output->WeakLinker = Linker;
					Output->WeakBlenderSystem = System->DoubleBlenderSystem;

					// Initialize the blend channel ID
					Output->BlendChannelID = System->DoubleBlenderSystem->AllocateBlendChannel();

					// Needs blending
					FComponentTypeID BlenderTypeTag = System->DoubleBlenderSystem->GetBlenderTypeTag();
					Mixin::CreateEntity(Linker, BoundObject, ParameterName, BlenderTypeTag, Inputs, Output);
				}

				const FComponentTypeID BlenderTypeTag = System->DoubleBlenderSystem->GetBlenderTypeTag();








				for (FMovieSceneEntityID Input : Inputs)
				{
					if (!Linker->EntityManager.HasComponent(Input, BuiltInComponents->BlendChannelInput))
					{
						Linker->EntityManager.AddComponent(Input, BuiltInComponents->BlendChannelInput, Output->BlendChannelID);
					}
					else
					{
						// If the bound object changed, we might have been re-assigned a different blend channel so make sure it's up to date
						Linker->EntityManager.WriteComponentChecked(Input, BuiltInComponents->BlendChannelInput, Output->BlendChannelID);
					}


					// Ensure we have the blender type tag on the inputs.
					Linker->EntityManager.AddComponent(Input, BlenderTypeTag);
				}
			}
			else if (!Output->OutputEntityID && Inputs.Num() == 1)
			{
				Linker->EntityManager.RemoveComponent(Inputs[0], BuiltInComponents->BlendChannelInput);

				Mixin::InitializeSoleInput(Linker, BoundObject, ParameterName, Inputs[0], Output);
			}

			Output->NumContributors = NumContributors;
		}

		void DestroyOutput(FObjectKey BoundObject, FName ParameterName, FAnimatedCustomPrimitiveDataInfo* Output, FEntityOutputAggregate Aggregate)
		{
			if (Output->OutputEntityID)
			{
				FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

				Linker->EntityManager.AddComponent(Output->OutputEntityID, BuiltInComponents->Tags.NeedsUnlink);
				Output->OutputEntityID = FMovieSceneEntityID();

				if (System->DoubleBlenderSystem)
				{
					System->DoubleBlenderSystem->ReleaseBlendChannel(Output->BlendChannelID);
				}
				Output->BlendChannelID = FMovieSceneBlendChannelID();
			}
		}

	};

	struct FCustomPrimitiveDataEntryTraits : FBoundObjectPreAnimatedStateTraits
	{
		struct FKeyType
		{
			FObjectKey Object;
			FName IndexName;

			/** Constructor that takes a BoundObject (the component) and the FName for the data index */
			FKeyType(FObjectKey InObject, FName InIndexName)
				: Object(InObject)
				, IndexName(InIndexName)
			{}

			/** Hashing and equality required for storage within a map */
			friend uint32 GetTypeHash(const FKeyType& InKey)
			{
				return HashCombine(GetTypeHash(InKey.Object), GetTypeHash(InKey.IndexName));
			}
			friend bool operator==(const FKeyType& A, const FKeyType& B)
			{
				return A.Object == B.Object && A.IndexName == B.IndexName;
			}
		};

		using KeyType = FKeyType;
		using StorageType = float;

		static float CachePreAnimatedValue(FObjectKey InObject, FName IndexName)
		{
			if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InObject.ResolveObjectPtr()))
			{
				FString ParameterNameString = IndexName.ToString();
				check(ParameterNameString.IsNumeric());
				int32 DataIndex = FCString::Atoi(*ParameterNameString);
				check(DataIndex >= 0 && DataIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats);
				const FCustomPrimitiveData& CustomPrimitiveData = PrimitiveComponent->GetCustomPrimitiveData();
				if (CustomPrimitiveData.Data.IsValidIndex(DataIndex))
				{
					return CustomPrimitiveData.Data[DataIndex];
				}
			}
			return 0.0f;
		}

		static void RestorePreAnimatedValue(const FKeyType& InKey, float OldValue, const FRestoreStateParams& Params)
		{
			if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(InKey.Object.ResolveObjectPtr()))
			{
				FString ParameterNameString = InKey.IndexName.ToString();
				check(ParameterNameString.IsNumeric());
				int32 DataIndex = FCString::Atoi(*ParameterNameString);
				check(DataIndex >= 0 && DataIndex < FCustomPrimitiveData::NumCustomPrimitiveDataFloats);
				PrimitiveComponent->SetCustomPrimitiveDataFloat(DataIndex, OldValue);
			}
		}
	};

	struct FPreAnimatedCustomPrimitiveDataEntryStorage
		: public TPreAnimatedStateStorage<FCustomPrimitiveDataEntryTraits>
	{
		static TAutoRegisterPreAnimatedStorageID<FPreAnimatedCustomPrimitiveDataEntryStorage> StorageID;
	};

	TAutoRegisterPreAnimatedStorageID<FPreAnimatedCustomPrimitiveDataEntryStorage> FPreAnimatedCustomPrimitiveDataEntryStorage::StorageID;

} // namespace UE::MovieScene

UMovieSceneCustomPrimitiveDataSystem::UMovieSceneCustomPrimitiveDataSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	RelevantComponent = TracksComponents->Tags.CustomPrimitiveData;
	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);
		DefineComponentConsumer(GetClass(), BuiltInComponents->HierarchicalBlendTarget);

		DefineImplicitPrerequisite(UFloatChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
		{
			DefineComponentConsumer(GetClass(), BuiltInComponents->DoubleResult[Index]);
		}

		DefineComponentConsumer(GetClass(), TracksComponents->Tags.CustomPrimitiveData);

		DefineImplicitPrerequisite(UMovieSceneHierarchicalEasingInstantiatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneInitialValueSystem::StaticClass());
	}
}

void UMovieSceneCustomPrimitiveDataSystem::OnLink()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	ScalarParameterTracker.Initialize(this);

	ScalarParameterStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedCustomPrimitiveDataEntryStorage>();

	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->LinkSystem<UMovieSceneEntityGroupingSystem>();
	GroupingKey = GroupingSystem->AddGrouping(FCustomPrimitiveDataGroupingPolicy(), BuiltInComponents->BoundObject, TracksComponents->ScalarParameterName);
}

void UMovieSceneCustomPrimitiveDataSystem::OnUnlink()
{
	using namespace UE::MovieScene;

	// Always reset the float blender system on link to ensure that recycled systems are correctly initialized.
	DoubleBlenderSystem = nullptr;

	ScalarParameterTracker.Destroy(TOverlappingCustomPrimitiveDataHandler<FCustomPrimitiveDataMixin>(this));

	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->FindSystem<UMovieSceneEntityGroupingSystem>();
	if (ensure(GroupingSystem))
	{
		GroupingSystem->RemoveGrouping(GroupingKey);
	}
	GroupingKey = FEntityGroupingPolicyKey();
}

void UMovieSceneCustomPrimitiveDataSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	ESystemPhase CurrentPhase = Runner->GetCurrentPhase();
	if (CurrentPhase == ESystemPhase::Instantiation)
	{
		OnInstantiation();
	}
	else if (CurrentPhase == ESystemPhase::Evaluation)
	{
		OnEvaluation(InPrerequisites, Subsequents);
	}
}


void UMovieSceneCustomPrimitiveDataSystem::OnInstantiation()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	auto HandleUnlinkedAllocation = [this](const FEntityAllocation* Allocation)
	{
		this->ScalarParameterTracker.VisitUnlinkedAllocation(Allocation);
	};

	auto HandleUpdatedAllocation = [this](const FEntityAllocation* Allocation, TRead<UObject*> InObjects, TRead<FName> ParameterNames)
	{
		this->ScalarParameterTracker.VisitActiveAllocation(Allocation, InObjects, ParameterNames);
	};

	// First step handle any new bound objects
	FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.Read(TracksComponents->ScalarParameterName)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink, TracksComponents->Tags.CustomPrimitiveData })
		.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerAllocation(&Linker->EntityManager, HandleUpdatedAllocation);

	// Next handle any entities that are going away
	FEntityTaskBuilder()
		.FilterAll({ BuiltInComponents->Tags.NeedsUnlink, TracksComponents->Tags.CustomPrimitiveData })
		.Iterate_PerAllocation(&Linker->EntityManager, HandleUnlinkedAllocation);

	// Process all blended parameters
	TOverlappingCustomPrimitiveDataHandler<FCustomPrimitiveDataMixin> ScalarHandler(this);
	ScalarParameterTracker.ProcessInvalidatedOutputs(Linker, ScalarHandler);


	// Gather inputs that contribute to the parameter by excluding outputs (which will not have an instance handle)
	TPreAnimatedStateTaskParams<FObjectKey, FName> Params;
	Params.AdditionalFilter.Reset();
	Params.AdditionalFilter.None({ BuiltInComponents->BlendChannelOutput });
	Params.AdditionalFilter.All({ BuiltInComponents->Tags.NeedsLink, TracksComponents->Tags.CustomPrimitiveData });

	ScalarParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, BuiltInComponents->BoundObject, TracksComponents->ScalarParameterName);

}

void UMovieSceneCustomPrimitiveDataSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.Read(TracksComponents->ScalarParameterName)
		.Read(BuiltInComponents->DoubleResult[0])
		.FilterAll({ TracksComponents->Tags.CustomPrimitiveData })
		.FilterNone({ BuiltInComponents->BlendChannelInput })
		.SetDesiredThread(Linker->EntityManager.GetDispatchThread())
		.Fork_PerAllocation<FApplyCustomPrimitiveDataParameters>(&Linker->EntityManager, TaskScheduler);
}

void UMovieSceneCustomPrimitiveDataSystem::OnEvaluation(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	if (Linker->EntityManager.ContainsComponent(TracksComponents->ScalarParameterName))
	{
		FEntityTaskBuilder()
			.Read(BuiltInComponents->BoundObject)
			.Read(TracksComponents->ScalarParameterName)
			.Read(BuiltInComponents->DoubleResult[0])
			.FilterAll({ TracksComponents->Tags.CustomPrimitiveData })
			.FilterNone({ BuiltInComponents->BlendChannelInput })
			.SetDesiredThread(Linker->EntityManager.GetDispatchThread())
			.Dispatch_PerAllocation<FApplyCustomPrimitiveDataParameters>(&Linker->EntityManager, InPrerequisites, &Subsequents);
	}
}

