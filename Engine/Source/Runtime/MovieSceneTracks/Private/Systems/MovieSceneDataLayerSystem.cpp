// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneDataLayerSystem.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneSpawnablesSystem.h"
#include "EntitySystem/MovieSceneBindingLifetimeSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneTracksComponentTypes.h"
#include "ProfilingDebugging/CsvProfiler.h"
#include "Sections/MovieSceneDataLayerSection.h"
#include "WorldPartition/DataLayer/DataLayerManager.h"
#include "WorldPartition/DataLayer/DataLayerInstance.h"
#include "WorldPartition/WorldPartitionSubsystem.h"
#include "WorldPartition/WorldPartitionRuntimeCell.h"
#include "WorldPartition/WorldPartitionStreamingSource.h"
#include "Engine/CoreSettings.h"
#include "Engine/World.h"
#include "Misc/EnumClassFlags.h"
#include "MovieSceneSequence.h"
#include "IMovieScenePlayer.h"

#include "Kismet/KismetSystemLibrary.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDataLayerSystem)

#if WITH_EDITOR
	#include "DataLayer/DataLayerEditorSubsystem.h"
#endif

namespace UE
{
namespace MovieScene
{

enum class EDataLayerUpdateFlags : uint8
{
	None						= 0,
	FlushStreamingVisibility	= 1,
	FlushStreamingFull			= 2
};
ENUM_CLASS_FLAGS(EDataLayerUpdateFlags)

/** Traits class governing how pre-animated state is (re)stored for data layers */
struct FPreAnimatedDataLayerStorageTraits : FPreAnimatedStateTraits
{
	using KeyType = TObjectKey<UDataLayerInstance>;
	using StorageType = EDataLayerRuntimeState;

	/** Called when a previously animated data layer needs to be restored */
	static void RestorePreAnimatedValue(const TObjectKey<UDataLayerInstance>& InKey, EDataLayerRuntimeState PreviousState, const FRestoreStateParams& Params);
};



/** Container class for all pre-animated data layer state */
struct FPreAnimatedDataLayerStorage
	: TPreAnimatedStateStorage<FPreAnimatedDataLayerStorageTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedDataLayerStorage> StorageID;

	/** Make an entry for the specified data layer */
	FPreAnimatedStateEntry FindEntry(const UDataLayerInstance* InDataLayer);
	FPreAnimatedStateEntry MakeEntry(const UDataLayerInstance* InDataLayer);

	/** Save the value of a data layer. Should only be used for runtime / PIE worlds */
	void SavePreAnimatedState(const UDataLayerInstance* DataLayer);

#if WITH_EDITOR
	/** Save the value of a data layer. Should only be used for editor worlds */
	void SavePreAnimatedStateInEditor(const UDataLayerInstance* DataLayer);
#endif
};


struct FDataLayerState
{
	void Reset();
	bool IsEmpty() const;
	void AddRequest(int16 InBias, EDataLayerRuntimeState RequestedState, bool bRequiresStreamingFlush);
	TOptional<EDataLayerRuntimeState> ComputeDesiredState() const;
	bool ShouldFlushStreaming(EDataLayerRuntimeState ComputedState) const;

private:

	int16 HierarchicalBias = 0;
	int32 UnloadedCount    = 0;
	int32 LoadedCount      = 0;
	int32 ActivatedCount   = 0;
	bool  bFlushUnloaded   = false;
	bool  bFlushActivated  = false;
};


struct FDesiredLayerStates
{
	bool IsEmpty() const;
	void Reset();
	EDataLayerUpdateFlags Apply(FPreAnimatedDataLayerStorage* PreAnimatedStorage);
#if WITH_EDITOR
	void ApplyInEditor(FPreAnimatedDataLayerStorage* PreAnimatedStorage, UDataLayerEditorSubsystem* EditorSubSystem);
#endif
	void ApplyNewState(const UDataLayerInstance* InDataLayer, int16 HierarchicalBias, EDataLayerRuntimeState DesiredState, bool bRequiresStreamingFlush);

	TMap<TObjectKey<UDataLayerInstance>, FDataLayerState> StatesByInstance;
};

// ---------------------------------------------------------------------
// FPreAnimatedDataLayerStorageTraits definitions
void FPreAnimatedDataLayerStorageTraits::RestorePreAnimatedValue(const TObjectKey<UDataLayerInstance>& InKey, EDataLayerRuntimeState PreviousState, const FRestoreStateParams& Params)
{
	UDataLayerInstance* DataLayer = InKey.ResolveObjectPtr();
	if (!DataLayer)
	{
		return;
	}

	UWorld* World = DataLayer->GetWorld();
	if (World)
	{
#if WITH_EDITOR
		if (World->WorldType == EWorldType::Editor)
		{
			UDataLayerEditorSubsystem* SubSystem = UDataLayerEditorSubsystem::Get();
			SubSystem->SetDataLayerVisibility(DataLayer, PreviousState == EDataLayerRuntimeState::Activated);
		}
		else
#endif
		if (UDataLayerManager* DataLayerManager = UDataLayerManager::GetDataLayerManager(DataLayer))
		{
			DataLayerManager->SetDataLayerInstanceRuntimeState(DataLayer, PreviousState);
		}
	}
}

// ---------------------------------------------------------------------
// FPreAnimatedDataLayerStorage definitions
TAutoRegisterPreAnimatedStorageID<FPreAnimatedDataLayerStorage> FPreAnimatedDataLayerStorage::StorageID;

FPreAnimatedStateEntry FPreAnimatedDataLayerStorage::FindEntry(const UDataLayerInstance* InDataLayer)
{
	FPreAnimatedStorageIndex StorageIndex = FindStorageIndex(InDataLayer);
	return FPreAnimatedStateEntry{ FPreAnimatedStorageGroupHandle(), FPreAnimatedStateCachedValueHandle{ StorageID, StorageIndex } };
}

FPreAnimatedStateEntry FPreAnimatedDataLayerStorage::MakeEntry(const UDataLayerInstance* InDataLayer)
{
	FPreAnimatedStorageIndex StorageIndex = GetOrCreateStorageIndex(InDataLayer);
	return FPreAnimatedStateEntry{ FPreAnimatedStorageGroupHandle(), FPreAnimatedStateCachedValueHandle{ StorageID, StorageIndex } };
}

void FPreAnimatedDataLayerStorage::SavePreAnimatedState(const UDataLayerInstance* DataLayer)
{
	if (!this->ShouldTrackCaptureSource(EPreAnimatedCaptureSourceTracking::CacheIfTracked, DataLayer))
	{
		return;
	}

	FPreAnimatedStateEntry Entry = MakeEntry(DataLayer);

	this->TrackCaptureSource(Entry, EPreAnimatedCaptureSourceTracking::CacheIfTracked);

	EPreAnimatedStorageRequirement StorageRequirement = this->ParentExtension->GetStorageRequirement(Entry);

	if (!IsStorageRequirementSatisfied(Entry.ValueHandle.StorageIndex, StorageRequirement))
	{
		// @todo: If a data layer is loading when Sequencer attempts to activate it,
		// should it return to ::Loading when sequencer is done?
		EDataLayerRuntimeState ExistingState = DataLayer->GetRuntimeState();

		AssignPreAnimatedValue(Entry.ValueHandle.StorageIndex, StorageRequirement, CopyTemp(ExistingState));
	}
}

#if WITH_EDITOR
void FPreAnimatedDataLayerStorage::SavePreAnimatedStateInEditor(const UDataLayerInstance* DataLayer)
{
	if (!this->ShouldTrackCaptureSource(EPreAnimatedCaptureSourceTracking::CacheIfTracked, DataLayer))
	{
		return;
	}

	FPreAnimatedStateEntry Entry = MakeEntry(DataLayer);
	this->TrackCaptureSource(Entry, EPreAnimatedCaptureSourceTracking::CacheIfTracked);

	EPreAnimatedStorageRequirement StorageRequirement = this->ParentExtension->GetStorageRequirement(Entry);

	if (!IsStorageRequirementSatisfied(Entry.ValueHandle.StorageIndex, StorageRequirement))
	{
		// We never unload data-layers in editor, so feign currently unloaded layers as loaded
		EDataLayerRuntimeState ExistingState = (DataLayer->IsVisible() && DataLayer->IsLoadedInEditor()) ? EDataLayerRuntimeState::Activated : EDataLayerRuntimeState::Loaded;

		AssignPreAnimatedValue(Entry.ValueHandle.StorageIndex, StorageRequirement, CopyTemp(ExistingState));
	}
}
#endif

// ---------------------------------------------------------------------
// FDataLayerState definitions
void FDataLayerState::Reset()
{
	HierarchicalBias = 0;
	UnloadedCount    = 0;
	LoadedCount      = 0;
	ActivatedCount   = 0;
	bFlushUnloaded   = false;
	bFlushActivated  = false;
}

void FDataLayerState::AddRequest(int16 InBias, EDataLayerRuntimeState RequestedState, bool bRequiresStreamingFlush)
{
	if (InBias > HierarchicalBias)
	{
		Reset();
		HierarchicalBias = InBias;
	}

	if (InBias == HierarchicalBias)
	{
		switch (RequestedState)
		{
		case EDataLayerRuntimeState::Unloaded:  ++UnloadedCount;  bFlushUnloaded |= bRequiresStreamingFlush; break;
		case EDataLayerRuntimeState::Loaded:    ++LoadedCount;    break;
		case EDataLayerRuntimeState::Activated: ++ActivatedCount; bFlushActivated |= bRequiresStreamingFlush; break;
		}
	}
}

bool FDataLayerState::IsEmpty() const
{
	return (UnloadedCount + LoadedCount + ActivatedCount) == 0;
}

TOptional<EDataLayerRuntimeState> FDataLayerState::ComputeDesiredState() const
{
	if (ActivatedCount > 0)
	{
		return EDataLayerRuntimeState::Activated;
	}
	else if (LoadedCount > 0)
	{
		return EDataLayerRuntimeState::Loaded;
	}
	else if (UnloadedCount > 0)
	{
		return EDataLayerRuntimeState::Unloaded;
	}

	return TOptional<EDataLayerRuntimeState>();
}

bool FDataLayerState::ShouldFlushStreaming(EDataLayerRuntimeState ComputedState) const
{
	switch (ComputedState)
	{
	case EDataLayerRuntimeState::Unloaded:  return bFlushUnloaded;
	case EDataLayerRuntimeState::Activated: return bFlushActivated;
	}

	return false;
}

// ---------------------------------------------------------------------
// FDataLayerStates definitions
bool FDesiredLayerStates::IsEmpty() const
{
	return StatesByInstance.Num() == 0;
}

void FDesiredLayerStates::Reset()
{
	for (auto& [DataLayerInstance, State] : StatesByInstance)
	{
		State.Reset();
	}
}

EDataLayerUpdateFlags FDesiredLayerStates::Apply(FPreAnimatedDataLayerStorage* PreAnimatedStorage)
{
	EDataLayerUpdateFlags Flags = EDataLayerUpdateFlags::None;

	auto IsDataLayerReady = [](const UDataLayerInstance* DataLayer, EDataLayerRuntimeState DesireState, bool bExactState)
	{
		EWorldPartitionRuntimeCellState QueryState;
		switch (DesireState)
		{
		case EDataLayerRuntimeState::Activated:
			QueryState = EWorldPartitionRuntimeCellState::Activated;
			break;
		case EDataLayerRuntimeState::Loaded:
			QueryState = EWorldPartitionRuntimeCellState::Loaded;
			break;
		case EDataLayerRuntimeState::Unloaded:
			QueryState = EWorldPartitionRuntimeCellState::Unloaded;
			break;
		default:
			UE_LOGF(LogMovieScene, Error, "Unkown data layer state");
			return true;
		}

		FWorldPartitionStreamingQuerySource QuerySource;
		QuerySource.bDataLayersOnly = true;
		QuerySource.bSpatialQuery = false; // @todo_ow: how would we support spatial query from sequencer?
		QuerySource.DataLayers.Add(DataLayer->GetFName());

		UWorldPartition* WorldPartition = DataLayer->GetOuterWorld()->GetWorldPartition();
		return WorldPartition->IsStreamingCompleted(QueryState, { QuerySource }, bExactState);
	};

	for (auto It = StatesByInstance.CreateIterator(); It; ++It)
	{
		const FDataLayerState& StateValue = It.Value();
		if (StateValue.IsEmpty())
		{
			It.RemoveCurrent();
			continue;
		}

		if (TOptional<EDataLayerRuntimeState> DesiredState = StateValue.ComputeDesiredState())
		{
			TObjectKey<UDataLayerInstance> DataLayerInstanceKey = It.Key();
			const UDataLayerInstance* DataLayer = DataLayerInstanceKey.ResolveObjectPtr();
			if (ensure(DataLayer))
			{
				if (PreAnimatedStorage)
				{
					PreAnimatedStorage->SavePreAnimatedState(DataLayer);
				}

				const EDataLayerRuntimeState DesiredStateValue = DesiredState.GetValue();
				if (UDataLayerManager::GetDataLayerManager(DataLayer)->SetDataLayerInstanceRuntimeState(DataLayer, DesiredStateValue))
				{
					if (StateValue.ShouldFlushStreaming(DesiredStateValue) && !IsDataLayerReady(DataLayer, DesiredStateValue, true))
					{
						// Exception for Full flush is if Desired State is Activated but we are not at least in Loaded state
						if (DesiredStateValue == EDataLayerRuntimeState::Activated && !IsDataLayerReady(DataLayer, EDataLayerRuntimeState::Loaded, false))
						{
							Flags |= EDataLayerUpdateFlags::FlushStreamingFull;
							UE_LOGF(LogMovieScene, Warning, "[UMovieSceneDataLayerSystem] Data layer with name '%ls' is causing a full streaming flush (%ls)",
								*DataLayer->GetDataLayerShortName(),
								*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)DesiredStateValue).ToString());
							
							CSV_EVENT_GLOBAL(TEXT("SeqDataLayerFlushFull-%s-%s"),
								*DataLayer->GetDataLayerShortName(),
								*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)DesiredStateValue).ToString());
						}
						else
						{
							Flags |= EDataLayerUpdateFlags::FlushStreamingVisibility;
							UE_LOGF(LogMovieScene, Log, "[UMovieSceneDataLayerSystem] Data layer with name '%ls' is causing a visibility streaming flush (%ls)",
								*DataLayer->GetDataLayerShortName(),
								*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)DesiredStateValue).ToString());
							
							CSV_EVENT_GLOBAL(TEXT("SeqDataLayerFlushVis-%s-%s"),
								*DataLayer->GetDataLayerShortName(),
								*StaticEnum<EDataLayerRuntimeState>()->GetDisplayNameTextByValue((int64)DesiredStateValue).ToString());
						}
					}
				}
			}
		}
	}

	return Flags;
}

#if WITH_EDITOR
void FDesiredLayerStates::ApplyInEditor(FPreAnimatedDataLayerStorage* PreAnimatedStorage, UDataLayerEditorSubsystem* SubSystem)
{
	TArray<UDataLayerInstance*> DatalayersNeedingLoad;
	TArray<UDataLayerInstance*> DatalayersNeedingShow;
	TArray<UDataLayerInstance*> DatalayersNeedingHide;

	for (auto It = StatesByInstance.CreateIterator(); It; ++It)
	{
		TObjectKey<UDataLayerInstance> DataLayerKey = It.Key();
		UDataLayerInstance* DataLayer = DataLayerKey.ResolveObjectPtr();
		const FDataLayerState& StateValue = It.Value();
		if (StateValue.IsEmpty())
		{
			It.RemoveCurrent();
			continue;
		}

		if (TOptional<EDataLayerRuntimeState> DesiredState = StateValue.ComputeDesiredState())
		{
			if (ensure(DataLayer))
			{
				if (PreAnimatedStorage)
				{
					PreAnimatedStorage->SavePreAnimatedStateInEditor(DataLayer);
				}

				// In-editor we only ever hide data layers, we never unload them
				switch (DesiredState.GetValue())
				{
				case EDataLayerRuntimeState::Unloaded:
					DatalayersNeedingHide.Add(DataLayer);
					break;
				case EDataLayerRuntimeState::Loaded:
					DatalayersNeedingLoad.Add(DataLayer);
					DatalayersNeedingHide.Add(DataLayer);
					break;
				case EDataLayerRuntimeState::Activated:
					DatalayersNeedingLoad.Add(DataLayer);
					DatalayersNeedingShow.Add(DataLayer);
					break;
				default:
					break;
				}
			}
		}
	}

	if (DatalayersNeedingLoad.Num() > 0)
	{
		// This blocks while we load data layers
		SubSystem->SetDataLayersIsLoadedInEditor(DatalayersNeedingLoad, true, /*bIsFromUserChange*/false);
	}
	if (DatalayersNeedingShow.Num() > 0)
	{
		SubSystem->SetDataLayersVisibility(DatalayersNeedingShow, true);
	}
	if (DatalayersNeedingHide.Num() > 0)
	{
		SubSystem->SetDataLayersVisibility(DatalayersNeedingHide, false);
	}
}
#endif

void FDesiredLayerStates::ApplyNewState(const UDataLayerInstance* InDataLayer, int16 HierarchicalBias, EDataLayerRuntimeState DesiredState, bool bRequiresStreamingFlush)
{
	using namespace UE::MovieScene;

	FDataLayerState* LayerState = StatesByInstance.Find(InDataLayer);
	if (!LayerState)
	{
		LayerState = &StatesByInstance.Add(InDataLayer, FDataLayerState());
	}

	LayerState->AddRequest(HierarchicalBias, DesiredState, bRequiresStreamingFlush);
}

} // namespace MovieScene
} // namespace UE

UMovieSceneDataLayerSystem::UMovieSceneDataLayerSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	Phase = UE::MovieScene::ESystemPhase::Spawn;
	RelevantComponent = TracksComponents->DataLayer;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(GetClass(), UMovieSceneBindingLifetimeSystem::StaticClass());
	}
	else
	{
		DesiredLayerStates = MakeShared<FDesiredLayerStates>();

		// We only need to run if there are data layer components that need (un)linking
		ApplicableFilter.Filter.All({ TracksComponents->DataLayer });
		ApplicableFilter.Filter.Any({ BuiltInComponents->Tags.NeedsLink, BuiltInComponents->Tags.NeedsUnlink });
	}
}

void UMovieSceneDataLayerSystem::OnLink()
{
}

bool UMovieSceneDataLayerSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	return InLinker->EntityManager.ContainsComponent(RelevantComponent) || (DesiredLayerStates && !DesiredLayerStates->IsEmpty());
}

void UMovieSceneDataLayerSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	// Only run if we must
	UWorld* World = GetWorld();

	if (!World || !ApplicableFilter.Matches(Linker->EntityManager))
	{
		return;
	}

	// Update the desired states of all data layers from the entity manager
	UpdateDesiredStates();

	// In-editor we apply desired states through the editor sub-system
#if WITH_EDITOR
	if (World->WorldType == EWorldType::Editor)
	{
		UDataLayerEditorSubsystem* DataLayerEditorSubsystem = UDataLayerEditorSubsystem::Get();
		if (!DataLayerEditorSubsystem)
		{
			UE_LOGF(LogMovieScene, Warning, "Unable to retrieve data layer editor subsystem - data layer tracks will not function correctly");
		}
		else
		{
			DesiredLayerStates->ApplyInEditor(WeakPreAnimatedStorage.Pin().Get(), DataLayerEditorSubsystem);
		}
	}
	else
#endif

	// Outside of editor, or in PIE, we use the runtime data layer sub-system
	{
		UWorldPartitionSubsystem* WorldPartitionSubsystem = World->GetSubsystem<UWorldPartitionSubsystem>();
		if (!WorldPartitionSubsystem)
		{
			UE_LOGF(LogMovieScene, Warning, "Unable to retrieve world partition subsystem - data layer tracks will not function correctly");
		}
		else
		{
			EDataLayerUpdateFlags UpdateFlags = DesiredLayerStates->Apply(WeakPreAnimatedStorage.Pin().Get());

			const double StartTime = FPlatformTime::Seconds();
			const TCHAR* FlushTypeString = nullptr;

			if (EnumHasAnyFlags(UpdateFlags, EDataLayerUpdateFlags::FlushStreamingFull))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UMovieSceneDataLayerSystem_FlushStreamingFull);
				FlushTypeString = TEXT("FlushStreamingFull");

				World->BlockTillLevelStreamingCompleted();
			}
			else if (EnumHasAnyFlags(UpdateFlags, EDataLayerUpdateFlags::FlushStreamingVisibility))
			{
				TRACE_CPUPROFILER_EVENT_SCOPE(UMovieSceneDataLayerSystem_FlushStreamingVisibility);
				FlushTypeString = TEXT("FlushStreamingVisibility");

				// Make sure any DataLayer state change is processed before flushing visibility					
				WorldPartitionSubsystem->OnUpdateStreamingState();
				World->FlushLevelStreaming(EFlushLevelStreamingType::Visibility);
			}

			UE_SUPPRESS(LogMovieScene, Warning,
			{
				if (FlushTypeString)
				{
					FString SequenceList;
					for (const FSequenceInstance& Instance : Linker->GetInstanceRegistry()->GetSparseInstances())
					{
						TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();
						UMovieSceneSequence* Sequence = SharedPlaybackState->GetSequence(Instance.GetSequenceID());

						if (SequenceList.Len())
						{
							SequenceList += TEXT(",");
						}
						SequenceList += Sequence->GetName();
						
					}

					UE_LOGF(LogMovieScene, Warning, "[UMovieSceneDataLayerSystem] %ls took %.4f seconds (%ls)", FlushTypeString, FPlatformTime::Seconds() - StartTime, *SequenceList);
				}
			});
		}
	}
}

UDataLayerManager* UMovieSceneDataLayerSystem::GetDataLayerManager(UE::MovieScene::FMovieSceneEntityID EntityID, UE::MovieScene::FRootInstanceHandle RootInstance)
{
	using namespace UE::MovieScene;

	const FSequenceInstance& Instance = Linker->GetInstanceRegistry()->GetInstance(RootInstance);
	UObject* PlaybackContext = Instance.GetSharedPlaybackState()->GetPlaybackContext();
	if (PlaybackContext)
	{
		return UDataLayerManager::GetDataLayerManager(PlaybackContext);
	}
	return nullptr;
}

void UMovieSceneDataLayerSystem::UpdateDesiredStates()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	BeginTrackingEntities();

	// Reset the container and re-harvest all active states from the entity manager
	DesiredLayerStates->Reset();

	auto GatherDataLayers = [this, BuiltInComponents](
		FEntityAllocationIteratorItem Item,
		TRead<FMovieSceneEntityID> EntityIDs,
		TRead<FRootInstanceHandle> RootInstanceHandles,
		const FMovieSceneDataLayerComponentData* ComponentData,
		const int16* OptHBiases)
	{
		const bool bPreroll = Item.GetAllocationType().Contains(BuiltInComponents->Tags.PreRoll);
		for (int32 Index = 0; Index < Item.GetAllocation()->Num(); ++Index)
		{
			const UMovieSceneDataLayerSection* Section = ComponentData[Index].Section.Get();
			if (!ensure(Section))
			{
				continue;
			}

			FMovieSceneEntityID EntityID = EntityIDs[Index];
			FRootInstanceHandle RootInstance = RootInstanceHandles[Index];

			const UDataLayerManager* DataLayerManager = GetDataLayerManager(EntityID, RootInstance);
			if (DataLayerManager)
			{
				EDataLayerRuntimeState DesiredState = bPreroll ? Section->GetPrerollState() : Section->GetDesiredState();
				const bool bRequiresStreamingFlush = (DesiredState == EDataLayerRuntimeState::Unloaded) ? Section->GetFlushOnUnload() :
													 (DesiredState == EDataLayerRuntimeState::Activated) ? Section->GetFlushOnActivated() : false;

				for (const UDataLayerAsset* DataLayerAsset : Section->GetDataLayerAssets())
				{
					const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstanceFromAsset(DataLayerAsset);
					if (DataLayerInstance)
					{
						this->DesiredLayerStates->ApplyNewState(DataLayerInstance, OptHBiases ? OptHBiases[Index] : 0, DesiredState, bRequiresStreamingFlush);
					}
				}
			}
		}
	};

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(TracksComponents->DataLayer)
	.ReadOptional(BuiltInComponents->HierarchicalBias)
	.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })  // Do not iterate things that are being destroyed
	.Iterate_PerAllocation(&Linker->EntityManager, GatherDataLayers);
}

void UMovieSceneDataLayerSystem::BeginTrackingEntities()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();
	FPreAnimatedEntityCaptureSource* EntityMetaData    = Linker->PreAnimatedState.GetOrCreateEntityMetaData();

	// Cache the preanimated storage
	TSharedPtr<FPreAnimatedDataLayerStorage> PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedDataLayerStorage>();
	WeakPreAnimatedStorage = PreAnimatedStorage;

	// ---------------------------------------------------------------------------------
	// Only gather entity meta-data during SavePreAnimatedState - the actual values will be cached
	// inside FDataLayerState::Apply
	auto GatherDataLayers = [this, EntityMetaData, PreAnimatedStorage](
		FEntityAllocationIteratorItem Item,
		TRead<FMovieSceneEntityID> EntityIDs,
		TRead<FRootInstanceHandle> RootInstanceHandles,
		TRead<FMovieSceneDataLayerComponentData> ComponentData)
	{
		const bool bRestoreState = Item.GetAllocationType().Contains(FBuiltInComponentTypes::Get()->Tags.RestoreState);

		if (!this->GetLinker()->PreAnimatedState.IsCapturingGlobalState() && !bRestoreState)
		{
			return;
		}

		for (int32 Index = 0; Index < Item.GetAllocation()->Num(); ++Index)
		{
			const UMovieSceneDataLayerSection* Section = ComponentData[Index].Section.Get();
			if (!ensure(Section))
			{
				continue;
			}

			FMovieSceneEntityID EntityID     = EntityIDs[Index];
			FRootInstanceHandle RootInstance = RootInstanceHandles[Index];

			const UDataLayerManager* DataLayerManager = GetDataLayerManager(EntityID, RootInstance);
			if (DataLayerManager)
			{
				for (const UDataLayerAsset* DataLayerAsset : Section->GetDataLayerAssets())
				{
					const UDataLayerInstance* DataLayerInstance = DataLayerManager->GetDataLayerInstanceFromAsset(DataLayerAsset);
					if (DataLayerInstance)
					{
						FPreAnimatedStateEntry Entry = PreAnimatedStorage->MakeEntry(DataLayerInstance);
						EntityMetaData->BeginTrackingEntity(Entry, EntityID, RootInstance, bRestoreState);
					}
				}
			}
		}
	};

	FComponentMask Filter{ BuiltInComponents->Tags.NeedsLink };

	// Iterate any data layer components that need link
	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(TracksComponents->DataLayer)
	.FilterAll(Filter)
	.Iterate_PerAllocation(&Linker->EntityManager, GatherDataLayers);
}

