// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"
#include "UObject/ObjectKey.h"
#include "Containers/Map.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "MovieSceneDataLayerSystem.generated.h"

class UDataLayerManager;

namespace UE
{
namespace MovieScene
{

struct FDesiredLayerStates;
struct FPreAnimatedDataLayerStorage;

} // namespace MovieScene
} // namespace UE


/**
 * System that applies all data layer changes to the world
 */
UCLASS(MinimalAPI)
class UMovieSceneDataLayerSystem
	: public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneDataLayerSystem(const FObjectInitializer& ObjInit);

private:

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnLink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

	UDataLayerManager* GetDataLayerManager(UE::MovieScene::FMovieSceneEntityID EntityID, UE::MovieScene::FRootInstanceHandle RootInstance);
	void UpdateDesiredStates();
	void BeginTrackingEntities();

private:

	/** Cached filter that tells us whether we need to run this frame */
	UE::MovieScene::FCachedEntityFilterResult_Match ApplicableFilter;
	/** Impl class that stores the desired layer states for the current (and last) frame */
	TSharedPtr<UE::MovieScene::FDesiredLayerStates> DesiredLayerStates;
	/** Weak ptr to the pre-animated storage if we need to cache previous states for layers */
	TWeakPtr<UE::MovieScene::FPreAnimatedDataLayerStorage> WeakPreAnimatedStorage;
};
