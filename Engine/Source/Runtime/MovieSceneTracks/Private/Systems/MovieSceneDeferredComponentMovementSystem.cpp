// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneDeferredComponentMovementSystem.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "MovieSceneTracksComponentTypes.h"

#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDeferredComponentMovementSystem)

namespace UE
{
namespace MovieScene
{

bool GDeferSequencerMovementUpdates = false;
static FAutoConsoleVariableRef CVarDeferSequencerMovementUpdates(
	TEXT("Sequencer.DeferMovementUpdates"),
	GDeferSequencerMovementUpdates,
	TEXT("(Default: false) Enables deferring all Scene Component movement updates to the end of a sequencer evaluation to avoid excessive calls to UpdateOverlaps or cascading transform updates for attached components.\n"),
	ECVF_Default
);

int32 GDeferredMovementOutputMode = 0;
static FAutoConsoleVariableRef CVarDeferredMovementOutputMode(
	TEXT("Sequencer.OutputDeferredMovementMode"),
	GDeferredMovementOutputMode,
	TEXT("Integer value specifying how to output Scene Components with deferred movement updates from Sequencer: (0 - Default) No output, (1 - Dump Once) Request a single output on the next evaluation, (2 - Dump every frame) Dump all movement updates every frame"),
	ECVF_Default
);


} // namespace MovieScene
} // namespace UE


USceneComponent* UMovieSceneDeferredComponentMovementSystem::FScopedSequencerMovementUpdate::GetComponent() const
{
	return Owner;
}


UMovieSceneDeferredComponentMovementSystem::UMovieSceneDeferredComponentMovementSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling | ESystemPhase::Finalization;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FMovieSceneTracksComponentTypes* Components = FMovieSceneTracksComponentTypes::Get();

		// This system consumes bound objects
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->BoundObject);

		// Define this system as a producer of the components it oeprates on - this ensures that anything that 
		// manuipulates or interacts with these components will occur after we have deferred updates for them
		DefineComponentProducer(GetClass(), Components->ComponentTransform.PropertyTag);
		DefineComponentProducer(GetClass(), Components->AttachParent);
	}
}

void UMovieSceneDeferredComponentMovementSystem::BeginDestroy()
{
	EnsureMovementsFlushed();
	Super::BeginDestroy();
}

void UMovieSceneDeferredComponentMovementSystem::OnUnlink()
{
	EnsureMovementsFlushed();
}

void UMovieSceneDeferredComponentMovementSystem::EnsureMovementsFlushed()
{
	if (!ensureMsgf(ScopedUpdates.Num() == 0, TEXT("System being destroyed while scoped updates still exist - ApplyMovementUpdates should always have been called at the end of the frame")))
	{
		// Ensure that destruction happens in reverse order
		for (int32 Index = ScopedUpdates.Num()-1; Index >= 0; --Index)
		{
			ScopedUpdates[Index].Reset();
		}

		ScopedUpdates.Empty();
	}
}

void UMovieSceneDeferredComponentMovementSystem::DeferMovementUpdates(USceneComponent* InComponent)
{
	if (!InComponent->IsDeferringMovementUpdates())
	{
		const int32 Index = ScopedUpdates.Add(1);
		ScopedUpdates[Index].Emplace(InComponent);
	}
}

bool UMovieSceneDeferredComponentMovementSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	if (!GDeferSequencerMovementUpdates)
	{
		return false;
	}

	FMovieSceneTracksComponentTypes* Components = FMovieSceneTracksComponentTypes::Get();
	return InLinker->EntityManager.ContainsComponent(Components->ComponentTransform.PropertyTag) ||
		InLinker->EntityManager.ContainsComponent(Components->AttachParent);
}

void UMovieSceneDeferredComponentMovementSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* Components        = FMovieSceneTracksComponentTypes::Get();

	FTaskParams TaskParams = FTaskParams(TEXT("Defer Movement")).ForceGameThread();
	TaskParams.bForcePropagateDownstream = true;

	struct FCacheDeferredUpdates
	{
		UMovieSceneDeferredComponentMovementSystem* System;

		void ForEachEntity(UObject* BoundObject) const
		{
			if (USceneComponent* SceneComponent = Cast<USceneComponent>(BoundObject))
			{
				System->DeferMovementUpdates(SceneComponent);
			}
		}
	};

	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObject)
	.FilterAny({ Components->ComponentTransform.PropertyTag })
	.SetParams(TaskParams)
	.Schedule_PerEntity<FCacheDeferredUpdates>(&Linker->EntityManager, TaskScheduler, this);
}

void UMovieSceneDeferredComponentMovementSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();
	ESystemPhase CurrentPhase = Runner->GetCurrentPhase();

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* Components        = FMovieSceneTracksComponentTypes::Get();

	auto ProcessBoundObject = [this](UObject* BoundObject)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(BoundObject))
		{
			this->DeferMovementUpdates(SceneComponent);
		}
	};

	if (CurrentPhase == ESystemPhase::Instantiation)
	{
		FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.FilterAll({ Components->AttachParent })
		.FilterAny({ BuiltInComponents->Tags.NeedsLink, BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerEntity(&Linker->EntityManager, ProcessBoundObject);
	}
	else if (CurrentPhase == ESystemPhase::Evaluation)
	{
		// Legacy back compat
		FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.FilterAny({ Components->ComponentTransform.PropertyTag })
		.Iterate_PerEntity(&Linker->EntityManager, ProcessBoundObject);
	}
	else if (CurrentPhase == ESystemPhase::Finalization)
	{
		ApplyMovementUpdates();
	}
}

void UMovieSceneDeferredComponentMovementSystem::ApplyMovementUpdates()
{
	OutputDeferredMovements();

	for (int32 Index = ScopedUpdates.Num()-1; Index >= 0; --Index)
	{
		ScopedUpdates[Index].Reset();
	}
	ScopedUpdates.Empty(ScopedUpdates.Num());
}

void UMovieSceneDeferredComponentMovementSystem::OutputDeferredMovements()
{
#if !NO_LOGGING
	using namespace UE::MovieScene;

	if (GDeferredMovementOutputMode == 0 || LogMovieScene.IsSuppressed(ELogVerbosity::Log) || ScopedUpdates.Num() == 0)
	{
		return;
	}

	if (GDeferredMovementOutputMode == 1)
	{
		GDeferredMovementOutputMode = 0;
	}

	TArray<USceneComponent*> NonCollisionComponents;
	TArray<USceneComponent*> CollisionComponents;

	for (const TOptional<FScopedMovementUpdateContainer>& ScopedUpdate : ScopedUpdates)
	{
		USceneComponent* Component = ScopedUpdate->Value.GetComponent();

		if (Component->IsQueryCollisionEnabled())
		{
			CollisionComponents.Add(Component);
		}
		else
		{
			NonCollisionComponents.Add(Component);
		}
	}

	UE_LOGF(LogMovieScene, Log, "Outputting deferred movement updates for %d components from Sequencer:", CollisionComponents.Num() + NonCollisionComponents.Num());

	if (CollisionComponents.Num() != 0)
	{
		UE_LOGF(LogMovieScene, Log, "\t %d Components with collision enabled:", CollisionComponents.Num());

		for (USceneComponent* Component : CollisionComponents)
		{
			UE_LOGF(LogMovieScene, Log, "\t\t %ls.%ls", *Component->GetOwner()->GetName(), *Component->GetName());
		}
	}

	if (NonCollisionComponents.Num() != 0)
	{
		UE_LOGF(LogMovieScene, Log, "\t %d Components without collision:", NonCollisionComponents.Num());

		for (USceneComponent* Component : NonCollisionComponents)
		{
			UE_LOGF(LogMovieScene, Log, "\t\t %ls.%ls", *Component->GetOwner()->GetName(), *Component->GetName());
		}
	}

#endif
}

