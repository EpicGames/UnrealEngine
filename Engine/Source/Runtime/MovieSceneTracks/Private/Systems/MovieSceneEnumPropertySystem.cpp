// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneEnumPropertySystem.h"
#include "Systems/ByteChannelEvaluatorSystem.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEnumPropertySystem)

UMovieSceneEnumPropertySystem::UMovieSceneEnumPropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Enum);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UByteChannelEvaluatorSystem::StaticClass(), GetClass());

		DefineComponentConsumer(GetClass(), UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Enum.PropertyTag);
	}
}

void UMovieSceneEnumPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}


