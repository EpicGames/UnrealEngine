// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneEventSystems.h"
#include "Algo/RemoveIf.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "EntitySystem/MovieSceneSpawnablesSystem.h"

#include "Evaluation/EventContextsPlaybackCapability.h"
#include "Evaluation/EventTriggerControlPlaybackCapability.h"
#include "Evaluation/SequenceDirectorPlaybackCapability.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "MovieSceneTracksComponentTypes.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneSequence.h"

#include "Templates/Greater.h"
#include "Templates/SharedPointer.h"
#include "Templates/SubclassOf.h"
#include "Engine/World.h"
#include "Engine/LevelScriptActor.h"
#include "Logging/TokenizedMessage.h"
#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEventSystems)

#define LOCTEXT_NAMESPACE "MovieSceneEventSystem"

DECLARE_CYCLE_STAT(TEXT("Event Systems"),  MovieSceneEval_Events,        STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Trigger Events"), MovieSceneEval_TriggerEvents, STATGROUP_MovieSceneECS);

UMovieSceneEventSystem::UMovieSceneEventSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

void UMovieSceneEventSystem::AddEvent(UE::MovieScene::FInstanceHandle RootInstance, const FMovieSceneEventTriggerData& TriggerData)
{
	check(TriggerData.Ptrs.Function != nullptr);
	EventsByRoot.FindOrAdd(RootInstance).Add(TriggerData);
}

bool UMovieSceneEventSystem::HasEvents() const
{
	return EventsByRoot.Num() != 0;
}

bool UMovieSceneEventSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	return HasEvents();
}

void UMovieSceneEventSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	if (EventsByRoot.Num() > 0)
	{
		TriggerAllEvents();
	}
}

void UMovieSceneEventSystem::OnUnlink()
{
	if (!ensure(EventsByRoot.Num() == 0))
	{
		EventsByRoot.Reset();
	}
}

void UMovieSceneEventSystem::TriggerAllEvents()
{
	using namespace UE::MovieScene;

	SCOPE_CYCLE_COUNTER(MovieSceneEval_Events);

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	struct FTriggerBatch
	{
		FTriggerBatch(TSharedRef<const FSharedPlaybackState> InSharedPlaybackState) : SharedPlaybackState(InSharedPlaybackState) {}
		TArray<FMovieSceneEventTriggerData> Triggers;
		TSharedRef<const FSharedPlaybackState> SharedPlaybackState;
	};
	TArray<FTriggerBatch> TriggerBatches;

	for (TPair<FInstanceHandle, TArray<FMovieSceneEventTriggerData>>& Pair : EventsByRoot)
	{
		const FSequenceInstance& RootInstance = InstanceRegistry->GetInstance(Pair.Key);
		TSharedRef<const FSharedPlaybackState> SharedPlaybackState = RootInstance.GetSharedPlaybackState();

		FFrameTime SkipUntil;
		bool bSkipTrigger = false;
		if (FEventTriggerControlPlaybackCapability* TriggerControlCapability = SharedPlaybackState->FindCapability<FEventTriggerControlPlaybackCapability>())
		{
			bSkipTrigger = TriggerControlCapability->IsDisablingEventTriggers(SkipUntil);
		}
		else if (IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState))
		{
			bSkipTrigger = Player->IsDisablingEventTriggers(SkipUntil);
		}

		if (RootInstance.GetContext().GetDirection() == EPlayDirection::Forwards)
		{
			if (bSkipTrigger)
			{
				Pair.Value.SetNum(Algo::RemoveIf(Pair.Value, [SkipUntil](FMovieSceneEventTriggerData& Data) { return Data.RootTime <= SkipUntil; }));
			}
			Algo::SortBy(Pair.Value, &FMovieSceneEventTriggerData::RootTime);
		}
		else
		{
			if (bSkipTrigger)
			{
				Pair.Value.SetNum(Algo::RemoveIf(Pair.Value, [SkipUntil](FMovieSceneEventTriggerData& Data) { return Data.RootTime >= SkipUntil; }));
			}
			Algo::SortBy(Pair.Value, &FMovieSceneEventTriggerData::RootTime, TGreater<>());
		}

		FTriggerBatch& TriggerBatch = TriggerBatches.Emplace_GetRef(SharedPlaybackState);
		TriggerBatch.Triggers = Pair.Value;
	}

	// We need to clean our state before actually triggering the events because one of those events could
	// call back into an evaluation (for instance, by starting play on another sequence). If we don't clean
	// this before, would would re-enter and re-trigger past events, resulting in an infinite loop!
	EventsByRoot.Empty();

	for (const FTriggerBatch& TriggerBatch : TriggerBatches)
	{
		TriggerEvents(TriggerBatch.Triggers, TriggerBatch.SharedPlaybackState);
	}
}

void UMovieSceneEventSystem::TriggerEvents(TArrayView<const FMovieSceneEventTriggerData> Events, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	using namespace UE::MovieScene;

#if !NO_LOGGING
	UMovieSceneSequence* PlayerSequence = SharedPlaybackState->GetRootSequence();
	FString SequenceName = PlayerSequence->GetName();
	UE_LOGF(LogMovieScene, VeryVerbose, "%ls: Triggering %d events", *SequenceName, Events.Num());
#endif

	// Auto-add the director playback capability, which is just really a cache for director instances after
	// they've been created by the sequences in the hierarchy.
	FSequenceDirectorPlaybackCapability* DirectorCapability = SharedPlaybackState->FindCapability<FSequenceDirectorPlaybackCapability>();
	if (!DirectorCapability)
	{
		TSharedRef<FSharedPlaybackState> MutableState = ConstCastSharedRef<FSharedPlaybackState>(SharedPlaybackState);
		DirectorCapability = &MutableState->AddCapability<FSequenceDirectorPlaybackCapability>();
	}

	TArray<UObject*> GlobalContexts;
	if (IEventContextsPlaybackCapability* EventContextsCapability = SharedPlaybackState->FindCapability<IEventContextsPlaybackCapability>())
	{
		// Query the playback capability first.
		GlobalContexts = EventContextsCapability->GetEventContexts();
	}
	else if (IMovieScenePlayer* Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState))
	{
		// Check the player interface first. If implemented, we get what we want. If not, it will fall back to the default
		// implementation that queries the playback capabality (see below), which covers the cases of a player that uses
		// the new API.
		GlobalContexts = Player->GetEventContexts();
	}

	for (const FMovieSceneEventTriggerData& Event : Events)
	{
		SCOPE_CYCLE_COUNTER(MovieSceneEval_TriggerEvents);

		UObject* DirectorInstance = DirectorCapability->GetOrCreateDirectorInstance(SharedPlaybackState, Event.SequenceID);
		if (!DirectorInstance)
		{
#if !NO_LOGGING
			UE_LOGF(LogMovieScene, Warning, "Failed to trigger event '%ls' because no director instance was available.", *Event.Ptrs.Function->GetName());
#endif
			continue;
		}

#if !NO_LOGGING
		UE_LOGF(LogMovieScene, VeryVerbose, "%ls: - Triggering event at frame %d, subframe %f", *SequenceName, Event.RootTime.FrameNumber.Value, Event.RootTime.GetSubFrame());
#endif

#if WITH_EDITOR
		const static FName NAME_CallInEditor("CallInEditor");

		UWorld* World = DirectorInstance->GetWorld();
		const bool bIsGameWorld = World && World->IsGameWorld();
#endif // WITH_EDITOR



#if WITH_EDITOR
		if (!bIsGameWorld && !Event.Ptrs.Function->HasMetaData(NAME_CallInEditor))
		{
			UE_LOGF(LogMovieScene, Verbose, "Refusing to trigger event '%ls' in editor world when 'Call in Editor' is false.", *Event.Ptrs.Function->GetName());
			continue;
		}
#endif // WITH_EDITOR


		UE_LOGF(LogMovieScene, VeryVerbose, "Triggering event '%ls'.", *Event.Ptrs.Function->GetName());

		if (Event.Ptrs.Function->NumParms == 0 && !Event.ObjectBindingID.IsValid())
		{
			DirectorInstance->ProcessEvent(Event.Ptrs.Function, nullptr);
		}
		else
		{
			TriggerEventWithParameters(DirectorInstance, Event, GlobalContexts, SharedPlaybackState);
		}
	}
}

void UMovieSceneEventSystem::TriggerEventWithParameters(UObject* DirectorInstance, const FMovieSceneEventTriggerData& Event, TArrayView<UObject* const> GlobalContexts, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
{
	const FMovieSceneEventPtrs& EventPtrs = Event.Ptrs;
	if (!ensureMsgf(
				!EventPtrs.BoundObjectProperty.Get() || 
				(EventPtrs.BoundObjectProperty->GetOwner<UObject>() == EventPtrs.Function && 
					EventPtrs.BoundObjectProperty->GetOffset_ForUFunction() < EventPtrs.Function->ParmsSize),
		TEXT("Bound object property belongs to the wrong function or has an offset greater than the parameter size!"
			 "This should never happen and indicates a BP compilation or nativization error.")))
	{
		return;
	}

	FMovieSceneEvaluationState* State = SharedPlaybackState->FindCapability<FMovieSceneEvaluationState>();
	if (!ensureMsgf(State, TEXT("No evaluation state found!")))
	{
		return;
	}

	// Parse all function parameters.
	uint8* Parameters = (uint8*)FMemory_Alloca(EventPtrs.Function->ParmsSize + EventPtrs.Function->MinAlignment);
	Parameters = Align(Parameters, EventPtrs.Function->MinAlignment);

	// Mem zero the parameter list
	FMemory::Memzero(Parameters, EventPtrs.Function->ParmsSize);

	// Initialize all CPF_Param properties - these are aways at the head of the list
	for (TFieldIterator<FProperty> It(EventPtrs.Function); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
	{
		FProperty* LocalProp = *It;
		checkSlow(LocalProp);
		if (!LocalProp->HasAnyPropertyFlags(CPF_ZeroConstructor))
		{
			LocalProp->InitializeValue_InContainer(Parameters);
		}
	}

	FProperty* BoundObjectProperty = EventPtrs.BoundObjectProperty.Get();

	// If the event exists on an object binding, only call the events for those bindings (never for the global contexts)
	if (Event.ObjectBindingID.IsValid())
	{
		for (TWeakObjectPtr<> WeakBoundObject : State->FindBoundObjects(Event.ObjectBindingID, Event.SequenceID, SharedPlaybackState))
		{
			if (UObject* BoundObject = WeakBoundObject.Get())
			{
				// Attempt to bind the object to the function parameters
				if (PatchBoundObject(Parameters, BoundObject, EventPtrs.BoundObjectProperty.Get(), SharedPlaybackState, Event.SequenceID))
				{
					DirectorInstance->ProcessEvent(EventPtrs.Function, Parameters);
				}
			}
		}
	}

	// At this point we know the event is a track with parameters - either trigger for global contexts, or just on its own
	else if (GlobalContexts.Num() != 0)
	{
		for (UObject* Context : GlobalContexts)
		{
			if (PatchBoundObject(Parameters, Context, EventPtrs.BoundObjectProperty.Get(), SharedPlaybackState, Event.SequenceID))
			{
				DirectorInstance->ProcessEvent(EventPtrs.Function, Parameters);
			}
		}
	}
	else
	{
		DirectorInstance->ProcessEvent(EventPtrs.Function, Parameters);
	}

	// Destroy all parameter properties one by one
	for (TFieldIterator<FProperty> It(EventPtrs.Function); It && It->HasAnyPropertyFlags(CPF_Parm); ++It)
	{
		It->DestroyValue_InContainer(Parameters);
	}
}

bool UMovieSceneEventSystem::PatchBoundObject(uint8* Parameters, UObject* BoundObject, FProperty* BoundObjectProperty, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID)
{
	if (!BoundObjectProperty)
	{
		return true;
	}

	if (FInterfaceProperty* InterfaceParameter = CastField<FInterfaceProperty>(BoundObjectProperty))
		{
		if (BoundObject->GetClass()->ImplementsInterface(InterfaceParameter->InterfaceClass))
		{
			FScriptInterface Interface;
			Interface.SetObject(BoundObject);
			Interface.SetInterface(BoundObject->GetInterfaceAddress(InterfaceParameter->InterfaceClass));
			InterfaceParameter->SetPropertyValue_InContainer(Parameters, Interface);
			return true;
		}

		FMessageLog("PIE").Warning()
			->AddToken(FUObjectToken::Create(BoundObjectProperty->GetOwnerUObject()))
			->AddToken(FUObjectToken::Create(SharedPlaybackState->GetSequence(SequenceID)))
			->AddToken(FTextToken::Create(FText::Format(LOCTEXT("LevelBP_InterfaceNotImplemented_Error", "Failed to trigger event because it does not implement the necessary interface. Function expects a '{0}'."), FText::FromName(InterfaceParameter->InterfaceClass->GetFName()))));
		return false;
	}

	if (FObjectProperty* ObjectParameter = CastField<FObjectProperty>(BoundObjectProperty))
	{
		if (BoundObject->IsA<ALevelScriptActor>())
		{
			FMessageLog("PIE").Warning()
				->AddToken(FUObjectToken::Create(BoundObjectProperty->GetOwnerUObject()))
				->AddToken(FUObjectToken::Create(SharedPlaybackState->GetSequence(SequenceID)))
				->AddToken(FTextToken::Create(LOCTEXT("LevelBP_LevelScriptActor_Error", "Failed to trigger event: only Interface pins are supported for root tracks within Level Sequences. Please remove the pin, or change it to an interface that is implemented on the desired level blueprint.")));

			return false;
		}
		else if (!BoundObject->IsA(ObjectParameter->PropertyClass))
		{
			FMessageLog("PIE").Warning()
				->AddToken(FUObjectToken::Create(SharedPlaybackState->GetSequence(SequenceID)))
				->AddToken(FUObjectToken::Create(BoundObjectProperty->GetOwnerUObject()))
				->AddToken(FUObjectToken::Create(BoundObject))
				->AddToken(FTextToken::Create(FText::Format(LOCTEXT("LevelBP_InvalidCast_Error", "Failed to trigger event: Cast to {0} failed."), FText::FromName(ObjectParameter->PropertyClass->GetFName()))));

			return false;
		}

		ObjectParameter->SetObjectPropertyValue_InContainer(Parameters, BoundObject);
		return true;
	}

	FMessageLog("PIE").Warning()
		->AddToken(FUObjectToken::Create(SharedPlaybackState->GetSequence(SequenceID)))
		->AddToken(FUObjectToken::Create(BoundObjectProperty->GetOwnerUObject()))
		->AddToken(FUObjectToken::Create(BoundObject))
		->AddToken(FTextToken::Create(FText::Format(LOCTEXT("LevelBP_UnsupportedProperty_Error", "Failed to trigger event: Unsupported property type for bound object: {0}."), FText::FromName(BoundObjectProperty->GetClass()->GetFName()))));
	return false;
}


UMovieScenePreSpawnEventSystem::UMovieScenePreSpawnEventSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Phase = UE::MovieScene::ESystemPhase::Spawn;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(GetClass(), UMovieSceneSpawnablesSystem::StaticClass());
	}
}

UMovieScenePostSpawnEventSystem::UMovieScenePostSpawnEventSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Phase = UE::MovieScene::ESystemPhase::Spawn;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneSpawnablesSystem::StaticClass(), GetClass());
	}
}

UMovieScenePostEvalEventSystem::UMovieScenePostEvalEventSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Phase = UE::MovieScene::ESystemPhase::Finalization;
}

void UMovieScenePostEvalEventSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	if (HasEvents())
	{
		TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();
		Runner->GetQueuedEventTriggers().AddUObject(this, &UMovieScenePostEvalEventSystem::TriggerAllEvents);
	}
}

#undef LOCTEXT_NAMESPACE

