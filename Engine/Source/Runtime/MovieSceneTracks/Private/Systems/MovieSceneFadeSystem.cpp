// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneFadeSystem.h"

#include "Async/TaskGraphInterfaces.h"
#include "Engine/Engine.h"
#include "Engine/GameInstance.h"
#include "EngineGlobals.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "Evaluation/ViewportSettingsPlaybackCapability.h"
#include "GameFramework/PlayerController.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneExecutionToken.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneFadeSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneFadeSystem)

namespace UE::MovieScene
{

struct FViewportSettingsPlaybackCapabilityCompatibilityWrapper
{
	FViewportSettingsPlaybackCapabilityCompatibilityWrapper(TSharedRef<FSharedPlaybackState> SharedPlaybackState)
	{
		ViewportSettingsCapability = SharedPlaybackState->FindCapability<FViewportSettingsPlaybackCapability>();
		Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
	}

	void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap)
	{
		if (ViewportSettingsCapability)
		{
			ViewportSettingsCapability->SetViewportSettings(ViewportParamsMap);
		}
		else if (Player)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Player->SetViewportSettings(ViewportParamsMap);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const
	{
		if (ViewportSettingsCapability)
		{
			ViewportSettingsCapability->GetViewportSettings(ViewportParamsMap);
		}
		else if (Player)
		{
			PRAGMA_DISABLE_DEPRECATION_WARNINGS
			Player->GetViewportSettings(ViewportParamsMap);
			PRAGMA_ENABLE_DEPRECATION_WARNINGS
		}
	}

	FViewportSettingsPlaybackCapability* ViewportSettingsCapability;
	IMovieScenePlayer* Player;
};

struct FFadeUtil
{
	static void ApplyFade(TSharedRef<FSharedPlaybackState> SharedPlaybackState, float FadeValue, const FLinearColor& FadeColor, bool bFadeAudio)
	{
		// Set editor preview/fade
		EMovieSceneViewportParams ViewportParams;
		ViewportParams.SetWhichViewportParam = ( EMovieSceneViewportParams::SetViewportParam )( EMovieSceneViewportParams::SVP_FadeAmount | EMovieSceneViewportParams::SVP_FadeColor );
		ViewportParams.FadeAmount = FadeValue;
		ViewportParams.FadeColor = FadeColor;

		FViewportSettingsPlaybackCapabilityCompatibilityWrapper ViewportSettingsCapability(SharedPlaybackState);
		TMap<FViewportClient*, EMovieSceneViewportParams> ViewportParamsMap;
		ViewportSettingsCapability.GetViewportSettings(ViewportParamsMap);
		for (auto ViewportParamsPair : ViewportParamsMap)
		{
			ViewportParamsMap[ViewportParamsPair.Key] = ViewportParams;
		}
		ViewportSettingsCapability.SetViewportSettings(ViewportParamsMap);

		// Set runtime fade
		UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext();
		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
		if( World && ( World->WorldType == EWorldType::Game || World->WorldType == EWorldType::PIE ) )
		{
			APlayerController* PlayerController = World->GetGameInstance()->GetFirstLocalPlayerController();
			if( PlayerController != nullptr && IsValid(PlayerController->PlayerCameraManager) )
			{
				PlayerController->PlayerCameraManager->SetManualCameraFade( FadeValue, FadeColor, bFadeAudio );
			}
		}
	}
};

struct FPreAnimatedFadeState
{
	float FadeValue;
	FLinearColor FadeColor;
	bool bFadeAudio;

	static FPreAnimatedFadeState SaveState(UObject* PlaybackContext)
	{
		float FadeAmount = 0.f;
		FLinearColor FadeColor = FLinearColor::Black;
		bool bFadeAudio = false;

		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
		if (World && (World->WorldType == EWorldType::Game || World->WorldType == EWorldType::PIE))
		{
			APlayerController* PlayerController = World->GetGameInstance()->GetFirstLocalPlayerController();
			if (PlayerController != nullptr && IsValid(PlayerController->PlayerCameraManager))
			{
				FadeAmount = PlayerController->PlayerCameraManager->FadeAmount;
				FadeColor = PlayerController->PlayerCameraManager->FadeColor;
				bFadeAudio = PlayerController->PlayerCameraManager->bFadeAudio;
			}
		}

		return FPreAnimatedFadeState{ FadeAmount, FadeColor, bFadeAudio };
	}

	void RestoreState(const FMovieSceneAnimTypeID& Unused, const FRestoreStateParams& Params)
	{
		TSharedPtr<FSharedPlaybackState> SharedPlaybackState = Params.GetTerminalPlaybackState();
		if (!ensure(SharedPlaybackState))
		{
			return;
		}
		
		FFadeUtil::ApplyFade(SharedPlaybackState.ToSharedRef(), FadeValue, FadeColor, bFadeAudio);
	}
};

struct FPreAnimatedFadeStateStorage : TSimplePreAnimatedStateStorage<FMovieSceneAnimTypeID, FPreAnimatedFadeState>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedFadeStateStorage> StorageID;

	FPreAnimatedStorageID GetStorageType() const override { return StorageID; }
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedFadeStateStorage> FPreAnimatedFadeStateStorage::StorageID;

struct FEvaluateFade
{
	const FInstanceRegistry* InstanceRegistry;
	TSharedPtr<FPreAnimatedFadeStateStorage> PreAnimatedStorage;

	FEvaluateFade(const FInstanceRegistry* InInstanceRegistry, TSharedPtr<FPreAnimatedFadeStateStorage> InPreAnimatedStorage)
		: InstanceRegistry(InInstanceRegistry)
		, PreAnimatedStorage(InPreAnimatedStorage)
	{}

	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> RootInstanceHandles, TRead<FFadeComponentData> FadeComponents, TRead<double> FadeAmounts) const
	{
		static const TMovieSceneAnimTypeID<FEvaluateFade> AnimTypeID;

		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		const int32 Num = Allocation->Num();
		const bool bWantsRestoreState = Allocation->HasComponent(BuiltInComponents->Tags.RestoreState);
		const FMovieSceneAnimTypeID Key = AnimTypeID;

		for (int32 Index = 0; Index < Num; ++Index)
		{
			FRootInstanceHandle RootInstanceHandle = RootInstanceHandles[Index];
			const FSequenceInstance& Instance = InstanceRegistry->GetInstance(RootInstanceHandle);
			UObject* PlaybackContext = Instance.GetSharedPlaybackState()->GetPlaybackContext();

			PreAnimatedStorage->BeginTrackingEntity(EntityIDs[Index], bWantsRestoreState, RootInstanceHandle, Key);
			PreAnimatedStorage->CachePreAnimatedValue(Key, [PlaybackContext](const FMovieSceneAnimTypeID&) { return FPreAnimatedFadeState::SaveState(PlaybackContext); });

			const FFadeComponentData& FadeComponent(FadeComponents[Index]);
			FFadeUtil::ApplyFade(Instance.GetSharedPlaybackState(), FadeAmounts[Index], FadeComponent.FadeColor, FadeComponent.bFadeAudio);
		}
	}
};

} // namespace UE::MovieScene

UMovieSceneFadeSystem::UMovieSceneFadeSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	RelevantComponent = FMovieSceneTracksComponentTypes::Get()->Fade;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		DefineComponentConsumer(GetClass(), BuiltInComponents->DoubleResult[0]);
	}
}

void UMovieSceneFadeSystem::OnLink()
{
	using namespace UE::MovieScene;

	PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedFadeStateStorage>();
}

void UMovieSceneFadeSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(TrackComponents->Fade)
	.Read(BuiltInComponents->DoubleResult[0])
	.SetDesiredThread(Linker->EntityManager.GetGatherThread())
	.Schedule_PerAllocation<FEvaluateFade>(& Linker->EntityManager, TaskScheduler, Linker->GetInstanceRegistry(), PreAnimatedStorage);
}

void UMovieSceneFadeSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(TrackComponents->Fade)
	.Read(BuiltInComponents->DoubleResult[0])
	.SetDesiredThread(Linker->EntityManager.GetGatherThread())
	.Dispatch_PerAllocation<FEvaluateFade>(& Linker->EntityManager, InPrerequisites, &Subsequents, Linker->GetInstanceRegistry(), PreAnimatedStorage);
}

