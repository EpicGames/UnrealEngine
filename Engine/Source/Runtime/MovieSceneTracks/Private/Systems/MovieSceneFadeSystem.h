// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntitySystem.h"

#include "MovieSceneFadeSystem.generated.h"

namespace UE::MovieScene
{
	struct FPreAnimatedFadeStateStorage;
}

UCLASS()
class UMovieSceneFadeSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:
	UMovieSceneFadeSystem(const FObjectInitializer& ObjInit);

	virtual void OnLink() override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:
	TSharedPtr<UE::MovieScene::FPreAnimatedFadeStateStorage> PreAnimatedStorage;
};

