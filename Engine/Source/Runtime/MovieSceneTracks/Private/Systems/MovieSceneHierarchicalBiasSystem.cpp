// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneHierarchicalBiasSystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityGroupingSystem.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "Systems/WeightAndEasingEvaluatorSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneHierarchicalBiasSystem)


namespace UE::MovieScene
{
	/** Temporary struct used for collating hbias meta data for each group */
	struct FHBiasMetaData
	{
		UE::MovieScene::FHierarchicalBlendTarget BlendTarget;
		int16 HBias;
		uint8 bBlendHierarchicalBias : 1;
		uint8 bInUse : 1;

		FHBiasMetaData()
		{
			bBlendHierarchicalBias = false;
			bInUse = false;
			HBias = TNumericLimits<int16>::Lowest();
		}
	};

	/** Mutation that adds or removes the Ignored tag for entities */
	struct FToggleIgnoredMutation : IMovieSceneConditionalEntityMutation
	{
		TArrayView<const FHBiasMetaData> HBiasMetaData;

		FToggleIgnoredMutation(TArrayView<const FHBiasMetaData> InHBiasMetaData)
			: HBiasMetaData(InHBiasMetaData)
		{}

		void MarkAllocation(FEntityAllocation* Allocation, TBitArray<>& OutEntitiesToMutate) const override
		{
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

			const bool bCurrentlyIgnored = Allocation->HasComponent(BuiltInComponents->Tags.Ignored);

			TComponentReader<FEntityGroupID> GroupComponents = Allocation->ReadComponents(BuiltInComponents->Group);
			TOptionalComponentReader<int16>  HBiasComponents = Allocation->TryReadComponents(BuiltInComponents->HierarchicalBias);

			const int32 Num = Allocation->Num();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const int16 HBias = HBiasComponents ? HBiasComponents[Index] : 0;

				if (!EnumHasAnyFlags(GroupComponents[Index].Flags, EEntityGroupFlags::RemovedFromGroup))
				{
					const FHBiasMetaData& MetaData = HBiasMetaData[GroupComponents[Index].GroupIndex];
					const bool bShouldBeIgnored = (!MetaData.bBlendHierarchicalBias && MetaData.HBias > HBias);
				
					if (bShouldBeIgnored != bCurrentlyIgnored)
					{
						OutEntitiesToMutate.PadToNum(Index + 1, false);
						OutEntitiesToMutate[Index] = true;
					}
				}
			}
		}

		void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
		{
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

			if (InOutEntityComponentTypes->Contains(BuiltInComponents->Tags.Ignored))
			{
				InOutEntityComponentTypes->Remove(BuiltInComponents->Tags.Ignored);
			}
			else
			{
				InOutEntityComponentTypes->Set(BuiltInComponents->Tags.Ignored);
			}

			InOutEntityComponentTypes->Set(BuiltInComponents->Tags.NeedsLink);
		}
	};

	/** Mutation that adds, removes or assigns the HierarchicalBlendTarget components for entities */
	struct FBlendTargetMutation : IMovieSceneConditionalEntityMutation
	{
		TArrayView<const FHBiasMetaData> HBiasMetaData;
		FEntityAllocationWriteContext WriteContext;

		FBlendTargetMutation(TArrayView<const FHBiasMetaData> InHBiasMetaData, FEntityAllocationWriteContext InWriteContext)
			: HBiasMetaData(InHBiasMetaData)
			, WriteContext(InWriteContext)
		{}

		void MarkAllocation(FEntityAllocation* Allocation, TBitArray<>& OutEntitiesToMutate) const override
		{
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

			TOptionalComponentWriter<FHierarchicalBlendTarget> OutBlendTargets = Allocation->TryWriteComponents(BuiltInComponents->HierarchicalBlendTarget, WriteContext);
			TComponentReader<FEntityGroupID>                   GroupComponents = Allocation->ReadComponents(BuiltInComponents->Group);

			const bool bIsIgnored = Allocation->HasComponent(BuiltInComponents->Tags.Ignored);
			const bool bHasBlendTarget = !!OutBlendTargets;

			const int32 Num = Allocation->Num();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				if (!EnumHasAnyFlags(GroupComponents[Index].Flags, EEntityGroupFlags::RemovedFromGroup))
				{
					const int32  GroupIndex = GroupComponents[Index].GroupIndex;
					const FHBiasMetaData& MetaData = HBiasMetaData[GroupIndex];

					const bool bNeedsBlendTarget = !bIsIgnored && MetaData.bBlendHierarchicalBias;
					if (bNeedsBlendTarget != bHasBlendTarget)
					{
						OutEntitiesToMutate.PadToNum(Index + 1, false);
						OutEntitiesToMutate[Index] = true;
					}
					else if (OutBlendTargets)
					{
						OutBlendTargets[Index] = HBiasMetaData[GroupIndex].BlendTarget;
					}
				}
			}
		}

		void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
		{
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

			if (InOutEntityComponentTypes->Contains(BuiltInComponents->HierarchicalBlendTarget))
			{
				InOutEntityComponentTypes->Remove(BuiltInComponents->HierarchicalBlendTarget);
			}
			else
			{
				InOutEntityComponentTypes->Set(BuiltInComponents->HierarchicalBlendTarget);
			}

			InOutEntityComponentTypes->Set(BuiltInComponents->Tags.NeedsLink);
		}

		void InitializeEntities(const FEntityRange& EntityRange, const FComponentMask& AllocationType) const override
		{
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

			TOptionalComponentWriter<FHierarchicalBlendTarget> BlendTargets = EntityRange.Allocation->TryWriteComponents(BuiltInComponents->HierarchicalBlendTarget, FEntityAllocationWriteContext::NewAllocation());
			if (BlendTargets)
			{
				TComponentReader<FEntityGroupID> GroupComponents = EntityRange.Allocation->ReadComponents(BuiltInComponents->Group);

				for (int32 Index = 0; Index < EntityRange.Num; ++Index)
				{
					const FHBiasMetaData& MetaData = HBiasMetaData[GroupComponents[EntityRange.ComponentStartOffset + Index].GroupIndex];

					BlendTargets[Index] = MetaData.BlendTarget;
				}
			}
		}

	};

} // namespace UE::MovieScene


UMovieSceneHierarchicalBiasSystem::UMovieSceneHierarchicalBiasSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	GroupingSystem = nullptr;
	SystemCategories = EEntitySystemCategory::Core;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		DefineComponentConsumer(GetClass(), BuiltInComponents->Group);

		DefineComponentProducer(GetClass(), BuiltInComponents->Tags.Ignored);
		DefineComponentProducer(GetClass(), BuiltInComponents->HierarchicalBlendTarget);
		DefineImplicitPrerequisite(GetClass(), UMovieSceneHierarchicalEasingInstantiatorSystem::StaticClass());
	}
}

bool UMovieSceneHierarchicalBiasSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	return InLinker->EntityManager.ContainsAllComponents({ Components->Group, Components->HierarchicalBias });
}

void UMovieSceneHierarchicalBiasSystem::OnLink()
{
	GroupingSystem = Linker->LinkSystem<UMovieSceneEntityGroupingSystem>();
	Linker->SystemGraph.AddReference(this, GroupingSystem.Get());
}

void UMovieSceneHierarchicalBiasSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	TArray<FHBiasMetaData> HBiasMetaData;
	HBiasMetaData.SetNum(GroupingSystem->NumGroups());

	auto GatherHierarchicalMetaData = [&HBiasMetaData, BuiltInComponents](FEntityAllocationIteratorItem Item, const FEntityGroupID* GroupIDs, const int16* OptionalHBias)
	{
		const FComponentMask& AllocationType = Item.GetAllocationType();

		const int32 Num = Item.GetAllocation()->Num();

		const bool bBlendHBias = AllocationType.Contains(BuiltInComponents->Tags.BlendHierarchicalBias);

		if (AllocationType.Contains(BuiltInComponents->Tags.IgnoreHierarchicalBias))
		{
			for (int32 Index = 0; Index < Num; ++Index)
			{
				if (!EnumHasAnyFlags(GroupIDs[Index].Flags, EEntityGroupFlags::RemovedFromGroup))
				{
					const int32 GroupIndex = GroupIDs[Index].GroupIndex;

					FHBiasMetaData& MetaData = HBiasMetaData[GroupIndex];

					MetaData.bInUse = true;
					MetaData.bBlendHierarchicalBias |= bBlendHBias;
				}
			}
		}
		else
		{
			for (int32 Index = 0; Index < Num; ++Index)
			{
				if (!EnumHasAnyFlags(GroupIDs[Index].Flags, EEntityGroupFlags::RemovedFromGroup))
				{
					const int32 GroupIndex = GroupIDs[Index].GroupIndex;
					const int16 ThisHBias  = OptionalHBias ? OptionalHBias[Index] : 0;

					FHBiasMetaData& MetaData = HBiasMetaData[GroupIndex];

					if (ThisHBias > MetaData.HBias)
					{
						MetaData.HBias = ThisHBias;
					}

					MetaData.bInUse = true;
					MetaData.bBlendHierarchicalBias |= bBlendHBias;
					MetaData.BlendTarget.Add(ThisHBias);
				}
			}
		}
	};

	// --------------------------------------------------------------------------
	// Step 1: Gather hbias meta-data for each group
	FEntityTaskBuilder()
	.Read(BuiltInComponents->Group)
	.ReadOptional(BuiltInComponents->HierarchicalBias)
	.Iterate_PerAllocation(&Linker->EntityManager, GatherHierarchicalMetaData);

	// --------------------------------------------------------------------------
	// Step 2: Toggle non-blended entities that are part of lower-hbias
	FToggleIgnoredMutation ToggleIgnoredMutation(HBiasMetaData);
	Linker->EntityManager.MutateConditional(FEntityComponentFilter().All({ BuiltInComponents->Group }), ToggleIgnoredMutation);

	// --------------------------------------------------------------------------
	// Step 3: Update blend targets on blended entities
	FBlendTargetMutation ToggleBlendTargetMutation(HBiasMetaData, FEntityAllocationWriteContext(Linker->EntityManager));
	Linker->EntityManager.MutateConditional(
		FEntityComponentFilter().Any({ BuiltInComponents->Group, BuiltInComponents->HierarchicalBlendTarget }), ToggleBlendTargetMutation);
}

