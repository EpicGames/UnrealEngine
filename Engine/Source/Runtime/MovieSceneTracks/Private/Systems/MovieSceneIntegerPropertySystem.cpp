// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneIntegerPropertySystem.h"
#include "Systems/IntegerChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseIntegerBlenderSystem.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneIntegerPropertySystem)

UMovieSceneIntegerPropertySystem::UMovieSceneIntegerPropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Integer);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieScenePiecewiseIntegerBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UIntegerChannelEvaluatorSystem::StaticClass(), GetClass());

		DefineComponentConsumer(GetClass(), UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Integer.PropertyTag);
	}
}

void UMovieSceneIntegerPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}


