// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneLevelVisibilitySystem.h"

#include "Engine/Level.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneRootInstantiatorSystem.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "EntitySystem/MovieSceneSpawnablesSystem.h"
#include "MovieSceneExecutionToken.h"
#include "MovieSceneTracksComponentTypes.h"

#include "Misc/PackageName.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"

#include "Algo/Find.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLevelVisibilitySystem)

namespace UE
{
namespace MovieScene
{

bool GetLevelVisibility(const ULevelStreaming& Level)
{
#if WITH_EDITOR
	if (GIsEditor && !Level.GetWorld()->IsPlayInEditor())
	{
		return Level.GetShouldBeVisibleInEditor();
	}
	else
#endif
	{
		return Level.ShouldBeVisible();
	}
}

void SetLevelVisibility(ULevelStreaming& Level, bool bVisible, EFlushLevelStreamingType* FlushStreamingType = nullptr)
{
#if WITH_EDITOR
	if (GIsEditor && !Level.GetWorld()->IsPlayInEditor())
	{
		Level.SetShouldBeVisibleInEditor(bVisible);
		Level.GetWorld()->FlushLevelStreaming();

		// Iterate over the level's actors
		ULevel* LoadedLevel = Level.GetLoadedLevel();
		if (LoadedLevel != nullptr)
		{
			auto& Actors = LoadedLevel->Actors;
			for ( int32 ActorIndex = 0; ActorIndex < Actors.Num(); ++ActorIndex )
			{
				AActor* Actor = Actors[ActorIndex];
				if ( Actor )
				{
					if (Actor->bHiddenEdLevel == bVisible )
					{
						Actor->bHiddenEdLevel = !bVisible;
						if ( bVisible )
						{
							Actor->ReregisterAllComponents();
						}
						else
						{
							Actor->UnregisterAllComponents();
						}
					}
				}
			}
		}
	}
	else
#endif
	{
		Level.SetShouldBeVisible(bVisible);

		if (FlushStreamingType && (*FlushStreamingType == EFlushLevelStreamingType::None))
		{
			*FlushStreamingType = EFlushLevelStreamingType::Visibility;
		}

		if (bVisible && !Level.IsLevelLoaded())
		{
			Level.SetShouldBeLoaded(true);
			if (FlushStreamingType)
			{
				*FlushStreamingType = EFlushLevelStreamingType::Full;
			}
		}
	}
}

// TODO: This was copied from LevelStreaming.cpp, it should be in a set of shared utilities somewhere.
FString MakeSafeLevelName(const FName& InLevelName, UWorld& World)
{
	// Special case for PIE, the PackageName gets mangled.
	if (!World.StreamingLevelsPrefix.IsEmpty())
	{
		FString PackageName = World.StreamingLevelsPrefix + FPackageName::GetShortName(InLevelName);
		if (!FPackageName::IsShortPackageName(InLevelName))
		{
			PackageName = FPackageName::GetLongPackagePath(InLevelName.ToString()) + TEXT( "/" ) + PackageName;
		}

		return PackageName;
	}

	return InLevelName.ToString();
}

ULevelStreaming* GetStreamingLevel(FString SafeLevelName, UWorld& World)
{
	if (FPackageName::IsShortPackageName(SafeLevelName))
	{
		// Make sure MyMap1 and Map1 names do not resolve to a same streaming level
		SafeLevelName = TEXT("/") + SafeLevelName;
	}

	for (ULevelStreaming* LevelStreaming : World.GetStreamingLevels())
	{
		if (LevelStreaming && LevelStreaming->GetWorldAssetPackageName().EndsWith(SafeLevelName, ESearchCase::IgnoreCase))
		{
			return LevelStreaming;
		}
	}

	return nullptr;
}

struct FLevelStreamingPreAnimatedToken : IMovieScenePreAnimatedToken
{
	FLevelStreamingPreAnimatedToken(bool bInIsVisible)
		: bVisible(bInIsVisible)
	{
	}

	virtual void RestoreState(UObject& Object, const UE::MovieScene::FRestoreStateParams& Params) override
	{
		ULevelStreaming* LevelStreaming = CastChecked<ULevelStreaming>(&Object);
		SetLevelVisibility(*LevelStreaming, bVisible);
	}

	bool bVisible;
};

struct FLevelStreamingPreAnimatedTokenProducer : IMovieScenePreAnimatedTokenProducer
{
	virtual IMovieScenePreAnimatedTokenPtr CacheExistingState(UObject& Object) const override
	{
		ULevelStreaming* LevelStreaming = CastChecked<ULevelStreaming>(&Object);
		return FLevelStreamingPreAnimatedToken(GetLevelVisibility(*LevelStreaming));
	}
};

bool FMovieSceneLevelStreamingSharedData::HasAnythingToDo() const
{
	return VisibilityMap.Num() != 0;
}

void FMovieSceneLevelStreamingSharedData::AssignLevelVisibilityOverrides(FInstanceHandle Instance, TArrayView<const FName> LevelNames, ELevelVisibility Visibility, int32 Bias, FMovieSceneEntityID EntityID)
{
	for (FName Name : LevelNames)
	{
		VisibilityMap.FindOrAdd(Name).Add(EntityID, Instance, Bias, Visibility);
	}
}

void FMovieSceneLevelStreamingSharedData::UnassignLevelVisibilityOverrides(TArrayView<const FName> LevelNames, FMovieSceneEntityID EntityID)
{
	for (FName Name : LevelNames)
	{
		FVisibilityData* Data = VisibilityMap.Find(Name);
		if (Data)
		{
			Data->Remove(EntityID);
		}
	}
}

void FMovieSceneLevelStreamingSharedData::Flush(UMovieSceneEntitySystemLinker* Linker)
{
	UWorld* World = Linker->GetWorld();
	if (!World)
	{
		return;
	}

	TArray<TSharedRef<FSharedPlaybackState>> StatesScratch;

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
	FPreAnimatedStateExtension& PreAnimatedState = Linker->PreAnimatedState;

	FLevelStreamingPreAnimatedTokenProducer TokenProducer;

	EFlushLevelStreamingType FlushStreamingType = EFlushLevelStreamingType::None;
	for (TPair<FName, FVisibilityData>& Pair : VisibilityMap)
	{
		if (Pair.Value.IsEmpty())
		{
			LevelsToRestore.Add(Pair.Key);
			continue;
		}

		FName SafeLevelName(*MakeSafeLevelName(Pair.Key, *World));

		ULevelStreaming* Level = GetLevel(SafeLevelName, *World);
		if (!Level)
		{
			continue;
		}

		TOptional<ELevelVisibility> DesiredVisibility = Pair.Value.CalculateVisibility();
		if (!DesiredVisibility.IsSet())
		{
			// This codepath means that we have an equal number of requests for both visible and hidden
			// In such a scenario we revert the level back to its original visibility

			if (Pair.Value.bPreviousState.IsSet())
			{
				const bool bShouldBeVisible = Pair.Value.bPreviousState.GetValue();
				if (GetLevelVisibility(*Level) != bShouldBeVisible)
				{
					SetLevelVisibility(*Level, Pair.Value.bPreviousState.GetValue(), &FlushStreamingType);
				}
			}
		}
		else
		{
			const bool bShouldBeVisible = DesiredVisibility.GetValue() == ELevelVisibility::Visible;
			if (GetLevelVisibility(*Level) != bShouldBeVisible)
			{
				if (!Pair.Value.bPreviousState.IsSet())
				{
					Pair.Value.bPreviousState = GetLevelVisibility(*Level);
				}

				StatesScratch.Reset();
				Pair.Value.GetSharedPlaybackStates(InstanceRegistry, StatesScratch);
				for (TSharedRef<FSharedPlaybackState> State : StatesScratch)
				{
					// Globally save preanimated state
					State->GetPreAnimatedState().SavePreAnimatedState(*Level, TMovieSceneAnimTypeID<FMovieSceneLevelStreamingSharedData>(), TokenProducer);
				}

				SetLevelVisibility(*Level, bShouldBeVisible, &FlushStreamingType);
			}
		}
	}

	if (FlushStreamingType != EFlushLevelStreamingType::None)
	{
		World->FlushLevelStreaming( FlushStreamingType );
	}
}

void FMovieSceneLevelStreamingSharedData::RestoreLevels(UMovieSceneEntitySystemLinker* Linker)
{
	UWorld* World = Linker->GetWorld();
	if (!World)
	{
		return;
	}

	EFlushLevelStreamingType FlushStreamingType = EFlushLevelStreamingType::None;
	for (FName LevelName : LevelsToRestore)
	{
		FName SafeLevelName(*MakeSafeLevelName(LevelName, *World));

		ULevelStreaming* Level = GetLevel(SafeLevelName, *World);
		if (Level)
		{
			if (VisibilityMap.FindChecked(LevelName).bPreviousState.IsSet())
			{
				const bool bShouldBeVisible = VisibilityMap.FindChecked(LevelName).bPreviousState.GetValue();
				if (GetLevelVisibility(*Level) != bShouldBeVisible)
				{
					SetLevelVisibility(*Level, bShouldBeVisible, &FlushStreamingType);
				}
			}
		}

		VisibilityMap.Remove(LevelName);
	}
	LevelsToRestore.Empty();

	if (FlushStreamingType != EFlushLevelStreamingType::None)
	{
		World->FlushLevelStreaming( FlushStreamingType );
	}
}

ULevelStreaming* FMovieSceneLevelStreamingSharedData::GetLevel(FName SafeLevelName, UWorld& World)
{
	if (TWeakObjectPtr<ULevelStreaming>* FoundStreamingLevel = NameToLevelMap.Find(SafeLevelName))
	{
		if (ULevelStreaming* Level = FoundStreamingLevel->Get())
		{
			return Level;
		}

		NameToLevelMap.Remove(SafeLevelName);
	}

	if (SafeLevelName == NAME_None)
	{
		return nullptr;
	}

	ULevelStreaming* Level = GetStreamingLevel(SafeLevelName.ToString(), World);
	if (Level)
	{
		NameToLevelMap.Add(SafeLevelName, Level);
	}

	return Level;
}

void FMovieSceneLevelStreamingSharedData::FVisibilityData::Add(FMovieSceneEntityID EntityID, FInstanceHandle Instance, int32 Bias, ELevelVisibility Visibility)
{
	FVisibilityRequest* ExistingRequest = Algo::FindBy(Requests, EntityID, &FVisibilityRequest::EntityID);
	if (ExistingRequest)
	{
		ExistingRequest->Bias = Bias;
		ExistingRequest->Visibility = Visibility;
	}
	else
	{
		Requests.Add(FVisibilityRequest {Instance, EntityID, Bias, Visibility });
	}
}

void FMovieSceneLevelStreamingSharedData::FVisibilityData::Remove(FMovieSceneEntityID EntityID)
{
	int32 NumRemoved = Requests.RemoveAll(
		[=](const FVisibilityRequest& In)
		{
			return In.EntityID == EntityID;
		}
	);
	check(NumRemoved <= 1);
	(void)NumRemoved;
}

/** Check whether this visibility data is empty */
bool FMovieSceneLevelStreamingSharedData::FVisibilityData::IsEmpty() const
{
	return Requests.Num() == 0;
}

/** Returns whether or not this level name should be visible or not */
TOptional<ELevelVisibility> FMovieSceneLevelStreamingSharedData::FVisibilityData::CalculateVisibility() const
{
	// Count of things asking for this level to be (in)visible. > 0 signifies visible, < 0 signifies invisible, 0 signifies previous state
	int32 VisibilityRequestCount = 0;

	int32 HighestBias = TNumericLimits<int32>::Lowest();
	for (const FVisibilityRequest& Request : Requests)
	{
		int32 IncAmount = Request.Visibility == ELevelVisibility::Visible ? 1 : -1;
		if (Request.Bias > HighestBias)
		{
			VisibilityRequestCount = IncAmount;
			HighestBias = Request.Bias;
		}
		else if (Request.Bias == HighestBias)
		{
			VisibilityRequestCount += IncAmount;
		}
	}

	if (VisibilityRequestCount == 0)
	{
		return TOptional<ELevelVisibility>();
	}
	else
	{
		return VisibilityRequestCount > 0 ? ELevelVisibility::Visible : ELevelVisibility::Hidden;
	}
}

void FMovieSceneLevelStreamingSharedData::FVisibilityData::GetSharedPlaybackStates(FInstanceRegistry* InstanceRegistry, TArray<TSharedRef<FSharedPlaybackState>>& OutSharedPlaybackStates) const
{
	for (const FVisibilityRequest& Request : Requests)
	{
		if (InstanceRegistry->IsHandleValid(Request.Instance))
		{
			TSharedRef<FSharedPlaybackState> State = InstanceRegistry->GetInstance(Request.Instance).GetSharedPlaybackState();
			OutSharedPlaybackStates.AddUnique(State);
		}
	}
}

} // namespace MovieScene
} // namespace UE

UMovieSceneLevelVisibilitySystem::UMovieSceneLevelVisibilitySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	Phase = UE::MovieScene::ESystemPhase::Spawn;
	RelevantComponent = TracksComponents->LevelVisibility;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(GetClass(), UMovieSceneSpawnablesSystem::StaticClass());
	}

	// We only need to run if there are level visibility components that need (un)linking
	ApplicableFilter.Filter.All({ TracksComponents->LevelVisibility });
	ApplicableFilter.Filter.Any({ BuiltInComponents->Tags.NeedsLink,BuiltInComponents->Tags.NeedsUnlink });
}

void UMovieSceneLevelVisibilitySystem::OnLink()
{
	UMovieSceneRestorePreAnimatedStateSystem* RestoreSystem = Linker->LinkSystem<UMovieSceneRestorePreAnimatedStateSystem>();
	Linker->SystemGraph.AddReference(this, RestoreSystem);
}

void UMovieSceneLevelVisibilitySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	// Only run if we must
	if (!ApplicableFilter.Matches(Linker->EntityManager))
	{
		return;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	auto ApplyLevelVisibilities = [this, BuiltInComponents](
		FEntityAllocationIteratorItem AllocationItem,
		TRead<FMovieSceneEntityID> EntityIDs,
		TRead<FInstanceHandle> InstanceHandles,
		TRead<FLevelVisibilityComponentData> LevelVisibilityData,
		TReadOptional<int16> OptHBiases)
	{
		const FComponentMask& AllocationType = AllocationItem.GetAllocationType();
		const bool bHasNeedsLink = AllocationType.Contains(BuiltInComponents->Tags.NeedsLink);
		const bool bHasNeedsUnlink = AllocationType.Contains(BuiltInComponents->Tags.NeedsUnlink);

		for (int32 Index = 0; Index < AllocationItem.GetAllocation()->Num(); ++Index)
		{
			const FMovieSceneEntityID EntityID = EntityIDs[Index];
			const UMovieSceneLevelVisibilitySection* Section = LevelVisibilityData[Index].Section;

			if (!ensure(Section))
			{
				continue;
			}

			const TArray<FName>& LevelNames = Section->GetLevelNames();
			const ELevelVisibility Visibility = Section->GetVisibility();

			if (bHasNeedsLink)
			{
				const int16 HBias = (OptHBiases ? OptHBiases[Index] : 0);
				SharedData.AssignLevelVisibilityOverrides(InstanceHandles[Index], LevelNames, Visibility, HBias, EntityID);
			}
			if (bHasNeedsUnlink)
			{
				SharedData.UnassignLevelVisibilityOverrides(LevelNames, EntityID);
			}
		}
	};

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(TracksComponents->LevelVisibility)
	.ReadOptional(BuiltInComponents->HierarchicalBias)
	.Iterate_PerAllocation(&Linker->EntityManager, ApplyLevelVisibilities);

	SharedData.Flush(Linker);
}

void UMovieSceneLevelVisibilitySystem::SavePreAnimatedState(const FPreAnimationParameters& InParameters)
{
}

void UMovieSceneLevelVisibilitySystem::RestorePreAnimatedState(const FPreAnimationParameters& InParameters)
{
	SharedData.RestoreLevels(Linker);
}


