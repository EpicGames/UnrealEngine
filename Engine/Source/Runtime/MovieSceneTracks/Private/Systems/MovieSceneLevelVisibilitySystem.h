// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "Sections/MovieSceneLevelVisibilitySection.h"
#include "Engine/LevelStreaming.h"
#include "Containers/ArrayView.h"

#include "MovieSceneLevelVisibilitySystem.generated.h"

class UWorld;

namespace UE
{
namespace MovieScene
{
	struct FSharedPlaybackState;

	struct FMovieSceneLevelStreamingSharedData
	{
		bool HasAnythingToDo() const;
		void AssignLevelVisibilityOverrides(FInstanceHandle Instance, TArrayView<const FName> LevelNames, ELevelVisibility Visibility, int32 Bias, FMovieSceneEntityID EntityID);
		void UnassignLevelVisibilityOverrides(TArrayView<const FName> LevelNames, FMovieSceneEntityID EntityID);
		void Flush(UMovieSceneEntitySystemLinker* Linker);
		void RestoreLevels(UMovieSceneEntitySystemLinker* Linker);

	private:
		ULevelStreaming* GetLevel(FName SafeLevelName, UWorld& World);

	private:
		struct FVisibilityData
		{
			TOptional<bool> bPreviousState;

			void Add(FMovieSceneEntityID EntityID, FInstanceHandle Instance, int32 Bias, ELevelVisibility Visibility);
			void Remove(FMovieSceneEntityID EntityID);

			/** Check whether this visibility data is empty */
			bool IsEmpty() const;

			/** Returns whether or not this level name should be visible or not */
			TOptional<ELevelVisibility> CalculateVisibility() const;

			/** Retrieve all shared playback states that are animating this level's visibility */
			void GetSharedPlaybackStates(FInstanceRegistry* InstanceRegistry, TArray<TSharedRef<FSharedPlaybackState>>& OutSharedPlaybackStates) const;

		private:
			struct FVisibilityRequest
			{
				/** The instance that contains the entity that made the request */
				FInstanceHandle Instance;
				/** The entity that made the request */
				FMovieSceneEntityID EntityID;
				/** The bias of the entity */
				int32 Bias;
				/** The actual visibility requested */
				ELevelVisibility Visibility;
			};
			TArray<FVisibilityRequest, TInlineAllocator<2>> Requests;
		};
		TMap<FName, FVisibilityData> VisibilityMap;

		TMap<FName, TWeakObjectPtr<ULevelStreaming>> NameToLevelMap;

		TArray<FName, TInlineAllocator<8>> LevelsToRestore;
	};
}
}

UCLASS(MinimalAPI)
class UMovieSceneLevelVisibilitySystem
	: public UMovieSceneEntitySystem
	, public IMovieScenePreAnimatedStateSystemInterface
{
public:

	GENERATED_BODY()

	UMovieSceneLevelVisibilitySystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnLink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

	virtual void SavePreAnimatedState(const FPreAnimationParameters& InParameters) override;
	virtual void RestorePreAnimatedState(const FPreAnimationParameters& InParameters) override;

private:

	/** Cached filter that tells us whether we need to run this frame */
	UE::MovieScene::FCachedEntityFilterResult_Match ApplicableFilter;
	UE::MovieScene::FMovieSceneLevelStreamingSharedData SharedData;
};
