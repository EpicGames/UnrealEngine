// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneMaterialParameterCollectionSystem.h"

#include "Engine/Engine.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneSequence.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityMutations.h"

#include "MovieSceneTracksComponentTypes.h"
#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieScenePreAnimatedMaterialParameters.h"
#include "Systems/WeightAndEasingEvaluatorSystem.h"

#include "Logging/MessageLog.h"
#include "Misc/UObjectToken.h"

#include "Materials/MaterialParameterCollection.h"
#include "Materials/MaterialParameterCollectionInstance.h"

#include "Components/PrimitiveComponent.h"
#include "Components/DecalComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMaterialParameterCollectionSystem)

namespace UE::MovieScene
{

struct FMPCGroupingPolicy
{
	using GroupKeyType = TTuple<FObjectKey, FMaterialParameterInfo>;

	void InitializeGroupKeys(
		TEntityGroupingHandlerBase<FMPCGroupingPolicy>& Handler,
		FEntityGroupBuilder* Builder,
		FEntityAllocationIteratorItem Item,
		FReadEntityIDs EntityIDs,
		TWrite<FEntityGroupID> GroupIDs,
		TRead<TWeakObjectPtr<UMaterialParameterCollection>> MPCs)
	{
		const FEntityAllocation* Allocation     = Item.GetAllocation();
		const FComponentMask&    AllocationType = Item.GetAllocationType();
		const int32              Num            = Allocation->Num();
		
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		TComponentTypeID<FName> ParameterName;
		if (AllocationType.Contains(TracksComponents->ScalarParameterName))
		{
			ParameterName = TracksComponents->ScalarParameterName;
		}
		else if (AllocationType.Contains(TracksComponents->VectorParameterName))
		{
			ParameterName = TracksComponents->VectorParameterName;
		}
		else if (AllocationType.Contains(TracksComponents->ColorParameterName))
		{
			ParameterName = TracksComponents->ColorParameterName;
		}

		TComponentTypeID<FMaterialParameterInfo> ParameterInfo;
		if (AllocationType.Contains(TracksComponents->ScalarMaterialParameterInfo))
		{
			ParameterInfo = TracksComponents->ScalarMaterialParameterInfo;
		}
		else if (AllocationType.Contains(TracksComponents->VectorMaterialParameterInfo))
		{
			ParameterInfo = TracksComponents->VectorMaterialParameterInfo;
		}
		else if (AllocationType.Contains(TracksComponents->ColorMaterialParameterInfo))
		{
			ParameterInfo = TracksComponents->ColorMaterialParameterInfo;
		}

		TReadOptional<FName>                  ParameterNames = ParameterName ? Allocation->TryReadComponents(ParameterName) : TReadOptional<FName>();
		TReadOptional<FMaterialParameterInfo> ParameterInfos = ParameterInfo ? Allocation->TryReadComponents(ParameterInfo) : TReadOptional<FMaterialParameterInfo>();

		if (!ensure(ParameterInfos || ParameterNames))
		{
			return;
		}

		for (int32 Index = 0; Index < Num; ++Index)
		{
			if (!GroupIDs[Index].IsValid())
			{
				FMaterialParameterInfo ThisParameterInfo = ParameterInfos ? ParameterInfos[Index] : FMaterialParameterInfo(ParameterNames[Index]);
				GroupKeyType Key = MakeTuple(FObjectKey(MPCs[Index].Get()), ThisParameterInfo);

				const int32    NewGroupIndex = Handler.GetOrAllocateGroupIndex(Key, Builder);
				FEntityGroupID NewGroupID = Builder->MakeGroupID(NewGroupIndex);

				Builder->AddEntityToGroup(EntityIDs[Index], NewGroupID);

				// Write out the group ID component
				GroupIDs[Index] = NewGroupID;
			}
		}
	}

#if WITH_EDITOR
	bool OnObjectsReplaced(GroupKeyType& InOutKey, const TMap<UObject*, UObject*>& ReplacementMap)
	{
		if (UObject* const * NewObject = ReplacementMap.Find(InOutKey.Get<0>().ResolveObjectPtr()))
		{
			InOutKey.Get<0>() = *NewObject;
			return true;
		}
		return false;
	}
#endif
};

} // namespace UE::MovieScene


UMovieSceneMaterialParameterCollectionSystem::UMovieSceneMaterialParameterCollectionSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	RelevantComponent = TracksComponents->MPC;
	Phase = ESystemPhase::Instantiation;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), BuiltInComponents->Tags.Root);
		DefineComponentProducer(GetClass(), TracksComponents->BoundMaterial);
		DefineImplicitPrerequisite(UMovieSceneCachePreAnimatedStateSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneHierarchicalEasingInstantiatorSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneMaterialParameterCollectionSystem::OnLink()
{
	using namespace UE::MovieScene;

	ScalarParameterStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedScalarMaterialParameterStorage>();
	VectorParameterStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedVectorMaterialParameterStorage>();

	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->LinkSystem<UMovieSceneEntityGroupingSystem>();

	FEntityComponentFilter Filter;
	Filter.None({ FBuiltInComponentTypes::Get()->Tags.ImportedEntity });

	GroupingKey = GroupingSystem->AddGrouping(
		FMPCGroupingPolicy(),
		MoveTemp(Filter),
		FMovieSceneTracksComponentTypes::Get()->MPC
	);
}

void UMovieSceneMaterialParameterCollectionSystem::OnUnlink()
{
	using namespace UE::MovieScene;

	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->FindSystem<UMovieSceneEntityGroupingSystem>();
	if (ensure(GroupingSystem))
	{
		GroupingSystem->RemoveGrouping(GroupingKey);
	}
	GroupingKey = FEntityGroupingPolicyKey();
}

void UMovieSceneMaterialParameterCollectionSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	struct FAddMPCMutation : IMovieSceneEntityMutation
	{
		FAddMPCMutation(UMovieSceneEntitySystemLinker* InLinker)
			: InstanceRegistry(InLinker->GetInstanceRegistry())
		{
			BuiltInComponents = FBuiltInComponentTypes::Get();
			TracksComponents  = FMovieSceneTracksComponentTypes::Get();
		}
		virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
		{
			InOutEntityComponentTypes->Set(TracksComponents->BoundMaterial);
		}
		virtual void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const
		{
			TComponentWriter<FObjectComponent> OutBoundMaterials = Allocation->WriteComponents(TracksComponents->BoundMaterial, FEntityAllocationWriteContext::NewAllocation());
			TComponentReader<TWeakObjectPtr<UMaterialParameterCollection>> MPCs = Allocation->ReadComponents(TracksComponents->MPC);
			TComponentReader<FInstanceHandle> InstanceHandles = Allocation->ReadComponents(BuiltInComponents->InstanceHandle);

			TOptionalComponentReader<FName> ScalarParameterNames = Allocation->TryReadComponents(TracksComponents->ScalarParameterName);
			TOptionalComponentReader<FName> VectorParameterNames = Allocation->TryReadComponents(TracksComponents->VectorParameterName);
			TOptionalComponentReader<FName> ColorParameterNames = Allocation->TryReadComponents(TracksComponents->ColorParameterName);

			const int32 Num = Allocation->Num();
			for (int32 Index = 0; Index < Num; ++Index)
			{
				OutBoundMaterials[Index] = FObjectComponent::Null();

				UMaterialParameterCollection* Collection = MPCs[Index].Get();
				TSharedRef<const FSharedPlaybackState> SharedPlaybackState = InstanceRegistry->GetInstance(InstanceHandles[Index]).GetSharedPlaybackState();
				UObject* WorldContextObject = SharedPlaybackState->GetPlaybackContext();
				UWorld* World = GEngine->GetWorldFromContextObject(WorldContextObject, EGetWorldErrorMode::LogAndReturnNull);
				if (World && Collection)
				{
					UMaterialParameterCollectionInstance* Instance = World->GetParameterCollectionInstance(Collection);
					OutBoundMaterials[Index] = FObjectComponent::Weak(Instance);

#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST
					if (ensureAlwaysMsgf(Instance != nullptr,
						TEXT("Unable to create MPC instance for %s with World %s. Material parameter collection tracks will not function."),
						*Collection->GetName(), *World->GetName()))
					{
						if (ScalarParameterNames)
						{
							FName Name = ScalarParameterNames[Index];
							if (Collection->GetScalarParameterByName(Name) == nullptr)
							{
								if (!Instance->bLoggedMissingParameterWarning)
								{
									MissingParameters.FindOrAdd(MakeTuple(Instance, SharedPlaybackState)).Add(Name.ToString());
								}
							}
						}
						else if (VectorParameterNames || ColorParameterNames)
						{
							FName Name = VectorParameterNames ? VectorParameterNames[Index] : ColorParameterNames[Index];
							if (Collection->GetVectorParameterByName(Name) == nullptr)
							{
								if (!Instance->bLoggedMissingParameterWarning)
								{
									MissingParameters.FindOrAdd(MakeTuple(Instance, SharedPlaybackState)).Add(Name.ToString());
								}
							}
						}
					}
#endif
				}
			}
		}

		void Cleanup(UMovieSceneEntitySystemLinker* InLinker)
		{
#if !UE_BUILD_SHIPPING && !UE_BUILD_TEST

			for (TPair<FMissingParameterKey, TArray<FString>>& Pair : MissingParameters)
			{
				FFormatNamedArguments Arguments;
				Arguments.Add(TEXT("ParamNames"), FText::FromString(FString::Join(Pair.Value, TEXT(", "))));
				FMessageLog("PIE").Warning()
					->AddToken(FTextToken::Create(NSLOCTEXT("MaterialParameterCollectionTrack", "InvalidParameterText", "Invalid parameter name or type applied in sequence")))
					->AddToken(FUObjectToken::Create(Pair.Key.Get<1>()->GetSequence(MovieSceneSequenceID::Root)))
					->AddToken(FTextToken::Create(NSLOCTEXT("MaterialParameterCollectionTrack", "OnText", "on")))
					->AddToken(FUObjectToken::Create(Pair.Key.Get<0>()))
					->AddToken(FTextToken::Create(FText::Format(NSLOCTEXT("MaterialParameterCollectionTrack", "InvalidParameterFormatText", "with the following invalid parameters: {ParamNames}."), Arguments)));
				Pair.Key.Get<0>()->bLoggedMissingParameterWarning = true;
			}

#endif
		}
	private:

		FInstanceRegistry* InstanceRegistry;
		FBuiltInComponentTypes* BuiltInComponents;
		FMovieSceneTracksComponentTypes* TracksComponents;

		using FMissingParameterKey = TTuple<UMaterialParameterCollectionInstance*, TSharedRef<const FSharedPlaybackState>>;
		using FMissingParametersMap = TMap<FMissingParameterKey, TArray<FString>>;
		mutable FMissingParametersMap MissingParameters;
	};


	// Only mutate things that are tagged as requiring linking
	FEntityComponentFilter Filter;
	Filter.All({ TracksComponents->MPC, BuiltInComponents->InstanceHandle, BuiltInComponents->Tags.NeedsLink });
	Filter.None({ BuiltInComponents->Tags.ImportedEntity });

	// Initialize bound dynamic materials (for material collection parameters)
	FAddMPCMutation BindMaterialsMutation(Linker);
	Linker->EntityManager.MutateAll(Filter, BindMaterialsMutation);
	BindMaterialsMutation.Cleanup(Linker);

	TPreAnimatedStateTaskParams<FObjectComponent, FName> Params;

	Params.AdditionalFilter.None({ BuiltInComponents->BlendChannelOutput });
	Params.AdditionalFilter.All({ TracksComponents->MPC });
	ScalarParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, TracksComponents->BoundMaterial, TracksComponents->ScalarParameterName);
	VectorParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, TracksComponents->BoundMaterial, TracksComponents->VectorParameterName);
	VectorParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, TracksComponents->BoundMaterial, TracksComponents->ColorParameterName);
}

