// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneMaterialParameterSystem.h"

#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieSceneHierarchicalBiasSystem.h"
#include "Systems/MovieSceneMaterialSystem.h"
#include "Systems/MovieSceneInitialValueSystem.h"
#include "Systems/MovieScenePreAnimatedMaterialParameters.h"
#include "Systems/WeightAndEasingEvaluatorSystem.h"

#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"

#include "Materials/MaterialParameterCollectionInstance.h"
#include "Materials/MaterialInstanceDynamic.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMaterialParameterSystem)

namespace UE::MovieScene
{

bool GMaterialParameterBlending = true;
FAutoConsoleVariableRef CVarMaterialParameterBlending(
	TEXT("Sequencer.MaterialParameterBlending"),
	GMaterialParameterBlending,
	TEXT("(Default: true) Defines whether material parameter blending should be enabled or not.\n"),
	ECVF_Default
);

bool GMaterialParameterEntityLifetimeTracking = false;
FAutoConsoleVariableRef CVarMaterialParameterEntityLifetimeTracking(
	TEXT("Sequencer.MaterialParameterEntityLifetimeTracking"),
	GMaterialParameterEntityLifetimeTracking,
	TEXT("(Default: false) Ensure on destruction that all entities have been cleaned up. This can report false positives (when the linker and material system are both cleaned up together) so is not enabled by default.\n"),
	ECVF_Default
);


void CollectGarbageForOutput(FAnimatedMaterialParameterInfo* Output)
{
	// This should only happen during garbage collection
	if (Output->OutputEntityID.IsValid())
	{
		UMovieSceneEntitySystemLinker* Linker = Output->WeakLinker.Get();
		if (Linker)
		{
			Linker->EntityManager.AddComponent(Output->OutputEntityID, FBuiltInComponentTypes::Get()->Tags.NeedsUnlink);
		}

		Output->OutputEntityID = FMovieSceneEntityID();
	}

	if (Output->BlendChannelID.IsValid())
	{
		UMovieSceneBlenderSystem* BlenderSystem = Output->WeakBlenderSystem.Get();
		if (BlenderSystem)
		{
			BlenderSystem->ReleaseBlendChannel(Output->BlendChannelID);
		}
		Output->BlendChannelID = FMovieSceneBlendChannelID();
	}
}

FAnimatedMaterialParameterInfo::~FAnimatedMaterialParameterInfo()
{
	if (GMaterialParameterEntityLifetimeTracking)
	{
		ensureAlways(!OutputEntityID.IsValid() && !BlendChannelID.IsValid());
	}
}

/** Apply scalar material parameters */
struct FApplyScalarParameters
{
	static void ForEachAllocation(FEntityAllocationIteratorItem Item,
		TRead<FObjectComponent> BoundMaterials,
		TReadOneOrMoreOf<FMaterialParameterInfo, FName> ScalarParameterInfosOrNames,
		TRead<double> ScalarValues)
	{
		const int32 Num = Item.GetAllocation()->Num();
		const FMaterialParameterInfo* ParameterInfos = ScalarParameterInfosOrNames.Get<0>();
		const FName* ParameterNames = ScalarParameterInfosOrNames.Get<1>();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			const FObjectComponent& BoundMaterial = BoundMaterials[Index];
			if (!BoundMaterial)
			{
				continue;
			}

			// WARNING: BoundMaterial may be nullptr here
			if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(BoundMaterial.GetObject()))
			{
				if (ParameterInfos)
				{
					MID->SetScalarParameterValueByInfo(ParameterInfos[Index], (float)ScalarValues[Index]);
				}
				else
				{
					MID->SetScalarParameterValue(ParameterNames[Index], (float)ScalarValues[Index]);
				}
			}
			else if (UMaterialParameterCollectionInstance* MPCI = Cast<UMaterialParameterCollectionInstance>(BoundMaterial.GetObject()))
			{
				if (ParameterInfos)
				{
					MPCI->SetScalarParameterValue(ParameterInfos[Index].Name, (float)ScalarValues[Index]);
				}
				else
				{
					MPCI->SetScalarParameterValue(ParameterNames[Index], (float)ScalarValues[Index]);
				}
			}


		}
	}
};

/** Apply vector material parameters */
struct FApplyVectorParameters
{
	static void ForEachAllocation(FEntityAllocationIteratorItem Item,
		TRead<FObjectComponent> BoundMaterials,
		TReadOneOrMoreOf<FMaterialParameterInfo, FMaterialParameterInfo, FName, FName> VectorOrColorParameterInfosOrNames,
		TReadOneOrMoreOf<double, double, double, double> VectorChannels)
	{
		const int32 Num = Item.GetAllocation()->Num();
		// Use either the vector parameter name, or the color parameter name
		const FMaterialParameterInfo* ParameterInfos = VectorOrColorParameterInfosOrNames.Get<0>() ? VectorOrColorParameterInfosOrNames.Get<0>() : VectorOrColorParameterInfosOrNames.Get<1>();
		const FName* ParameterNames = VectorOrColorParameterInfosOrNames.Get<2>() ? VectorOrColorParameterInfosOrNames.Get<2>() : VectorOrColorParameterInfosOrNames.Get<3>();
		const double* RESTRICT R = VectorChannels.Get<0>();
		const double* RESTRICT G = VectorChannels.Get<1>();
		const double* RESTRICT B = VectorChannels.Get<2>();
		const double* RESTRICT A = VectorChannels.Get<3>();
		
		for (int32 Index = 0; Index < Num; ++Index)
		{
			const FObjectComponent& BoundMaterial = BoundMaterials[Index];
			if (!BoundMaterial)
			{
				continue;
			}

			// Default to white for unanimated channels for backwards compatibility with the legacy track
			FLinearColor Color(
				R ? (float)R[Index] : 1.f,
				G ? (float)G[Index] : 1.f,
				B ? (float)B[Index] : 1.f,
				A ? (float)A[Index] : 1.f
			);
			if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(BoundMaterial.GetObject()))
			{
				if (ParameterInfos)
				{
					MID->SetVectorParameterValueByInfo(ParameterInfos[Index], Color);
				}
				else if (ParameterNames)
				{
					MID->SetVectorParameterValue(ParameterNames[Index], Color);
				}
			}
			else if (UMaterialParameterCollectionInstance* MPCI = Cast<UMaterialParameterCollectionInstance>(BoundMaterial.GetObject()))
			{
				if (ParameterInfos)
				{
					MPCI->SetVectorParameterValue(ParameterInfos[Index].Name, Color);
				}
				else if (ParameterNames)
				{
					MPCI->SetVectorParameterValue(ParameterNames[Index], Color);
				}
			}
		}
	}
};

struct FScalarMixin
{
	void CreateEntity(UMovieSceneEntitySystemLinker* Linker, const FObjectComponent& BoundMaterial, const FMaterialParameterInfo& ParameterInfo, FComponentTypeID BlenderTypeTag, TArrayView<const FMovieSceneEntityID> Inputs, FAnimatedMaterialParameterInfo* Output)
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		bool bHasInitialValue = false;
		float InitialValue = 0.f;

		for (FMovieSceneEntityID Input : Inputs)
		{
			if (TOptionalComponentReader<double> ExistingInitialValue = Linker->EntityManager.ReadComponent(Input, TracksComponents->FloatParameter.InitialValue))
			{
				InitialValue = static_cast<float>(*ExistingInitialValue);
				bHasInitialValue = true;
				break;
			}
		}

		if (!bHasInitialValue)
		{
			if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(BoundMaterial.GetObject()))
			{
				bHasInitialValue = MID->GetScalarParameterValue(ParameterInfo, InitialValue);
			}
			else if (UMaterialParameterCollectionInstance* MPCI = Cast<UMaterialParameterCollectionInstance>(BoundMaterial.GetObject()))
			{
				bHasInitialValue = MPCI->GetScalarParameterValue(ParameterInfo.Name, InitialValue);
			}
		}

		Output->OutputEntityID = FEntityBuilder()
		.Add(TracksComponents->BoundMaterial, BoundMaterial)
		.Add(BuiltInComponents->BlendChannelOutput, Output->BlendChannelID)
		.Add(BuiltInComponents->DoubleResult[0], 0.0)
		.AddConditional(TracksComponents->FloatParameter.InitialValue, InitialValue, bHasInitialValue)
		.AddTag(TracksComponents->FloatParameter.PropertyTag)
		.AddTag(BlenderTypeTag)
		.AddTag(BuiltInComponents->Tags.NeedsLink)
		.AddMutualComponents()
		.CreateEntity(&Linker->EntityManager);

		Linker->EntityManager.CopyComponents(Inputs[0], Output->OutputEntityID, Linker->EntityManager.GetComponents()->GetCopyAndMigrationMask());
	}

	void InitializeSoleInput(UMovieSceneEntitySystemLinker* Linker, const FObjectComponent& BoundMaterial, const FMaterialParameterInfo& ParameterInfo, FMovieSceneEntityID SoleContributor, FAnimatedMaterialParameterInfo* Output)
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		const FComponentMask& SoleContributorType = Linker->EntityManager.GetEntityType(SoleContributor);
		const bool bNeedsInitialValue = SoleContributorType.ContainsAny({ BuiltInComponents->Tags.RestoreState, BuiltInComponents->Tags.AlwaysCacheInitialValue });
		const bool bAlreadyHasInitialValue = SoleContributorType.Contains(TracksComponents->FloatParameter.InitialValue);

		if (bNeedsInitialValue && !bAlreadyHasInitialValue)
		{
			float InitialValue = 0.0;
			if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(BoundMaterial.GetObject()))
			{
				MID->GetScalarParameterValue(ParameterInfo, InitialValue);
				Linker->EntityManager.AddComponent(SoleContributor, TracksComponents->FloatParameter.InitialValue, InitialValue);
			}
			else if (UMaterialParameterCollectionInstance* MPCI = Cast<UMaterialParameterCollectionInstance>(BoundMaterial.GetObject()))
			{
				MPCI->GetScalarParameterValue(ParameterInfo.Name, InitialValue);
				Linker->EntityManager.AddComponent(SoleContributor, TracksComponents->FloatParameter.InitialValue, InitialValue);
			}
		}

		Linker->EntityManager.RemoveComponent(SoleContributor, FBuiltInComponentTypes::Get()->HierarchicalBlendTarget);
	}
};

struct FVectorMixin
{
	void CreateEntity(UMovieSceneEntitySystemLinker* Linker, const FObjectComponent& BoundMaterial, const FMaterialParameterInfo& ParameterInfo, FComponentTypeID BlenderTypeTag, TArrayView<const FMovieSceneEntityID> Inputs, FAnimatedMaterialParameterInfo* Output)
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		bool bHasInitialValue = false;
		FIntermediateColor InitialValue;

		for (FMovieSceneEntityID Input : Inputs)
		{
			if (TOptionalComponentReader<FIntermediateColor> ExistingInitialValue = Linker->EntityManager.ReadComponent(Input, TracksComponents->ColorParameter.InitialValue))
			{
				InitialValue = *ExistingInitialValue;
				bHasInitialValue = true;
				break;
			}
		}

		if (!bHasInitialValue)
		{
			FLinearColor ColorValue = FLinearColor::Black;
			if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(BoundMaterial.GetObject()))
			{
				bHasInitialValue = MID->GetVectorParameterValue(ParameterInfo, ColorValue);
			}
			else if (UMaterialParameterCollectionInstance* MPCI = Cast<UMaterialParameterCollectionInstance>(BoundMaterial.GetObject()))
			{
				bHasInitialValue = MPCI->GetVectorParameterValue(ParameterInfo.Name, ColorValue);
			}

			if (bHasInitialValue)
			{
				InitialValue = FIntermediateColor(ColorValue.R, ColorValue.G, ColorValue.B, ColorValue.A);
			}
		}

		Output->OutputEntityID = FEntityBuilder()
		.Add(TracksComponents->BoundMaterial, BoundMaterial)
		.Add(BuiltInComponents->BlendChannelOutput, Output->BlendChannelID)
		.Add(BuiltInComponents->DoubleResult[0], 0.0)
		.Add(BuiltInComponents->DoubleResult[1], 0.0)
		.Add(BuiltInComponents->DoubleResult[2], 0.0)
		.Add(BuiltInComponents->DoubleResult[3], 0.0)
		.AddConditional(TracksComponents->ColorParameter.InitialValue, InitialValue, bHasInitialValue)
		.AddTag(TracksComponents->ColorParameter.PropertyTag)
		.AddTag(BlenderTypeTag)
		.AddTag(BuiltInComponents->Tags.NeedsLink)
		.AddMutualComponents()
		.CreateEntity(&Linker->EntityManager);

		Linker->EntityManager.CopyComponents(Inputs[0], Output->OutputEntityID, Linker->EntityManager.GetComponents()->GetCopyAndMigrationMask());
	}

	void InitializeSoleInput(UMovieSceneEntitySystemLinker* Linker, const FObjectComponent& BoundMaterial, const FMaterialParameterInfo& ParameterInfo, FMovieSceneEntityID SoleContributor, FAnimatedMaterialParameterInfo* Output)
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		const FComponentMask& SoleContributorType = Linker->EntityManager.GetEntityType(SoleContributor);
		const bool bNeedsInitialValue = SoleContributorType.ContainsAny({ BuiltInComponents->Tags.RestoreState, BuiltInComponents->Tags.AlwaysCacheInitialValue });
		const bool bAlreadyHasInitialValue = SoleContributorType.Contains(TracksComponents->ColorParameter.InitialValue);

		if (bNeedsInitialValue && !bAlreadyHasInitialValue)
		{
			FLinearColor ColorValue = FLinearColor::White;
			if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(BoundMaterial.GetObject()))
			{
				if (MID->GetVectorParameterValue(ParameterInfo, ColorValue))
				{
					Linker->EntityManager.AddComponent(SoleContributor, TracksComponents->ColorParameter.InitialValue, FIntermediateColor(ColorValue));
				}
			}
			else if (UMaterialParameterCollectionInstance* MPCI = Cast<UMaterialParameterCollectionInstance>(BoundMaterial.GetObject()))
			{
				if (MPCI->GetVectorParameterValue(ParameterInfo.Name, ColorValue))
				{
					Linker->EntityManager.AddComponent(SoleContributor, TracksComponents->ColorParameter.InitialValue, FIntermediateColor(ColorValue));
				}
			}
		}
	}
};

/** Handler that manages creation of blend outputs where there are multiple contributors for the same material parameter */
template<typename Mixin>
struct TOverlappingMaterialParameterHandler : Mixin
{
	UMovieSceneEntitySystemLinker* Linker;
	UMovieSceneMaterialParameterInstantiatorSystem* System;

	TOverlappingMaterialParameterHandler(UMovieSceneMaterialParameterInstantiatorSystem* InSystem)
		: Linker(InSystem->GetLinker())
		, System(InSystem)
	{}

	void InitializeOutput(const FObjectComponent& BoundMaterial, const FMaterialParameterInfo& ParameterInfo, TArrayView<const FMovieSceneEntityID> Inputs, FAnimatedMaterialParameterInfo* Output, FEntityOutputAggregate Aggregate)
	{
		UpdateOutput(BoundMaterial, ParameterInfo, Inputs, Output, Aggregate);
	}

	void UpdateOutput(const FObjectComponent& BoundMaterial, const FMaterialParameterInfo& ParameterInfo, TArrayView<const FMovieSceneEntityID> Inputs, FAnimatedMaterialParameterInfo* Output, FEntityOutputAggregate Aggregate)
	{
		using namespace UE::MovieScene;

		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		const int32 NumContributors = Inputs.Num();
		if (!ensure(NumContributors != 0))
		{
			return;
		}

		if (!BoundMaterial)
		{
			// Handle null bound materials - this is a rare case but must be handled explicitly because it is possible
			//  a GC has already destroyed our OutputEntityID because it was considered garbage. In this instance we just
			//  remove the invalid blend channel input components off all inputs.
			DestroyOutput(BoundMaterial, ParameterInfo, Output, Aggregate);

			for (FMovieSceneEntityID Input : Inputs)
			{
				Linker->EntityManager.RemoveComponent(Input, BuiltInComponents->BlendChannelInput);
			}
			return;
		}

		const bool bUseBlending = NumContributors > 1 || !Linker->EntityManager.HasComponent(Inputs[0], BuiltInComponents->Tags.AbsoluteBlend) || Linker->EntityManager.HasComponent(Inputs[0], BuiltInComponents->WeightAndEasingResult);
		if (bUseBlending)
		{
			if (!Output->OutputEntityID)
			{
				if (!System->DoubleBlenderSystem)
				{
					System->DoubleBlenderSystem = Linker->LinkSystem<UMovieScenePiecewiseDoubleBlenderSystem>();
					Linker->SystemGraph.AddReference(System, System->DoubleBlenderSystem);
				}

				Output->WeakLinker = Linker;
				Output->WeakBlenderSystem = System->DoubleBlenderSystem;

				// Initialize the blend channel ID
				Output->BlendChannelID = System->DoubleBlenderSystem->AllocateBlendChannel();

				// Needs blending
				FComponentTypeID BlenderTypeTag = System->DoubleBlenderSystem->GetBlenderTypeTag();
				Mixin::CreateEntity(Linker, BoundMaterial, ParameterInfo, BlenderTypeTag, Inputs, Output);
			}

			const FComponentTypeID BlenderTypeTag = System->DoubleBlenderSystem->GetBlenderTypeTag();








			for (FMovieSceneEntityID Input : Inputs)
			{
				if (!Linker->EntityManager.HasComponent(Input, BuiltInComponents->BlendChannelInput))
				{
					Linker->EntityManager.AddComponent(Input, BuiltInComponents->BlendChannelInput, Output->BlendChannelID);
				}
				else
				{
					// If the bound material changed, we might have been re-assigned a different blend channel so make sure it's up to date
					Linker->EntityManager.WriteComponentChecked(Input, BuiltInComponents->BlendChannelInput, Output->BlendChannelID);
				}


				// Ensure we have the blender type tag on the inputs.
				Linker->EntityManager.AddComponent(Input, BlenderTypeTag);
			}
		}
		else if (!Output->OutputEntityID && Inputs.Num() == 1)
		{
			Linker->EntityManager.RemoveComponent(Inputs[0], BuiltInComponents->BlendChannelInput);

			Mixin::InitializeSoleInput(Linker, BoundMaterial, ParameterInfo, Inputs[0], Output);
		}

		Output->NumContributors = NumContributors;
	}

	void DestroyOutput(const FObjectComponent& BoundMaterial, const FMaterialParameterInfo& ParameterInfo, FAnimatedMaterialParameterInfo* Output, FEntityOutputAggregate Aggregate)
	{
		if (Output->OutputEntityID)
		{
			FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

			Linker->EntityManager.AddComponent(Output->OutputEntityID, BuiltInComponents->Tags.NeedsUnlink);
			Output->OutputEntityID = FMovieSceneEntityID();

			if (System->DoubleBlenderSystem)
			{
				System->DoubleBlenderSystem->ReleaseBlendChannel(Output->BlendChannelID);
			}
			Output->BlendChannelID = FMovieSceneBlendChannelID();
		}
	}

};

} // namespace UE::MovieScene

UMovieSceneMaterialParameterInstantiatorSystem::UMovieSceneMaterialParameterInstantiatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	RelevantComponent = TracksComponents->BoundMaterial;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), TracksComponents->BoundMaterial);

		DefineComponentConsumer(GetClass(), BuiltInComponents->HierarchicalBlendTarget);

		DefineImplicitPrerequisite(UMovieSceneHierarchicalEasingInstantiatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneInitialValueSystem::StaticClass());
	}
}

void UMovieSceneMaterialParameterInstantiatorSystem::OnLink()
{
	using namespace UE::MovieScene;

	ScalarParameterTracker.Initialize(this);
	VectorParameterTracker.Initialize(this);

	ScalarParameterStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedScalarMaterialParameterStorage>();
	VectorParameterStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedVectorMaterialParameterStorage>();
}

void UMovieSceneMaterialParameterInstantiatorSystem::OnUnlink()
{
	using namespace UE::MovieScene;

	// Always reset the float blender system on link to ensure that recycled systems are correctly initialized.
	DoubleBlenderSystem = nullptr;

	ScalarParameterTracker.Destroy(TOverlappingMaterialParameterHandler<FScalarMixin>(this));
	VectorParameterTracker.Destroy(TOverlappingMaterialParameterHandler<FVectorMixin>(this));
}

void UMovieSceneMaterialParameterInstantiatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	if (GMaterialParameterBlending)
	{
		auto HandleUnlinkedAllocation = [this](const FEntityAllocation* Allocation)
		{
			this->ScalarParameterTracker.VisitUnlinkedAllocation(Allocation);
			this->VectorParameterTracker.VisitUnlinkedAllocation(Allocation);
		};

		auto HandleUpdatedAllocation = [this](const FEntityAllocation* Allocation, TRead<FObjectComponent> InObjects, TReadOneOf<FMaterialParameterInfo, FMaterialParameterInfo, FMaterialParameterInfo, FName, FName, FName> ScalarVectorOrColorParameterInfosOrNames)
		{
			if (TComponentPtr<const FMaterialParameterInfo> ScalarParameterInfos = TComponentPtr<const FMaterialParameterInfo>(ScalarVectorOrColorParameterInfosOrNames.Get<0>()))
			{
				this->ScalarParameterTracker.VisitActiveAllocation(Allocation, InObjects, ScalarParameterInfos);
			}
			else if (TComponentPtr<const FMaterialParameterInfo> VectorParameterInfos = TComponentPtr<const FMaterialParameterInfo>(ScalarVectorOrColorParameterInfosOrNames.Get<1>()))
			{
				this->VectorParameterTracker.VisitActiveAllocation(Allocation, InObjects, VectorParameterInfos);
			}
			else if (TComponentPtr<const FMaterialParameterInfo> ColorParameterInfos = TComponentPtr<const FMaterialParameterInfo>(ScalarVectorOrColorParameterInfosOrNames.Get<2>()))
			{
				this->VectorParameterTracker.VisitActiveAllocation(Allocation, InObjects, ColorParameterInfos);
			}
			else if (TComponentPtr<const FName> ScalarParameterNames = TComponentPtr<const FName>(ScalarVectorOrColorParameterInfosOrNames.Get<3>()))
			{
				this->ScalarParameterTracker.VisitActiveAllocation(Allocation, InObjects, ScalarParameterNames);
			}
			else if (TComponentPtr<const FName> VectorParameterNames = TComponentPtr<const FName>(ScalarVectorOrColorParameterInfosOrNames.Get<4>()))
			{
				this->VectorParameterTracker.VisitActiveAllocation(Allocation, InObjects, VectorParameterNames);
			}
			else if (TComponentPtr<const FName> ColorParameterNames = TComponentPtr<const FName>(ScalarVectorOrColorParameterInfosOrNames.Get<5>()))
			{
				this->VectorParameterTracker.VisitActiveAllocation(Allocation, InObjects, ColorParameterNames);
			}
		};

		// First step handle any new or updated bound materials
		FEntityTaskBuilder()
		.Read(TracksComponents->BoundMaterial)
		// Handle both new parameter info and old parameter names for backwards compatibility with MovieSceneParameterSection
		.ReadOneOf(	TracksComponents->ScalarMaterialParameterInfo, 
					TracksComponents->VectorMaterialParameterInfo, 
					TracksComponents->ColorMaterialParameterInfo,
					TracksComponents->ScalarParameterName,
					TracksComponents->VectorParameterName,
					TracksComponents->ColorParameterName)
		.FilterAny({ BuiltInComponents->Tags.NeedsLink, TracksComponents->Tags.BoundMaterialChanged })
		.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerAllocation(&Linker->EntityManager, HandleUpdatedAllocation);

		// Next handle any entities that are going away
		FEntityTaskBuilder()
		.FilterAll({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerAllocation(&Linker->EntityManager, HandleUnlinkedAllocation);

		// Process all blended scalar parameters
		TOverlappingMaterialParameterHandler<FScalarMixin> ScalarHandler(this);
		ScalarParameterTracker.ProcessInvalidatedOutputs(Linker, ScalarHandler);

		// Process all blended vector parameters
		TOverlappingMaterialParameterHandler<FVectorMixin> VectorHandler(this);
		VectorParameterTracker.ProcessInvalidatedOutputs(Linker, VectorHandler);

		// Gather inputs that contribute to the material parameter by excluding outputs (which will not have an instance handle)
		TPreAnimatedStateTaskParams<FObjectComponent, FMaterialParameterInfo> Params;
		Params.AdditionalFilter.Reset();
		Params.AdditionalFilter.None({ TracksComponents->MPC, BuiltInComponents->BlendChannelOutput });
		Params.AdditionalFilter.Any({ BuiltInComponents->Tags.NeedsLink, TracksComponents->Tags.BoundMaterialChanged });
		
		// Handle both new parameter info and old parameter names for backwards compatibility with MovieSceneParameterSection
		ScalarParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, TracksComponents->BoundMaterial, TracksComponents->ScalarMaterialParameterInfo);
		ScalarParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, TracksComponents->BoundMaterial, TracksComponents->ScalarParameterName);
		VectorParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, TracksComponents->BoundMaterial, TracksComponents->VectorMaterialParameterInfo);
		VectorParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, TracksComponents->BoundMaterial, TracksComponents->ColorMaterialParameterInfo);
		VectorParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, TracksComponents->BoundMaterial, TracksComponents->VectorParameterName);
		VectorParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, TracksComponents->BoundMaterial, TracksComponents->ColorParameterName);
	}
}

UMovieSceneMaterialParameterEvaluationSystem::UMovieSceneMaterialParameterEvaluationSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	RelevantComponent = TracksComponents->BoundMaterial;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), TracksComponents->BoundMaterial);

		for (int32 Index = 0; Index < UE_ARRAY_COUNT(BuiltInComponents->DoubleResult); ++Index)
		{
			DefineComponentConsumer(GetClass(), BuiltInComponents->DoubleResult[Index]);
		}

		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneMaterialParameterEvaluationSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(TracksComponents->BoundMaterial)
	// Handle both new parameter info and old parameter names for backwards compatibility with MovieSceneParameterSection
	.ReadOneOrMoreOf(TracksComponents->ScalarMaterialParameterInfo, TracksComponents->ScalarParameterName)
	.Read(BuiltInComponents->DoubleResult[0])
	.FilterNone({ BuiltInComponents->BlendChannelInput })
	.SetDesiredThread(Linker->EntityManager.GetDispatchThread())
	.Fork_PerAllocation<FApplyScalarParameters>(&Linker->EntityManager, TaskScheduler);

	// Vectors and colors use the same API
	FEntityTaskBuilder()
	.Read(TracksComponents->BoundMaterial)
	// Handle both new parameter info and old parameter names for backwards compatibility with MovieSceneParameterSection
	.ReadOneOrMoreOf(TracksComponents->VectorMaterialParameterInfo, TracksComponents->ColorMaterialParameterInfo, TracksComponents->VectorParameterName, TracksComponents->ColorParameterName)
	.ReadOneOrMoreOf(BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2], BuiltInComponents->DoubleResult[3])
	.FilterNone({ BuiltInComponents->BlendChannelInput })
	.SetDesiredThread(Linker->EntityManager.GetDispatchThread())
	.Fork_PerAllocation<FApplyVectorParameters>(&Linker->EntityManager, TaskScheduler);
}

void UMovieSceneMaterialParameterEvaluationSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	if (Linker->EntityManager.ContainsComponent(TracksComponents->ScalarMaterialParameterInfo) || Linker->EntityManager.ContainsComponent(TracksComponents->ScalarParameterName))
	{
		FEntityTaskBuilder()
		.Read(TracksComponents->BoundMaterial)	
		// Handle both new parameter info and old parameter names for backwards compatibility with MovieSceneParameterSection
		.ReadOneOrMoreOf(TracksComponents->ScalarMaterialParameterInfo, TracksComponents->ScalarParameterName)
		.Read(BuiltInComponents->DoubleResult[0])
		.FilterNone({ BuiltInComponents->BlendChannelInput })
		.SetDesiredThread(Linker->EntityManager.GetDispatchThread())
		.Dispatch_PerAllocation<FApplyScalarParameters>(&Linker->EntityManager, InPrerequisites, &Subsequents);
	}

	// Vectors and colors use the same API
	if (Linker->EntityManager.ContainsComponent(TracksComponents->VectorMaterialParameterInfo) 
		|| Linker->EntityManager.ContainsComponent(TracksComponents->ColorMaterialParameterInfo)
		|| Linker->EntityManager.ContainsComponent(TracksComponents->VectorParameterName)
		|| Linker->EntityManager.ContainsComponent(TracksComponents->ColorParameterName))
	{
		FEntityTaskBuilder()
		.Read(TracksComponents->BoundMaterial)
		.ReadOneOrMoreOf(TracksComponents->VectorMaterialParameterInfo, TracksComponents->ColorMaterialParameterInfo, TracksComponents->VectorParameterName, TracksComponents->ColorParameterName)
		.ReadOneOrMoreOf(BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2], BuiltInComponents->DoubleResult[3])
		.FilterNone({ BuiltInComponents->BlendChannelInput })
		.SetDesiredThread(Linker->EntityManager.GetDispatchThread())
		.Dispatch_PerAllocation<FApplyVectorParameters>(&Linker->EntityManager, InPrerequisites, &Subsequents);
	}
}

