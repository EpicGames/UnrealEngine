// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneMaterialSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMaterialSystem)

namespace UE::MovieScene
{

bool GUseSoftObjectPtrsForPreAnimatedMaterial = true;
FAutoConsoleVariableRef CVarUseSoftObjectPtrsForPreAnimatedMaterial(
	TEXT("Sequencer.UseSoftObjectPtrsForPreAnimatedMaterial"),
	GUseSoftObjectPtrsForPreAnimatedMaterial,
	TEXT("Defines whether to use soft-object-ptrs for referencing pre-animated state (default), or strong TObjectPtrs.\n"),
	ECVF_Default
);

} // namespace UE::MovieScene


UMaterialInterface* FMovieScenePreAnimatedMaterialParameters::GetMaterial() const
{
	if (UE::MovieScene::GUseSoftObjectPtrsForPreAnimatedMaterial)
	{
		UMaterialInterface* Material = SoftPreviousMaterial.Get();
		if (!Material)
		{
			Material = SoftPreviousMaterial.LoadSynchronous();
		}
		return Material;
	}
	else
	{
		return PreviousMaterial;
	}
}

void FMovieScenePreAnimatedMaterialParameters::SetMaterial(UMaterialInterface* InMaterial)
{
	if (UE::MovieScene::GUseSoftObjectPtrsForPreAnimatedMaterial)
	{
		SoftPreviousMaterial = InMaterial;
	}
	else
	{
		PreviousMaterial = InMaterial;
	}
}
