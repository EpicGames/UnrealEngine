// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneMotionVectorSimulationSystem.h"
#include "Systems/MovieSceneTransformOriginSystem.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "Tracks/IMovieSceneTransformOrigin.h"
#include "MovieSceneTracksComponentTypes.h"

#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/MovieSceneComponentTransformSystem.h"

#include "Rendering/MotionVectorSimulation.h"

#include "IMovieScenePlayer.h"
#include "IMovieScenePlaybackClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMotionVectorSimulationSystem)

namespace UE
{
namespace MovieScene
{


FFrameTime GetSimulatedMotionVectorTime(const FMovieSceneContext& Context)
{
	FFrameTime DeltaTime     = FMath::Max(Context.GetDelta(), 0.0041666666666667 * Context.GetFrameRate());
	FFrameTime SimulatedTime = Context.GetOffsetTime(DeltaTime);

	return SimulatedTime;
}


} // namespace MovieScene
} // namespace UE


UMovieSceneMotionVectorSimulationSystem::UMovieSceneMotionVectorSimulationSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Finalization;
}

bool UMovieSceneMotionVectorSimulationSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	return FMotionVectorSimulation::IsEnabled();
}

void UMovieSceneMotionVectorSimulationSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();
	if (Runner->GetCurrentPhase() == ESystemPhase::Finalization && bSimulationEnabled)
	{
		Runner->GetQueuedEventTriggers().AddLambda([this] { this->OnPostEvaluation(); });
	}

}

void UMovieSceneMotionVectorSimulationSystem::OnPostEvaluation()
{
	using namespace UE::MovieScene;

	if (bSimulateTransformsRequested)
	{
		ComputeSimulatedMotion();
	}

	PropagateMotionToComponents();
}

void UMovieSceneMotionVectorSimulationSystem::ComputeSimulatedMotion()
{
	using namespace UE::MovieScene;

	// ---------------------------------------------------------------------------------------
	// Simulated transforms are computed by disabling everything in the ecs that is not a
	// component transform, fudging the eval times, then harvesting the results.

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	// --------------------------------------------------------------------------------------------------------------------------------------------
	// Disable everything and re-execute the evaluation phase
	Disable();
	if (UMovieSceneComponentTransformSystem* ComponentTransformSystem = Linker->FindSystem<UMovieSceneComponentTransformSystem>())
	{
		ComponentTransformSystem->Disable();
	}
	if (UMovieSceneEvalTimeSystem* EvalTime = Linker->FindSystem<UMovieSceneEvalTimeSystem>())
	{
		EvalTime->Disable();
	}

	// Simulate a structure change to force the persistent tasks to be re-scheduled without
	// the systems we just disabled.
	Linker->EntityManager.MimicStructureChanged();

	FEntityComponentFilter* GlobalFilter     = &Linker->EntityManager.ModifyGlobalIterationFilter();
	FEntityComponentFilter  GlobalFilterCopy = *GlobalFilter;

	// Only allow visitation of component transforms for this procedure
	GlobalFilter->All({ TracksComponents->ComponentTransform.PropertyTag });

	// --------------------------------------------------------------------------------------------------------------------------------------------
	// Fudge frame times based on GetSimulatedMotionVectorTime
	{
		TArray<FFrameTime> FrameTimesByInstance;

		FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

		FrameTimesByInstance.SetNum(InstanceRegistry->GetSparseInstances().GetMaxIndex());
		for (auto It = InstanceRegistry->GetSparseInstances().CreateConstIterator(); It; ++It)
		{
			FrameTimesByInstance[It.GetIndex()] = GetSimulatedMotionVectorTime(It->GetContext());
		}

		FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Write(BuiltInComponents->EvalTime)
		.FilterNone({ FBuiltInComponentTypes::Get()->Tags.FixedTime })
		.Iterate_PerEntity(&Linker->EntityManager,
			[&FrameTimesByInstance](UE::MovieScene::FInstanceHandle InstanceHandle, FFrameTime& EvalTime)
			{
				EvalTime = FrameTimesByInstance[InstanceHandle.InstanceID];
			}
		);
	}

	// --------------------------------------------------------------------------------------------------------------------------------------------
	// Re-execute the evaluation phase to flush the new transforms
	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();
	Runner->FlushSingleEvaluationPhase();

	// --------------------------------------------------------------------------------------------------------------------------------------------
	// Harvest results
	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObject)
	.ReadAnyOf(
		BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2],
		BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5],
		BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8])
	.FilterAll({ TracksComponents->ComponentTransform.PropertyTag })
	.FilterAny({ BuiltInComponents->CustomPropertyIndex, BuiltInComponents->FastPropertyOffset, BuiltInComponents->SlowProperty })
	.FilterAny({ 
		// must have at least one float result component
		BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2],
		BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5],
		BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8] })
	.Iterate_PerEntity(&Linker->EntityManager, [this](UObject* BoundObject, const double* T_X, const double* T_Y, const double* T_Z, const double* R_X, const double* R_Y, const double* R_Z, const double* S_X, const double* S_Y, const double* S_Z)
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(BoundObject))
		{
			FTransform SimulatedTransform = SceneComponent->GetRelativeTransform();

			FVector    SimulatedLocation  = SimulatedTransform.GetLocation();
			FRotator   SimulatedRotation  = SimulatedTransform.Rotator();
			FVector    SimulatedScale     = SimulatedTransform.GetScale3D();

			if (T_X) { SimulatedLocation.X     = *T_X; }
			if (T_Y) { SimulatedLocation.Y     = *T_Y; }
			if (T_Z) { SimulatedLocation.Z     = *T_Z; }
			if (R_X) { SimulatedRotation.Roll  = *R_X; }
			if (R_Y) { SimulatedRotation.Pitch = *R_Y; }
			if (R_Z) { SimulatedRotation.Yaw   = *R_Z; }
			if (S_X) { SimulatedScale.X        = *S_X; }
			if (S_Y) { SimulatedScale.Y        = *S_Y; }
			if (S_Z) { SimulatedScale.Z        = *S_Z; }

			this->AddSimulatedTransform(SceneComponent, FTransform(SimulatedRotation, SimulatedLocation, SimulatedScale), NAME_None);
		}
	});

	// Reset the global filter to what it was before
	*GlobalFilter = MoveTemp(GlobalFilterCopy);

	// --------------------------------------------------------------------------------------------------------------------------------------------
	// Re-enable systems and reset the simulation components
	if (UMovieSceneComponentTransformSystem* ComponentTransformSystem = Linker->FindSystem<UMovieSceneComponentTransformSystem>())
	{
		ComponentTransformSystem->Enable();
	}
	if (UMovieSceneEvalTimeSystem* EvalTime = Linker->FindSystem<UMovieSceneEvalTimeSystem>())
	{
		EvalTime->Enable();
	}
	Enable();

	// Force persistent tasks to be re-created again (see previous comment).
	Linker->EntityManager.MimicStructureChanged();

	bSimulateTransformsRequested = false;
}

void UMovieSceneMotionVectorSimulationSystem::EnableThisFrame()
{
	bSimulationEnabled = true;
}

void UMovieSceneMotionVectorSimulationSystem::SimulateAllTransforms()
{
	bSimulationEnabled = true;
	bSimulateTransformsRequested = true;
}

void UMovieSceneMotionVectorSimulationSystem::PreserveSimulatedMotion(bool bShouldPreserveTransforms)
{
	bPreserveTransforms = bShouldPreserveTransforms;
}

void UMovieSceneMotionVectorSimulationSystem::AddSimulatedTransform(USceneComponent* Component, const FTransform& SimulatedTransform, FName SocketName)
{
	TransformData.Add(Component, FSimulatedTransform(SimulatedTransform, SocketName));
}

void UMovieSceneMotionVectorSimulationSystem::PropagateMotionToComponents()
{
	TSet<USceneComponent*> RootComponents;

	for (auto It = TransformData.CreateIterator(); It; ++It)
	{
		// If this is a socket transform, we want to add the component as a whole, to ensure that anything attached to this socket gets simulated correctly

		USceneComponent* Component = Cast<USceneComponent>(It.Key().ResolveObjectPtr());
		if (!Component || HavePreviousTransformForParent(Component))
		{
			continue;
		}

		RootComponents.Add(Component);
	}

	for (USceneComponent* Component : RootComponents)
	{
		FTransform ParentToWorld = FTransform::Identity;

		USceneComponent* ParentComp = Component->GetAttachParent();
		FName AttachSocket = Component->GetAttachSocketName();
		if (ParentComp)
		{
			FTransform ParentTransform = ParentComp->GetSocketTransform(AttachSocket, RTS_World);
			if (!Component->IsUsingAbsoluteLocation())
			{
				ParentToWorld.SetTranslation(ParentTransform.GetTranslation());
			}
			if (!Component->IsUsingAbsoluteRotation())
			{
				ParentToWorld.SetRotation(ParentTransform.GetRotation());
			}
			if (!Component->IsUsingAbsoluteScale())
			{
				ParentToWorld.SetScale3D(ParentTransform.GetScale3D());
			}
		}

		ApplySimulatedTransforms(Component, GetTransform(Component) * ParentToWorld);
	}

	if (!bPreserveTransforms)
	{
		TransformData.Reset();
		bSimulationEnabled = false;
	}
}

FTransform UMovieSceneMotionVectorSimulationSystem::GetTransform(USceneComponent* Component)
{
	FObjectKey Key(Component);
	for (auto It = TransformData.CreateConstKeyIterator(Key); It; ++It)
	{
		if (It.Value().SocketName == NAME_None)
		{
			return It.Value().Transform;
		}
	}

	return Component->GetRelativeTransform();
}

FTransform UMovieSceneMotionVectorSimulationSystem::GetSocketTransform(USceneComponent* Component, FName SocketName)
{
	FObjectKey Key(Component);
	for (auto It = TransformData.CreateConstKeyIterator(Key); It; ++It)
	{
		if (It.Value().SocketName == SocketName)
		{
			return It.Value().Transform;
		}
	}

	return Component->GetSocketTransform(SocketName, RTS_Component);
}

bool UMovieSceneMotionVectorSimulationSystem::HavePreviousTransformForParent(USceneComponent* InComponent) const
{
	USceneComponent* Parent = InComponent->GetAttachParent();
	return Parent && (TransformData.Contains(Parent) || HavePreviousTransformForParent(Parent));
}

void UMovieSceneMotionVectorSimulationSystem::ApplySimulatedTransforms(USceneComponent* InComponent, const FTransform& InPreviousTransform)
{
	check(InComponent);
	FMotionVectorSimulation::Get().SetPreviousTransform(InComponent, InPreviousTransform);

	for (USceneComponent* Child : InComponent->GetAttachChildren())
	{
		if (!Child)
		{
			continue;
		}

		FName AttachSocketName = Child->GetAttachSocketName();

		FTransform SocketTransform = (AttachSocketName == NAME_None) ? FTransform::Identity : GetSocketTransform(InComponent, AttachSocketName);
		FTransform ParentToWorld = SocketTransform * InPreviousTransform;

		if (Child->IsUsingAbsoluteLocation())
		{
			ParentToWorld.SetTranslation(FVector::ZeroVector);
		}
		if (Child->IsUsingAbsoluteRotation())
		{
			ParentToWorld.SetRotation(FQuat::Identity);
		}
		if (Child->IsUsingAbsoluteScale())
		{
			ParentToWorld.SetScale3D(FVector(1.f, 1.f, 1.f));
		}

		FTransform ChildTransform = GetTransform(Child);
		ApplySimulatedTransforms(Child, ChildTransform * ParentToWorld);
	}
}

