// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieScenePiecewiseBoolBlenderSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePiecewiseBoolBlenderSystem)

namespace UE
{
namespace MovieScene
{

/** 
 * Custom blend result traits for booleans:
 *
 * - We don't blend booleans, we just OR them.
 * - Number of contributors doesn't matter. If there's at least one true value, the result is true.
 */
template<>
struct TSimpleBlendResultTraits<bool>
{
	static void ZeroAccumulationBuffer(TArrayView<TSimpleBlendResult<bool>> Buffer)
	{
		FMemory::Memzero(Buffer.GetData(), sizeof(TSimpleBlendResult<bool>) * Buffer.Num());
	}

	static void AccumulateResult(TSimpleBlendResult<bool>& InOutValue, bool Contributor)
	{
		InOutValue.Value |= Contributor;
		++InOutValue.NumContributors;
	}

	static bool BlendResult(const TSimpleBlendResult<bool>& InResult)
	{
		return InResult.Value;
	}
};

} // namespace MovieScene
} // namespace UE

UMovieScenePiecewiseBoolBlenderSystem::UMovieScenePiecewiseBoolBlenderSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Scheduling;
	
	Impl.Setup(
			this, 
			UE::MovieScene::FBuiltInComponentTypes::Get()->BoolResult,
			nullptr);
}

void UMovieScenePiecewiseBoolBlenderSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	CompactBlendChannels();

	Impl.Schedule(Linker, AllocatedBlendChannels, TaskScheduler);
}

void UMovieScenePiecewiseBoolBlenderSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	CompactBlendChannels();

	Impl.Run(Linker, AllocatedBlendChannels, InPrerequisites, Subsequents);
}


