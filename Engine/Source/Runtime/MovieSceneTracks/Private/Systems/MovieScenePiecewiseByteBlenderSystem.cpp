// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieScenePiecewiseByteBlenderSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "Systems/ByteChannelEvaluatorSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePiecewiseByteBlenderSystem)

UMovieScenePiecewiseByteBlenderSystem::UMovieScenePiecewiseByteBlenderSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	Impl.Setup(
			this, 
			UE::MovieScene::FBuiltInComponentTypes::Get()->ByteResult,
			UByteChannelEvaluatorSystem::StaticClass());
}

void UMovieScenePiecewiseByteBlenderSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	CompactBlendChannels();

	Impl.Run(Linker, AllocatedBlendChannels, InPrerequisites, Subsequents);
}


