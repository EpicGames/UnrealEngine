// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/MovieSceneBaseValueEvaluatorSystem.h"
#include "Systems/MovieSceneQuaternionInterpolationRotationSystem.h"
#include "Systems/WeightAndEasingEvaluatorSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePiecewiseDoubleBlenderSystem)

DECLARE_CYCLE_STAT(TEXT("Blend double values"),                   MovieSceneEval_BlendDoubleValues,        STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Default combine blended double values"), MovieSceneEval_BlendCombineDoubleValues, STATGROUP_MovieSceneECS);
 
namespace UE
{
namespace MovieScene
{

struct FForkedAccumulationTask
{
	FForkedAccumulationTask(TArray<FBlendResult>* InAccumulationBuffer)
		: AccumulationBuffer(InAccumulationBuffer)
	{}

	void ForEachAllocation(const FEntityAllocation* InAllocation, const double* InResults, const FMovieSceneBlendChannelID* BlendIDs, const double* OptionalEasingAndWeights) const
	{
		const int32 Num = InAllocation->Num();
		if (OptionalEasingAndWeights)
		{
			// We have some easing/weight factors to multiply values with.
			for (int32 Index = 0; Index < Num; ++Index)
			{
				FBlendResult& Result = (*AccumulationBuffer)[BlendIDs[Index].ChannelID];

				const float Weight = OptionalEasingAndWeights[Index];
				Result.Total += InResults[Index] * Weight;
				Result.Weight += Weight;
			}
		}
		else
		{
			// Faster path for when there's no weight to multiply values with.
			for (int32 Index = 0; Index < Num; ++Index)
			{
				FBlendResult& Result = (*AccumulationBuffer)[BlendIDs[Index].ChannelID];
				Result.Total += InResults[Index];
				Result.Weight += 1.f;
			}
		}
	}

	TArray<FBlendResult>* AccumulationBuffer;
};

/** Task for accumulating all weighted blend inputs into arrays based on BlendID. Will be run for Absolute, Additive and Relative blend modes*/
struct FAccumulationTask
{
	FAccumulationTask(TSortedMap<FComponentTypeID, TArray<FBlendResult>>* InAccumulationBuffers)
		: AccumulationBuffers(InAccumulationBuffers)
	{}

	/** Task entry point - iterates the allocation's headers and accumulates results for any required components */
	void ForEachAllocation(FEntityAllocationIteratorItem InItem, TRead<FMovieSceneBlendChannelID> BlendIDs, TReadOptional<double> OptionalEasingAndWeights) const
	{
		const FEntityAllocation* Allocation = InItem;
		const FComponentMask& AllocationType = InItem;

		for (const FComponentHeader& ComponentHeader : Allocation->GetComponentHeaders())
		{
			if (TArray<FBlendResult>* AccumulationBuffer = AccumulationBuffers->Find(ComponentHeader.ComponentType))
			{
				if (Allocation->GetCurrentLockMode() != EComponentHeaderLockMode::LockFree)
				{
					ComponentHeader.ReadWriteLock.ReadLock();
				}

				const double* Results = static_cast<const double*>(ComponentHeader.GetValuePtr(0));
				AccumulateResults(Allocation, Results, BlendIDs, OptionalEasingAndWeights, *AccumulationBuffer);

				if (Allocation->GetCurrentLockMode() != EComponentHeaderLockMode::LockFree)
				{
					ComponentHeader.ReadWriteLock.ReadUnlock();
				}
			}
		}
	}

private:

	void AccumulateResults(const FEntityAllocation* InAllocation, const double* InResults, const FMovieSceneBlendChannelID* BlendIDs, const double* OptionalEasingAndWeights, TArray<FBlendResult>& OutBlendResults) const
	{
		static const FMovieSceneBlenderSystemID BlenderSystemID = UMovieSceneBlenderSystem::GetBlenderSystemID<UMovieScenePiecewiseDoubleBlenderSystem>();

		const int32 Num = InAllocation->Num();
		if (OptionalEasingAndWeights)
		{
			// We have some easing/weight factors to multiply values with.
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneBlendChannelID& BlendID(BlendIDs[Index]);
				ensureMsgf(BlendID.SystemID == BlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));

				if (!OutBlendResults.IsValidIndex(BlendID.ChannelID))
				{
					OutBlendResults.SetNum(BlendID.ChannelID + 1, EAllowShrinking::No);
				}

				FBlendResult& Result = OutBlendResults[BlendID.ChannelID];

				const float Weight = OptionalEasingAndWeights[Index];
				Result.Total += InResults[Index] * Weight;
				Result.Weight += Weight;
			}
		}
		else
		{
			// Faster path for when there's no weight to multiply values with.
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneBlendChannelID& BlendID(BlendIDs[Index]);
				ensureMsgf(BlendID.SystemID == BlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));

				if (!OutBlendResults.IsValidIndex(BlendID.ChannelID))
				{
					OutBlendResults.SetNum(BlendID.ChannelID + 1, EAllowShrinking::No);
				}

				FBlendResult& Result = OutBlendResults[BlendID.ChannelID];
				Result.Total += InResults[Index];
				Result.Weight += 1.f;
			}
		}
	}

	TSortedMap<FComponentTypeID, TArray<FBlendResult>>* AccumulationBuffers;
};

/** Same as the task above, but also reads a "base value" that is subtracted from all values.
 *
 *  Only used by entities with the "additive from base" blend type.
 */
struct FAdditiveFromBaseBlendTask
{
	TSortedMap<FComponentTypeID, FAdditiveFromBaseBuffer>* AccumulationBuffers;

	void ForEachAllocation(FEntityAllocationIteratorItem InItem, TRead<FMovieSceneBlendChannelID> BlendIDs, TReadOptional<double> EasingAndWeightResults) const
	{
		FEntityAllocation* Allocation = InItem;
		const FComponentMask& AllocationType = InItem;

		for (const FComponentHeader& ComponentHeader : Allocation->GetComponentHeaders())
		{
			if (FAdditiveFromBaseBuffer* Buffer = AccumulationBuffers->Find(ComponentHeader.ComponentType))
			{
				TComponentReader<double> BaseValues = Allocation->ReadComponents(Buffer->BaseComponent.template ReinterpretCast<double>());
				TComponentReader<double> Results(&ComponentHeader, Allocation->GetCurrentLockMode());

				AccumulateResults(Allocation, Results.AsPtr(), BaseValues.AsPtr(), BlendIDs, EasingAndWeightResults, Buffer->Buffer);
			}
		}
	}

private:

	void AccumulateResults(const FEntityAllocation* InAllocation, const double* Results, const double* BaseValues, const FMovieSceneBlendChannelID* BlendIDs, const double* OptionalEasingAndWeights, TArray<FBlendResult>& OutBlendResults) const
	{
		static const FMovieSceneBlenderSystemID BlenderSystemID = UMovieSceneBlenderSystem::GetBlenderSystemID<UMovieScenePiecewiseDoubleBlenderSystem>();

		const int32 Num = InAllocation->Num();

		if (OptionalEasingAndWeights)
		{
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneBlendChannelID& BlendID(BlendIDs[Index]);
				ensureMsgf(BlendID.SystemID == BlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));

				FBlendResult& Result = OutBlendResults[BlendID.ChannelID];

				const float Weight = OptionalEasingAndWeights[Index];
				Result.Total  += (Results[Index] - BaseValues[Index]) * Weight;
				Result.Weight += Weight;
			}
		}
		else
		{
			// Faster path for when there's no weight to multiply values with.
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneBlendChannelID& BlendID(BlendIDs[Index]);
				ensureMsgf(BlendID.SystemID == BlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));

				FBlendResult& Result = OutBlendResults[BlendID.ChannelID];
				Result.Total  += (Results[Index] - BaseValues[Index]);
				Result.Weight += 1.f;
			}
		}
	}
};

/** Task to gather all of the additives and overrides that need to get blended in order.
 *
 *  Only used by entities with the specified blending order.
 */
struct FAdditiveAndOverridesTask
{
	TSortedMap<FComponentTypeID, FAdditiveAndOverrideBuffer>* AccumulationBuffers;

	void ForEachAllocation(FEntityAllocationIteratorItem InItem, TRead<FMovieSceneBlendChannelID> BlendIDs, TRead<int32> BlendingOrder, TReadOptional<double> EasingAndWeightResults) const
	{
		FEntityAllocation* Allocation = InItem;
		const FComponentMask& AllocationType = InItem;

		for (const FComponentHeader& ComponentHeader : Allocation->GetComponentHeaders())
		{
			if (FAdditiveAndOverrideBuffer* Buffer = AccumulationBuffers->Find(ComponentHeader.ComponentType))
			{
				TComponentReader<double> Results(&ComponentHeader, Allocation->GetCurrentLockMode());

				AccumulateResults(Allocation, Results.AsPtr(), BlendingOrder, BlendIDs, EasingAndWeightResults, Buffer->Values);
			}
		}
	}


private:

	void AccumulateResults(const FEntityAllocation* InAllocation, const double* Results, const int32* BlendingOrder, const FMovieSceneBlendChannelID* BlendIDs, const double* OptionalEasingAndWeights, TArray<FAdditiveAndOverrides>& OutOverrides) const
	{
		static const FMovieSceneBlenderSystemID BlenderSystemID = UMovieSceneBlenderSystem::GetBlenderSystemID<UMovieScenePiecewiseDoubleBlenderSystem>();

		const int32 Num = InAllocation->Num();
		const bool bAdditive = InAllocation->HasComponent(FBuiltInComponentTypes::Get()->Tags.AdditiveBlend);
		const bool bOverride = InAllocation->HasComponent(FBuiltInComponentTypes::Get()->Tags.OverrideBlend);
		if (!bAdditive && !bOverride)
		{
			return;
		}
		if (OptionalEasingAndWeights)
		{
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneBlendChannelID& BlendID(BlendIDs[Index]);
				ensureMsgf(BlendID.SystemID == BlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));

				FAdditiveAndOverrides& Result = OutOverrides[BlendID.ChannelID];
				FAdditveAndOverrideData Data;
				Data.bIsAdditive = bAdditive;
				Data.Value = Results[Index];
				Data.Weight = OptionalEasingAndWeights[Index];
				Data.BlendingOrder = BlendingOrder[Index];
				Result.Data.Add(Data.BlendingOrder, Data);
			}
		}
		else
		{
			// Faster path for when there's no weight to multiply values with.
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneBlendChannelID& BlendID(BlendIDs[Index]);
				ensureMsgf(BlendID.SystemID == BlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));
				FAdditiveAndOverrides& Result = OutOverrides[BlendID.ChannelID];
				FAdditveAndOverrideData Data;
				Data.bIsAdditive = bAdditive;
				Data.Value = Results[Index];
				Data.Weight = 1.0;
				Data.BlendingOrder = BlendingOrder[Index];
				Result.Data.Add(Data.BlendingOrder, Data);
			}
		}
	}
};

void AddOverrideResults(const FAdditiveAndOverrides& Overrides, double& OutFinalBlendValue)
{
	for (const TPair<int32,FAdditveAndOverrideData>& Data : Overrides.Data)
	{
		if (Data.Value.bIsAdditive)
		{
			OutFinalBlendValue += (Data.Value.Value * Data.Value.Weight);
		}
		else //override do weighted blend
		{
			if (Data.Value.Weight >= 1.0)
			{
				OutFinalBlendValue = Data.Value.Value;
			}
			else
			{
				OutFinalBlendValue =(OutFinalBlendValue *(1.0 - Data.Value.Weight)) +
					(Data.Value.Value * Data.Value.Weight);
			}
		}
	}
}

void BlendResults(const FAccumulationResult& Results, uint16 BlendID, double& OutFinalBlendResult)
{
	FBlendResult AbsoluteResult = Results.GetAbsoluteResult(BlendID);
	FBlendResult AdditiveResult = Results.GetAdditiveResult(BlendID);
	FBlendResult AdditiveFromBaseResult = Results.GetAdditiveFromBaseResult(BlendID);
	FAdditiveAndOverrides Overrides = Results.GetAdditiveAndOverrideResult(BlendID);
#if DO_GUARD_SLOW
	ensureMsgf(AbsoluteResult.Weight != 0.f, TEXT("Default blend combine being used for an entity that has no absolute weight. This should have an initial value and should be handled by each system, and excluded by default with UMovieSceneBlenderSystem::FinalCombineExclusionFilter ."));
#endif

	const float TotalWeight = AbsoluteResult.Weight;
	if (TotalWeight != 0)
	{
		const double Value = AbsoluteResult.Total / AbsoluteResult.Weight + AdditiveResult.Total + AdditiveFromBaseResult.Total;
		OutFinalBlendResult = Value;
	}
	if (Overrides.Data.Num() > 0)
	{
		AddOverrideResults(Overrides, OutFinalBlendResult);
	}
}

void BlendResultsWithInitial(const FAccumulationResult& Results, uint16 BlendID, const double InitialValue, double& OutFinalBlendResult)
{
	FBlendResult AbsoluteResult = Results.GetAbsoluteResult(BlendID);
	FBlendResult RelativeResult = Results.GetRelativeResult(BlendID);
	FBlendResult AdditiveResult = Results.GetAdditiveResult(BlendID);
	FBlendResult AdditiveFromBaseResult = Results.GetAdditiveFromBaseResult(BlendID);
	FAdditiveAndOverrides Overrides = Results.GetAdditiveAndOverrideResult(BlendID);

	if (RelativeResult.Weight != 0)
	{
		RelativeResult.Total += InitialValue * RelativeResult.Weight;
	}

	FBlendResult TotalAdditiveResult = { AdditiveResult.Total + AdditiveFromBaseResult.Total, AdditiveResult.Weight + AdditiveFromBaseResult.Weight };
	const float TotalWeight = AbsoluteResult.Weight + RelativeResult.Weight;
	if (TotalWeight != 0)
	{
		// If the absolute value has some partial weighting (for ease-in/out for instance), we ramp it from/to the initial value. This means
		// that the "initial value" adds a contribution to the entire blending process, so we add its weight to the total that we
		// normalize absolutes and relatives with.
		//
		// Note that "partial weighting" means strictly between 0 and 100%. At 100% and above, we don't need to do this thing with the initial
		// value. At 0%, we have no absolute value (only a relative value) and we therefore don't want to include the initial value either.
		const bool bInitialValueContributes = (0.f < AbsoluteResult.Weight && AbsoluteResult.Weight < 1.f);
		const double AbsoluteBlendedValue = bInitialValueContributes ?
			(InitialValue * (1.f - AbsoluteResult.Weight) + AbsoluteResult.Total) :
			AbsoluteResult.Total;
		const float FinalTotalWeight = bInitialValueContributes ? (TotalWeight + (1.f - AbsoluteResult.Weight)) : TotalWeight;

		const double Value = (AbsoluteBlendedValue + RelativeResult.Total) / FinalTotalWeight + TotalAdditiveResult.Total;
		OutFinalBlendResult = Value;
	}
	else if (TotalAdditiveResult.Weight != 0)
	{
		OutFinalBlendResult = TotalAdditiveResult.Total + InitialValue;
	}
	else
	{
		OutFinalBlendResult = InitialValue;
	}
	if (Overrides.Data.Num() > 0)
	{
		AddOverrideResults(Overrides, OutFinalBlendResult);
	}
}

/** Task that combines all accumulated blends for any tracked property type that has blend inputs/outputs */
struct FScheduleCombineBlendsForProperties
{
	const FAccumulationBuffers* AccumulationBuffers;
	TArrayView<const FPropertyCompositeDefinition> Composites;
	FEntityAllocationWriteContext WriteContext;
	uint32 DoubleCompositeMask;

	FScheduleCombineBlendsForProperties(const FEntityManager* EntityManager, const FAccumulationBuffers* InAccumulationBuffers, const FPropertyDefinition* InPropertyDefinition)
		: AccumulationBuffers(InAccumulationBuffers)
		, Composites(FBuiltInComponentTypes::Get()->PropertyRegistry.GetComposites(*InPropertyDefinition))
		, WriteContext(*EntityManager)
		, DoubleCompositeMask(InPropertyDefinition->DoubleCompositeMask)
	{}

	void UpdateWriteContext(FEntityAllocationWriteContext InWriteContext)
	{
		WriteContext = InWriteContext;
	}

	void ForEachAllocation(FEntityAllocationIteratorItem Item, const FMovieSceneBlendChannelID* BlendIDs, FReadErasedOptional OptInitialValues) const
	{
		const FComponentMask& AllocationType = Item.GetAllocationType();
		const FEntityAllocation* Allocation = Item.GetAllocation();

		for (int32 CompositeIndex = 0; CompositeIndex < Composites.Num(); ++CompositeIndex)
		{
			const bool bIsCompositeSupported = ((DoubleCompositeMask & (1 << CompositeIndex)) != 0);
			FAccumulationResult Results = AccumulationBuffers->FindResults(Composites[CompositeIndex].ComponentTypeID);
			if (bIsCompositeSupported && AllocationType.Contains(Composites[CompositeIndex].ComponentTypeID) && Results.IsValid())
			{
				TComponentWriter<double> OutValues = Allocation->WriteComponents(Composites[CompositeIndex].ComponentTypeID.ReinterpretCast<double>(), WriteContext);
				ForEachAllocation(Allocation, Results, BlendIDs, OutValues, OptInitialValues, Composites[CompositeIndex].CompositeOffset);
			}
		}
	}

	void ForEachAllocation(const FEntityAllocation* Allocation, FAccumulationResult Results, const FMovieSceneBlendChannelID* BlendIDs, double* OutValues, FReadErasedOptional OptInitialValues, uint16 InitialValueProjectionOffset) const
	{
		if (OptInitialValues.IsValid())
		{
			for (int32 Index = 0; Index < Allocation->Num(); ++Index)
			{
				const double InitialValue = *reinterpret_cast<const double*>(static_cast<const uint8*>(OptInitialValues[Index]) + InitialValueProjectionOffset);
				BlendResultsWithInitial(Results, BlendIDs[Index].ChannelID, InitialValue, OutValues[Index]);
			}
		}
		else
		{
			for (int32 Index = 0; Index < Allocation->Num(); ++Index)
			{
				BlendResults(Results, BlendIDs[Index].ChannelID, OutValues[Index]);
			}
		}
	}
};

/** Task that combines all accumulated blends for any tracked property type that has blend inputs/outputs */
struct FLegacyCombineBlendsForProperties
{
	explicit FLegacyCombineBlendsForProperties(const TBitArray<>& InCachedRelevantProperties, const FAccumulationBuffers* InAccumulationBuffers, FEntityAllocationWriteContext InWriteContext)
		: CachedRelevantProperties(InCachedRelevantProperties)
		, PropertyRegistry(&FBuiltInComponentTypes::Get()->PropertyRegistry)
		, AccumulationBuffers(InAccumulationBuffers)
		, WriteContext(InWriteContext)
	{}

	void ForEachAllocation(FEntityAllocationIteratorItem InItem, TRead<FMovieSceneBlendChannelID> BlendIDs) const
	{
		FEntityAllocation* Allocation = InItem;
		const FComponentMask& AllocationType = InItem;

		// Find out what kind of property this is
		for (TConstSetBitIterator<> PropertyIndex(CachedRelevantProperties); PropertyIndex; ++PropertyIndex)
		{
			const FPropertyDefinition& PropertyDefinition = PropertyRegistry->GetDefinition(FCompositePropertyTypeID::FromIndex(PropertyIndex.GetIndex()));
			if (AllocationType.Contains(PropertyDefinition.PropertyType))
			{
				ProcessPropertyType(Allocation, AllocationType, PropertyDefinition, BlendIDs);
				return;
			}
		}
	}

private:

	void ProcessPropertyType(FEntityAllocation* Allocation, const FComponentMask& AllocationType, const FPropertyDefinition& PropertyDefinition, const FMovieSceneBlendChannelID* BlendIDs) const
	{
		static const FMovieSceneBlenderSystemID BlenderSystemID = UMovieSceneBlenderSystem::GetBlenderSystemID<UMovieScenePiecewiseDoubleBlenderSystem>();

		TArrayView<const FPropertyCompositeDefinition> Composites = PropertyRegistry->GetComposites(PropertyDefinition);

		FOptionalComponentReader OptInitialValues = Allocation->TryReadComponentsErased(PropertyDefinition.InitialValueType);

		for (int32 CompositeIndex = 0; CompositeIndex < Composites.Num(); ++CompositeIndex)
		{
			const bool bIsCompositeSupported = ((PropertyDefinition.DoubleCompositeMask & (1 << CompositeIndex)) != 0);
			if (!bIsCompositeSupported)
			{
				continue;
			}

			TComponentTypeID<double> ResultComponent = Composites[CompositeIndex].ComponentTypeID.template ReinterpretCast<double>();
			if (!AllocationType.Contains(ResultComponent))
			{
				continue;
			}

			FAccumulationResult Results = AccumulationBuffers->FindResults(ResultComponent);
			if (Results.IsValid())
			{
				const uint16 InitialValueProjectionOffset = Composites[CompositeIndex].CompositeOffset;

				// Open the result channel for write
				TComponentWriter<double> ValueResults = Allocation->WriteComponents(ResultComponent, WriteContext);

				if (OptInitialValues)
				{
					for (int32 Index = 0; Index < Allocation->Num(); ++Index)
					{
						ensureMsgf(BlendIDs[Index].SystemID == BlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));
						const double InitialValue = *reinterpret_cast<const double*>(static_cast<const uint8*>(OptInitialValues[Index]) + InitialValueProjectionOffset);
						BlendResultsWithInitial(Results, BlendIDs[Index].ChannelID, InitialValue, ValueResults[Index]);
					}
				}
				else
				{
					for (int32 Index = 0; Index < Allocation->Num(); ++Index)
					{
						ensureMsgf(BlendIDs[Index].SystemID == BlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));
						BlendResults(Results, BlendIDs[Index].ChannelID, ValueResults[Index]);
					}
				}
			}
		}
	}

private:

	TBitArray<> CachedRelevantProperties;
	const FPropertyRegistry* PropertyRegistry;
	const FAccumulationBuffers* AccumulationBuffers;
	FEntityAllocationWriteContext WriteContext;
};

/** Task that combines all accumulated blends for any tracked non-property type that has blend inputs/outputs */
struct FCombineBlends
{
	explicit FCombineBlends(const FAccumulationBuffers* InAccumulationBuffers, FEntityAllocationWriteContext InWriteContext)
		: AccumulationBuffers(InAccumulationBuffers)
		, WriteContext(InWriteContext)
	{}

	void ForEachAllocation(FEntityAllocationIteratorItem InItem, TRead<FMovieSceneBlendChannelID> BlendIDs) const
	{
		static const FMovieSceneBlenderSystemID BlenderSystemID = UMovieSceneBlenderSystem::GetBlenderSystemID<UMovieScenePiecewiseDoubleBlenderSystem>();

		FEntityAllocation* Allocation = InItem;
		const FComponentMask& AllocationType = InItem;

		// Find all result types on this allocation
		TArrayView<TComponentTypeID<double>> ResultComponents(FBuiltInComponentTypes::Get()->DoubleResult);
		for (int32 ResultIndex = 0; ResultIndex < ResultComponents.Num(); ++ResultIndex)
		{
			TComponentTypeID<double> ResultComponent = ResultComponents[ResultIndex];
			if (!AllocationType.Contains(ResultComponent))
			{
				continue;
			}

			FAccumulationResult Results = AccumulationBuffers->FindResults(ResultComponent);
			if (!Results.IsValid())
			{
				continue;
			}

			// Open the result channel for write
			TComponentWriter<double> ValueResults = Allocation->WriteComponents(ResultComponent, WriteContext);
			for (int32 Index = 0; Index < Allocation->Num(); ++Index)
			{
				ensureMsgf(BlendIDs[Index].SystemID == BlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));
				BlendResults(Results, BlendIDs[Index].ChannelID, ValueResults[Index]);
			}
		}
	}

private:

	const FAccumulationBuffers* AccumulationBuffers;
	FEntityAllocationWriteContext WriteContext;
};

bool FAccumulationBuffers::IsEmpty() const
{
	return Absolute.Num() == 0 && Relative.Num() == 0 && Additive.Num() == 0 && AdditiveFromBase.Num() == 0
		&& AdditiveAndOverrides.Num() == 0;
}

void FAccumulationBuffers::Reset()
{
	Absolute.Empty();
	Relative.Empty();
	Additive.Empty();
	AdditiveFromBase.Empty();
	AdditiveAndOverrides.Empty();
}

FAccumulationResult FAccumulationBuffers::FindResults(FComponentTypeID InComponentType) const
{
	FAccumulationResult Result;
	if (const TArray<FBlendResult>* Absolutes = Absolute.Find(InComponentType))
	{
		Result.Absolutes = Absolutes->GetData();
	}
	if (const TArray<FBlendResult>* Relatives = Relative.Find(InComponentType))
	{
		Result.Relatives = Relatives->GetData();
	}
	if (const TArray<FBlendResult>* Additives = Additive.Find(InComponentType))
	{
		Result.Additives = Additives->GetData();
	}
	if (const FAdditiveFromBaseBuffer* AdditivesFromBase = AdditiveFromBase.Find(InComponentType))
	{
		Result.AdditivesFromBase = AdditivesFromBase->Buffer.GetData();
	}
	if (const FAdditiveAndOverrideBuffer* AandOs = AdditiveAndOverrides.Find(InComponentType))
	{
		Result.Overrides = AandOs->Values.GetData();
	}
	return Result;
}

} // namespace MovieScene
} // namespace UE

UMovieScenePiecewiseDoubleBlenderSystem::UMovieScenePiecewiseDoubleBlenderSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UFloatChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBaseValueEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneQuaternionInterpolationRotationSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UWeightAndEasingEvaluatorSystem::StaticClass(), GetClass());
	}
}

void UMovieScenePiecewiseDoubleBlenderSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	CompactBlendChannels();

	FEntityManager& EntityManager = Linker->EntityManager;
	const TStatId BlendValuesStatId = GET_STATID(MovieSceneEval_BlendDoubleValues);
	const TStatId CombineBlendsStatId = GET_STATID(MovieSceneEval_BlendCombineDoubleValues);

	// We allocate space for every blend even if there are gaps so we can do a straight index into each array
	if (AllocatedBlendChannels.Num() == 0)
	{
		return;
	}

	ReinitializeAccumulationBuffers();
	if (AccumulationBuffers.IsEmpty())
	{
		return;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FTaskID ResetWeightsTask = TaskScheduler->AddMemberFunctionTask(
		FTaskParams(TEXT("Reset Double Blender Weights")),
		this, &UMovieScenePiecewiseDoubleBlenderSystem::ZeroAccumulationBuffers);

	FTaskID SyncTask = TaskScheduler->AddNullTask();

	// Do not use weights when ExternalBlending is present
	FComponentTypeIDFilter WeightFilter(BuiltInComponents->Tags.ExternalBlending, false);

	if (AccumulationBuffers.Absolute.Num() != 0)
	{
		for (TPair<FComponentTypeID, TArray<FBlendResult>>& Pair : AccumulationBuffers.Absolute)
		{
			FTaskID AbsoluteTask = FEntityTaskBuilder()
			.Read(Pair.Key.ReinterpretCast<double>())
			.Read(BuiltInComponents->BlendChannelInput)
			.ReadOptional(BuiltInComponents->WeightAndEasingResult, WeightFilter)
			.FilterAll({ BuiltInComponents->Tags.AbsoluteBlend, GetBlenderTypeTag() })
			.FilterAny(BlendedResultMask)
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.AddDynamicReadDependency(BlendedResultMask)
			.SetStat(BlendValuesStatId)
			.Schedule_PerAllocation<FForkedAccumulationTask>(&EntityManager, TaskScheduler, &Pair.Value);

			TaskScheduler->AddPrerequisite(ResetWeightsTask, AbsoluteTask);
			TaskScheduler->AddPrerequisite(AbsoluteTask, SyncTask);
		}
	}

	if (AccumulationBuffers.Relative.Num() != 0)
	{
		for (TPair<FComponentTypeID, TArray<FBlendResult>>& Pair : AccumulationBuffers.Relative)
		{
			FTaskID RelativeTask = FEntityTaskBuilder()
			.Read(Pair.Key.ReinterpretCast<double>())
			.Read(BuiltInComponents->BlendChannelInput)
			.ReadOptional(BuiltInComponents->WeightAndEasingResult, WeightFilter)
			.FilterAll({ BuiltInComponents->Tags.RelativeBlend, GetBlenderTypeTag() })
			.FilterAny(BlendedResultMask)
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.AddDynamicReadDependency(BlendedResultMask)
			.SetStat(BlendValuesStatId)
			.Schedule_PerAllocation<FForkedAccumulationTask>(&EntityManager, TaskScheduler, &Pair.Value);

			TaskScheduler->AddPrerequisite(ResetWeightsTask, RelativeTask);
			TaskScheduler->AddPrerequisite(RelativeTask, SyncTask);
		}
	}

	if (AccumulationBuffers.Additive.Num() != 0)
	{
		for (TPair<FComponentTypeID, TArray<FBlendResult>>& Pair : AccumulationBuffers.Additive)
		{
			FTaskID AdditiveTask = FEntityTaskBuilder()
			.Read(Pair.Key.ReinterpretCast<double>())
			.Read(BuiltInComponents->BlendChannelInput)
			.ReadOptional(BuiltInComponents->WeightAndEasingResult, WeightFilter)
			.FilterAll({ BuiltInComponents->Tags.AdditiveBlend, GetBlenderTypeTag() })
			.FilterAny(BlendedResultMask)
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.AddDynamicReadDependency(BlendedResultMask)
			.SetStat(BlendValuesStatId)
			.Schedule_PerAllocation<FForkedAccumulationTask>(&EntityManager, TaskScheduler, &Pair.Value);

			TaskScheduler->AddPrerequisite(ResetWeightsTask, AdditiveTask);
			TaskScheduler->AddPrerequisite(AdditiveTask, SyncTask);
		}
	}

	if (AccumulationBuffers.AdditiveFromBase.Num() != 0)
	{
		FTaskID AdditiveFromBaseTask = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult, WeightFilter)
		.FilterAll({ BuiltInComponents->Tags.AdditiveFromBaseBlend, GetBlenderTypeTag() })
		.FilterAny(BlendedResultMask)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.AddDynamicReadDependency(BlendedResultMask)
		.SetStat(BlendValuesStatId)
		.Schedule_PerAllocation<FAdditiveFromBaseBlendTask>(&EntityManager, TaskScheduler, &AccumulationBuffers.AdditiveFromBase);

		TaskScheduler->AddPrerequisite(ResetWeightsTask, AdditiveFromBaseTask);
		TaskScheduler->AddPrerequisite(AdditiveFromBaseTask, SyncTask);
	}

	if (AccumulationBuffers.AdditiveAndOverrides.Num() != 0)
	{
		FTaskID OverrideTask = FEntityTaskBuilder()
			.Read(BuiltInComponents->BlendChannelInput)
			.Read(BuiltInComponents->BlendingOrder)
			.ReadOptional(BuiltInComponents->WeightAndEasingResult)
			.FilterAll({ GetBlenderTypeTag() })
			.FilterAny({ BlendedResultMask | BuiltInComponents->Tags.AdditiveBlend |
					BuiltInComponents->Tags.OverrideBlend }).FilterNone({ BuiltInComponents->Tags.Ignored })
			.AddDynamicReadDependency(BlendedResultMask)
			.SetStat(BlendValuesStatId)
			.Schedule_PerAllocation<FAdditiveAndOverridesTask>(&EntityManager, TaskScheduler, &AccumulationBuffers.AdditiveAndOverrides);

		TaskScheduler->AddPrerequisite(ResetWeightsTask, OverrideTask);
		TaskScheduler->AddPrerequisite(OverrideTask, SyncTask);
	}

	// Combine blends for all blend outputs on properties
	if (BlendedPropertyMask.Find(true) != INDEX_NONE)
	{
		// Schedule tasks for every allocation that has a blended property
		for (TConstSetBitIterator<> PropertyIndex(CachedRelevantProperties); PropertyIndex; ++PropertyIndex)
		{
			const FPropertyDefinition& PropertyDefinition = BuiltInComponents->PropertyRegistry.GetDefinition(FCompositePropertyTypeID::FromIndex(PropertyIndex.GetIndex()));
			TArrayView<const FPropertyCompositeDefinition> Composites = BuiltInComponents->PropertyRegistry.GetComposites(PropertyDefinition);

			FComponentMask WriteDependencies;
			for (int32 CompositeIndex = 0; CompositeIndex < Composites.Num(); ++CompositeIndex)
			{
				const bool bIsCompositeSupported = ((PropertyDefinition.DoubleCompositeMask & (1 << CompositeIndex)) != 0);
				if (bIsCompositeSupported)
				{
					WriteDependencies.Set(Composites[CompositeIndex].ComponentTypeID);
				}
			}

			FTaskID CombineTask = FEntityTaskBuilder()
			.Read(BuiltInComponents->BlendChannelOutput)
			.ReadErasedOptional(PropertyDefinition.InitialValueType)
			.FilterAll({ GetBlenderTypeTag() })
			.FilterAny({ PropertyDefinition.PropertyType })
			.AddDynamicWriteDependency(WriteDependencies)
			.SetStat(CombineBlendsStatId)
			.Fork_PerAllocation<FScheduleCombineBlendsForProperties>(
				&EntityManager,
				TaskScheduler,
				&EntityManager,
				&AccumulationBuffers,
				&PropertyDefinition
			);

			TaskScheduler->AddPrerequisite(SyncTask, CombineTask);
		}
	}

	if (bContainsNonPropertyBlends)
	{
		// Blend task that combines vanilla (non-property-based) components
		FTaskID CombineTask = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelOutput)
		.FilterAll({ GetBlenderTypeTag() })
		.FilterAny(BlendedResultMask)
		.FilterNone(BlendedPropertyMask)
		.AddDynamicWriteDependency(MakeArrayView(FBuiltInComponentTypes::Get()->DoubleResult))
		.SetStat(CombineBlendsStatId)
		.Fork_PerAllocation<FCombineBlends>(&EntityManager, TaskScheduler, &AccumulationBuffers, FEntityAllocationWriteContext(EntityManager));
		
		TaskScheduler->AddPrerequisite(SyncTask, CombineTask);
	}
}

void UMovieScenePiecewiseDoubleBlenderSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	CompactBlendChannels();

	FEntityManager& EntityManager = Linker->EntityManager;
	const TStatId BlendValuesStatId = GET_STATID(MovieSceneEval_BlendDoubleValues);
	const TStatId CombineBlendsStatId = GET_STATID(MovieSceneEval_BlendCombineDoubleValues);

	// We allocate space for every blend even if there are gaps so we can do a straight index into each array
	if (AllocatedBlendChannels.Num() == 0)
	{
		return;
	}

	// Update cached channel data if necessary
	if (ChannelRelevancyCache.Update(EntityManager) == ECachedEntityManagerState::Stale)
	{
		ReinitializeAccumulationBuffers();
	}
	else
	{
		ZeroAccumulationBuffers();
	}

	if (AccumulationBuffers.IsEmpty())
	{
		return;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	// Do not use weights when ExternalBlending is present
	FComponentTypeIDFilter WeightFilter(BuiltInComponents->Tags.ExternalBlending, false);

	FSystemTaskPrerequisites Prereqs;
	if (AccumulationBuffers.Absolute.Num() != 0)
	{
		FGraphEventRef Task = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult, WeightFilter)
		.FilterAll({ BuiltInComponents->Tags.AbsoluteBlend, GetBlenderTypeTag() })
		.FilterAny(BlendedResultMask)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(BlendValuesStatId)
		.Dispatch_PerAllocation<FAccumulationTask>(&EntityManager, InPrerequisites, nullptr, &AccumulationBuffers.Absolute);

		if (Task)
		{
			Prereqs.AddRootTask(Task);
		}
	}

	if (AccumulationBuffers.Relative.Num() != 0)
	{
		FGraphEventRef Task = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult, WeightFilter)
		.FilterAll({ BuiltInComponents->Tags.RelativeBlend, GetBlenderTypeTag() })
		.FilterAny(BlendedResultMask)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(BlendValuesStatId)
		.Dispatch_PerAllocation<FAccumulationTask>(&EntityManager, InPrerequisites, nullptr, &AccumulationBuffers.Relative);

		if (Task)
		{
			Prereqs.AddRootTask(Task);
		}
	}

	if (AccumulationBuffers.Additive.Num() != 0)
	{
		FGraphEventRef Task = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult, WeightFilter)
		.FilterAll({ BuiltInComponents->Tags.AdditiveBlend, GetBlenderTypeTag() })
		.FilterAny(BlendedResultMask)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(BlendValuesStatId)
		.Dispatch_PerAllocation<FAccumulationTask>(&EntityManager, InPrerequisites, nullptr, &AccumulationBuffers.Additive);

		if (Task)
		{
			Prereqs.AddRootTask(Task);
		}
	}

	if (AccumulationBuffers.AdditiveAndOverrides.Num() != 0)
	{
		for (TPair<FComponentTypeID, FAdditiveAndOverrideBuffer>& Pair : AccumulationBuffers.AdditiveAndOverrides)
		{
			FGraphEventRef Task = FEntityTaskBuilder()
				.Read(BuiltInComponents->BlendChannelInput)
				.Read(BuiltInComponents->BlendingOrder)
				.ReadOptional(BuiltInComponents->WeightAndEasingResult)
				.FilterAll({ GetBlenderTypeTag() })
				.FilterAny({ BlendedResultMask | BuiltInComponents->Tags.AdditiveBlend |
					 BuiltInComponents->Tags.OverrideBlend })
				.FilterNone({ BuiltInComponents->Tags.Ignored })
				.AddDynamicReadDependency(BlendedResultMask)
				.SetStat(BlendValuesStatId)
				.Dispatch_PerAllocation<FAdditiveAndOverridesTask>(&EntityManager, InPrerequisites, nullptr, &AccumulationBuffers.AdditiveAndOverrides);

			if (Task)
			{
				Prereqs.AddRootTask(Task);
			}
		}
	}

	if (AccumulationBuffers.AdditiveFromBase.Num() != 0)
	{
		FGraphEventRef Task = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult, WeightFilter)
		.FilterAll({ BuiltInComponents->Tags.AdditiveFromBaseBlend, GetBlenderTypeTag() })
		.FilterAny(BlendedResultMask)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(BlendValuesStatId)
		.Dispatch_PerAllocation<FAdditiveFromBaseBlendTask>(&EntityManager, InPrerequisites, nullptr, &AccumulationBuffers.AdditiveFromBase);

		if (Task)
		{
			Prereqs.AddRootTask(Task);
		}
	}

	// Combine blends for all blend outputs on properties
	if (BlendedPropertyMask.Find(true) != INDEX_NONE)
	{
		FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelOutput)
		.FilterAll({ GetBlenderTypeTag() })
		.FilterAny(BlendedPropertyMask)
		.SetStat(CombineBlendsStatId)
		.Dispatch_PerAllocation<FLegacyCombineBlendsForProperties>(&EntityManager, Prereqs, &Subsequents, CachedRelevantProperties, &AccumulationBuffers, FEntityAllocationWriteContext(EntityManager));
	}

	if (bContainsNonPropertyBlends)
	{
		// Blend task that combines vanilla (non-property-based) components
		FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelOutput)
		.FilterAll({ GetBlenderTypeTag() })
		.FilterAny(BlendedResultMask)
		.FilterNone(BlendedPropertyMask)
		.SetStat(CombineBlendsStatId)
		.Dispatch_PerAllocation<FCombineBlends>(&EntityManager, Prereqs, &Subsequents, &AccumulationBuffers, FEntityAllocationWriteContext(EntityManager));
	}
}

void UMovieScenePiecewiseDoubleBlenderSystem::ReinitializeAccumulationBuffers()
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	BlendedResultMask.Reset();
	AccumulationBuffers.Reset();

	TArrayView<TComponentTypeID<double>> BaseComponents(FBuiltInComponentTypes::Get()->BaseDouble);
	TArrayView<TComponentTypeID<double>> ResultComponents(FBuiltInComponentTypes::Get()->DoubleResult);

	TComponentTypeID<int32> BlendingOrder(FBuiltInComponentTypes::Get()->BlendingOrder);

	check(BaseComponents.Num() == ResultComponents.Num());

	// Recompute which result types are blended
	const int32 NumResults = ResultComponents.Num();
	const int32 MaximumNumBlends = AllocatedBlendChannels.Num();
	FEntityManager& EntityManager = Linker->EntityManager;

	FEntityComponentFilter AnyCompositeFilter;
	FComponentMask AnyCompositeAllocationsMask;

	for (int32 Index = 0; Index < NumResults; ++Index)
	{
		TComponentTypeID<double> Component = ResultComponents[Index];

		AnyCompositeFilter.Reset();
		AnyCompositeFilter.All({ Component, GetBlenderTypeTag(), BuiltInComponents->BlendChannelInput });
		AnyCompositeFilter.Any({
				BuiltInComponents->Tags.AbsoluteBlend, 
				BuiltInComponents->Tags.RelativeBlend, 
				BuiltInComponents->Tags.AdditiveBlend, 
				BuiltInComponents->Tags.OverrideBlend,
				BuiltInComponents->Tags.AdditiveFromBaseBlend });

		AnyCompositeAllocationsMask.Reset();
		EntityManager.AccumulateMask(AnyCompositeFilter, AnyCompositeAllocationsMask);

		const bool bHasAbsolutes         = AnyCompositeAllocationsMask.Contains(BuiltInComponents->Tags.AbsoluteBlend);
		const bool bHasRelatives         = AnyCompositeAllocationsMask.Contains(BuiltInComponents->Tags.RelativeBlend);
		const bool bHasAdditives         = AnyCompositeAllocationsMask.Contains(BuiltInComponents->Tags.AdditiveBlend);
		const bool bHasOverrides		 = AnyCompositeAllocationsMask.Contains(BuiltInComponents->Tags.OverrideBlend);
		const bool bHasAdditivesFromBase = AnyCompositeAllocationsMask.Contains(BuiltInComponents->Tags.AdditiveFromBaseBlend);

		if (!(bHasAbsolutes || bHasRelatives || bHasAdditives || bHasAdditivesFromBase || bHasOverrides))
		{
			continue;
		}

		BlendedResultMask.Set(Component);

		if (bHasAbsolutes)
		{
			TArray<FBlendResult>& Buffer = AccumulationBuffers.Absolute.Add(Component);
			Buffer.SetNum(MaximumNumBlends);
		}
		if (bHasRelatives)
		{
			TArray<FBlendResult>& Buffer = AccumulationBuffers.Relative.Add(Component);
			Buffer.SetNum(MaximumNumBlends);
		}
		if (bHasAdditives && !bHasOverrides) //if it has overrides we use additives with them
		{
			TArray<FBlendResult>& Buffer = AccumulationBuffers.Additive.Add(Component);
			Buffer.SetNum(MaximumNumBlends);
		}
		if (bHasAdditivesFromBase)
		{
			FAdditiveFromBaseBuffer& Buffer = AccumulationBuffers.AdditiveFromBase.Add(Component);
			Buffer.Buffer.SetNum(MaximumNumBlends);
			Buffer.BaseComponent = BaseComponents[Index];
		}
		if (bHasOverrides)
		{
			FAdditiveAndOverrideBuffer& Buffer = AccumulationBuffers.AdditiveAndOverrides.Add(Component);
			Buffer.Values.SetNum(MaximumNumBlends);
		}
	}
	
	// Update property relevancy
	CachedRelevantProperties.Empty();

	// If we have no accumulation buffers, we have nothing more to do
	if (AccumulationBuffers.IsEmpty())
	{
		return;
	}

	ZeroAccumulationBuffers();

	FComponentMask AllPropertyTypes;

	// This code works on the assumption that properties can never be removed (which is safe)
	FEntityComponentFilter InclusionFilter;
	TArrayView<const FPropertyDefinition> Properties = BuiltInComponents->PropertyRegistry.GetProperties();
	for (int32 PropertyTypeIndex = 0; PropertyTypeIndex < Properties.Num(); ++PropertyTypeIndex)
	{
		const FPropertyDefinition& PropertyDefinition = Properties[PropertyTypeIndex];
		const bool bHasAnyComposite = (PropertyDefinition.DoubleCompositeMask != 0);
		if (bHasAnyComposite)
		{
			AllPropertyTypes.Set(PropertyDefinition.PropertyType);

			InclusionFilter.Reset();
			InclusionFilter.All({ GetBlenderTypeTag(), BuiltInComponents->BlendChannelOutput, PropertyDefinition.PropertyType });
			if (EntityManager.Contains(InclusionFilter))
			{
				CachedRelevantProperties.PadToNum(PropertyTypeIndex + 1, false);
				CachedRelevantProperties[PropertyTypeIndex] = true;

				BlendedPropertyMask.Set(PropertyDefinition.PropertyType);
			}
		}
	}

	bContainsNonPropertyBlends = EntityManager.Contains(FEntityComponentFilter().All({ GetBlenderTypeTag(), BuiltInComponents->BlendChannelOutput }).None(AllPropertyTypes));
}

void UMovieScenePiecewiseDoubleBlenderSystem::ZeroAccumulationBuffers()
{
	using namespace UE::MovieScene;

	// Arrays should only ever exist in these containers if they have size (they are always initialized to MaximumNumBlends in ReinitializeAccumulationBuffers)
	for (TPair<FComponentTypeID, TArray<FBlendResult>>& Pair : AccumulationBuffers.Absolute)
	{
		FMemory::Memzero(Pair.Value.GetData(), sizeof(FBlendResult)*Pair.Value.Num());
	}
	for (TPair<FComponentTypeID, TArray<FBlendResult>>& Pair : AccumulationBuffers.Relative)
	{
		FMemory::Memzero(Pair.Value.GetData(), sizeof(FBlendResult)*Pair.Value.Num());
	}
	for (TPair<FComponentTypeID, TArray<FBlendResult>>& Pair : AccumulationBuffers.Additive)
	{
		FMemory::Memzero(Pair.Value.GetData(), sizeof(FBlendResult)*Pair.Value.Num());
	}
	for (TPair<FComponentTypeID, FAdditiveFromBaseBuffer>& Pair : AccumulationBuffers.AdditiveFromBase)
	{
		FMemory::Memzero(Pair.Value.Buffer.GetData(), sizeof(FBlendResult)*Pair.Value.Buffer.Num());
	}
}

FGraphEventRef UMovieScenePiecewiseDoubleBlenderSystem::DispatchDecomposeTask(const UE::MovieScene::FValueDecompositionParams& Params, UE::MovieScene::FAlignedDecomposedValue* Output)
{
	using namespace UE::MovieScene;

	if (!Params.ResultComponentType)
	{
		return nullptr;
	}

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	TComponentTypeID<double> ResultComponentType = Params.ResultComponentType.ReinterpretCast<double>();
	TComponentTypeID<double> BaseValueComponentType = BuiltInComponents->GetBaseValueComponentType(Params.ResultComponentType).ReinterpretCast<double>();

	struct FChannelResultTask
	{
		TArray<FMovieSceneEntityID, TInlineAllocator<8>> EntitiesToDecompose;
		FAlignedDecomposedValue* Result;
		uint16 DecomposeBlendChannel;
		FComponentTypeID AdditiveBlendTag;
		FComponentTypeID AdditiveFromBaseBlendTag;
		FComponentTypeID OverrideTag;

		explicit FChannelResultTask(const FValueDecompositionParams& Params, FAlignedDecomposedValue* InResult)
			: Result(InResult)
			, DecomposeBlendChannel(Params.DecomposeBlendChannel)
			, AdditiveBlendTag(FBuiltInComponentTypes::Get()->Tags.AdditiveBlend)
			, AdditiveFromBaseBlendTag(FBuiltInComponentTypes::Get()->Tags.AdditiveFromBaseBlend)
			, OverrideTag(FBuiltInComponentTypes::Get()->Tags.OverrideBlend)
		{
			EntitiesToDecompose.Append(Params.Query.Entities.GetData(), Params.Query.Entities.Num());
		}

		void ForEachAllocation(
				const FEntityAllocation* Allocation, TRead<FMovieSceneEntityID> EntityToDecomposeIDs, 
				TRead<FMovieSceneBlendChannelID> BlendChannels, TRead<double> ValueResultComponent, 
				TReadOptional<int32> OptionalBlendingOrder, TReadOptional<double> OptionalBaseValueComponent, TReadOptional<double> OptionalWeightComponent)
		{
			static const FMovieSceneBlenderSystemID BlenderSystemID = UMovieSceneBlenderSystem::GetBlenderSystemID<UMovieScenePiecewiseDoubleBlenderSystem>();

			const bool bOverride = Allocation->HasComponent(OverrideTag);
			const bool bAdditive = Allocation->HasComponent(AdditiveBlendTag);
			const bool bAdditiveFromBase = Allocation->HasComponent(AdditiveFromBaseBlendTag);
			
			const int32 Num = Allocation->Num();
			
			for (int32 EntityIndex = 0; EntityIndex < Num; ++EntityIndex)
			{
				const FMovieSceneBlendChannelID& BlendChannel(BlendChannels[EntityIndex]);
				ensureMsgf(BlendChannel.SystemID == BlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));

				if (BlendChannel.ChannelID != DecomposeBlendChannel)
				{
					continue;
				}

				// We've found a contributor for this blend channel
				const FMovieSceneEntityID EntityToDecompose = EntityToDecomposeIDs[EntityIndex];
				const double              ValueResult       = ValueResultComponent[EntityIndex];
				const double              BaseValue         = OptionalBaseValueComponent ? OptionalBaseValueComponent[EntityIndex] : 0.f;
				const float               Weight            = OptionalWeightComponent ? OptionalWeightComponent[EntityIndex] : 1.f;
				const int32				  BlendingOrder		= OptionalBlendingOrder ? OptionalBlendingOrder[EntityIndex] : INDEX_NONE;
				
				if (BlendingOrder != INDEX_NONE && (bAdditive || bOverride))
				{
					Result->Value.AllDecomposedOverrides.Add(FWeightedValue{ ValueResult, Weight, 0.0, BlendingOrder, bAdditive });
				}
				if (EntitiesToDecompose.Contains(EntityToDecompose))
				{
					if (BlendingOrder != INDEX_NONE && (bAdditive || bOverride))
					{
						Result->Value.DecomposedOverrides.Add(MakeTuple(EntityToDecompose, FWeightedValue{ ValueResult, Weight, 0.0, BlendingOrder, bAdditive }));
					}
					else if (bAdditive)
					{
						Result->Value.DecomposedAdditives.Add(MakeTuple(EntityToDecompose, FWeightedValue{ ValueResult, Weight }));
					}
					else if (bAdditiveFromBase)
					{
						Result->Value.DecomposedAdditives.Add(MakeTuple(EntityToDecompose, FWeightedValue{ ValueResult, Weight, BaseValue }));
					}
					else
					{
						Result->Value.DecomposedAbsolutes.Add(MakeTuple(EntityToDecompose, FWeightedValue{ ValueResult, Weight }));
					}
				}
				else if (bAdditive)
				{
					Result->Value.Result.Additive += ValueResult * Weight;
				}
				else if (bAdditiveFromBase)
				{
					Result->Value.Result.Additive += (ValueResult - BaseValue) * Weight;
				}
				else
				{
					Result->Value.Result.Absolute.Total += ValueResult * Weight;
					Result->Value.Result.Absolute.TotalWeight += Weight;
				}
			}
		}
	};

	// Do not use weights when ExternalBlending is present
	FComponentTypeIDFilter WeightFilter(BuiltInComponents->Tags.ExternalBlending, false);

	if (Params.Query.bConvertFromSourceEntityIDs)
	{
		return FEntityTaskBuilder()
			.Read(BuiltInComponents->ParentEntity)
			.Read(BuiltInComponents->BlendChannelInput)
			.Read(ResultComponentType)
			.ReadOptional(BuiltInComponents->BlendingOrder)
			.ReadOptional(BaseValueComponentType)
			.ReadOptional(BuiltInComponents->WeightAndEasingResult, WeightFilter)
			.FilterAll({ Params.PropertyTag, GetBlenderTypeTag() })
			.Dispatch_PerAllocation<FChannelResultTask>(&Linker->EntityManager, FSystemTaskPrerequisites(), nullptr, Params, Output);
	}
	else
	{
		return FEntityTaskBuilder()
			.ReadEntityIDs()
			.Read(BuiltInComponents->BlendChannelInput)
			.Read(ResultComponentType)
			.ReadOptional(BuiltInComponents->BlendingOrder)
			.ReadOptional(BaseValueComponentType)
			.ReadOptional(BuiltInComponents->WeightAndEasingResult, WeightFilter)
			.FilterAll({ Params.PropertyTag, GetBlenderTypeTag() })
			.Dispatch_PerAllocation<FChannelResultTask>(&Linker->EntityManager, FSystemTaskPrerequisites(), nullptr, Params, Output);
	}
}


