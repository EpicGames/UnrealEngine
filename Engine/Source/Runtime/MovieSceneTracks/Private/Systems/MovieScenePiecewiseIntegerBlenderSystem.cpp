// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieScenePiecewiseIntegerBlenderSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/IntegerChannelEvaluatorSystem.h"
#include "Systems/WeightAndEasingEvaluatorSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePiecewiseIntegerBlenderSystem)

//#include "Algo/Find.h"
//#include "Algo/AnyOf.h"
//#include "Algo/Accumulate.h"


namespace UE
{
namespace MovieScene
{

/** Task for accumulating all weighted blend inputs into arrays based on BlendID. Will be run for Absolute, Additive and Relative blend modes*/
struct FIntegerAccumulationTask
{
	FIntegerAccumulationTask(TArray<FIntegerBlendResult>* InAccumulationBuffer)
		: AccumulationBuffer(*InAccumulationBuffer)
	{}

	/** Task entry point - iterates the allocation's headers and accumulates int64 results for any required components */
	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<FMovieSceneBlendChannelID> BlendIDs, TRead<int64> Integers, TReadOptional<double> EasingAndWeights) const
	{
		static const FMovieSceneBlenderSystemID IntegerBlenderSystemID = UMovieSceneBlenderSystem::GetBlenderSystemID<UMovieScenePiecewiseIntegerBlenderSystem>();

		const int32 Num = Allocation->Num();

		if (EasingAndWeights)
		{
			// We have some easing/weight factors to multiply values with.
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneBlendChannelID& BlendID(BlendIDs[Index]);
				ensureMsgf(BlendID.SystemID == IntegerBlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));

				FIntegerBlendResult& Result = AccumulationBuffer[BlendID.ChannelID];

				const double Weight = EasingAndWeights[Index];
				Result.Total  += (int64)(Integers[Index] * Weight);
				Result.Weight += Weight;
			}
		}
		else
		{
			// Faster path for when there's no weight to multiply values with.
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneBlendChannelID& BlendID(BlendIDs[Index]);
				ensureMsgf(BlendID.SystemID == IntegerBlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));

				FIntegerBlendResult& Result = AccumulationBuffer[BlendID.ChannelID];

				Result.Total  += Integers[Index];
				Result.Weight += 1.f;
			}
		}
	}

	TArray<FIntegerBlendResult>& AccumulationBuffer;
};

/** Same as the task above, but also reads a "base value" that is subtracted from all values.
 *
 *  Only used by entities with the "additive from base" blend type.
 */
struct FIntegerAdditiveFromBaseBlendTask
{
	FIntegerAdditiveFromBaseBlendTask(TArray<FIntegerBlendResult>* InAccumulationBuffer)
		: AccumulationBuffer(*InAccumulationBuffer)
	{}

	void PreTask()
	{
		if (AccumulationBuffer.Num() > 0)
		{
			FMemory::Memzero(AccumulationBuffer.GetData(), sizeof(FIntegerBlendResult) * AccumulationBuffer.Num());
		}
	}

	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<FMovieSceneBlendChannelID> BlendIDs, TRead<int64> Integers, TRead<int64> BaseValues, TReadOptional<double> EasingAndWeights) const
	{
		static const FMovieSceneBlenderSystemID IntegerBlenderSystemID = UMovieSceneBlenderSystem::GetBlenderSystemID<UMovieScenePiecewiseIntegerBlenderSystem>();

		const int32 Num = Allocation->Num();

		if (EasingAndWeights)
		{
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneBlendChannelID& BlendID(BlendIDs[Index]);
				ensureMsgf(BlendID.SystemID == IntegerBlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));

				FIntegerBlendResult& Result = AccumulationBuffer[BlendID.ChannelID];

				const double Weight = EasingAndWeights[Index];
				Result.Total  += (int64)((Integers[Index] - BaseValues[Index]) * Weight);
				Result.Weight += Weight;
			}
		}
		else
		{
			// Faster path for when there's no weight to multiply values with.
			for (int32 Index = 0; Index < Num; ++Index)
			{
				const FMovieSceneBlendChannelID& BlendID(BlendIDs[Index]);
				ensureMsgf(BlendID.SystemID == IntegerBlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));

				FIntegerBlendResult& Result = AccumulationBuffer[BlendID.ChannelID];

				Result.Total  += (Integers[Index] - BaseValues[Index]);
				Result.Weight += 1.f;
			}
		}
	}

	TArray<FIntegerBlendResult>& AccumulationBuffer;
};

/** Task that combines all accumulated blends for any tracked property type that has blend inputs/outputs */
struct FIntegerCombineBlends
{
	explicit FIntegerCombineBlends(const FIntegerAccumulationBuffers* InAccumulationBuffers)
		: AccumulationBuffers(*InAccumulationBuffers)
	{}

	void ForEachAllocation(FEntityAllocation* Allocation, TRead<FMovieSceneBlendChannelID> BlendIDs, TReadOptional<int64> InitialValues, TWrite<int64> IntegerResults) const
	{
		static const FMovieSceneBlenderSystemID IntegerBlenderSystemID = UMovieSceneBlenderSystem::GetBlenderSystemID<UMovieScenePiecewiseIntegerBlenderSystem>();

		if (InitialValues)
		{
			for (int32 Index = 0; Index < Allocation->Num(); ++Index)
			{
				ensureMsgf(BlendIDs[Index].SystemID == IntegerBlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));
				BlendResultsWithInitial(BlendIDs[Index].ChannelID, InitialValues[Index], IntegerResults[Index]);
			}
		}
		else
		{
			for (int32 Index = 0; Index < Allocation->Num(); ++Index)
			{
				ensureMsgf(BlendIDs[Index].SystemID == IntegerBlenderSystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));
				BlendResults(BlendIDs[Index].ChannelID, IntegerResults[Index]);
			}
		}
	}

	void BlendResultsWithInitial(uint16 BlendID, const int64 InitialValue, int64& OutFinalBlendResult) const
	{
		FIntegerBlendResult AbsoluteResult = AccumulationBuffers.Absolute.Num() > 0 ? AccumulationBuffers.Absolute[BlendID] : FIntegerBlendResult();
		FIntegerBlendResult RelativeResult = AccumulationBuffers.Relative.Num() > 0 ? AccumulationBuffers.Relative[BlendID] : FIntegerBlendResult();
		FIntegerBlendResult AdditiveResult = AccumulationBuffers.Additive.Num() > 0 ? AccumulationBuffers.Additive[BlendID] : FIntegerBlendResult();
		FIntegerBlendResult AdditiveFromBaseResult = AccumulationBuffers.AdditiveFromBase.Num() > 0 ? AccumulationBuffers.AdditiveFromBase[BlendID] : FIntegerBlendResult();

		if (RelativeResult.Weight != 0)
		{
			RelativeResult.Total += InitialValue * RelativeResult.Weight;
		}

		FIntegerBlendResult TotalAdditiveResult = { AdditiveResult.Total + AdditiveFromBaseResult.Total, AdditiveResult.Weight + AdditiveFromBaseResult.Weight };

		const double TotalWeight = AbsoluteResult.Weight + RelativeResult.Weight;
		if (TotalWeight != 0)
		{
			// If the absolute value has some partial weighting (for ease-in/out for instance), we ramp it from/to the initial value. This means
			// that the "initial value" adds a contribution to the entire blending process, so we add its weight to the total that we
			// normalize absolutes and relatives with.
			//
			// Note that "partial weighting" means strictly between 0 and 100%. At 100% and above, we don't need to do this thing with the initial
			// value. At 0%, we have no absolute value (only a relative value) and we therefore don't want to include the initial value either.
			const bool bInitialValueContributes = (0.f < AbsoluteResult.Weight && AbsoluteResult.Weight < 1.f);
			const int64 AbsoluteBlendedValue = bInitialValueContributes ?
				((int64)(InitialValue * (1.f - AbsoluteResult.Weight)) + AbsoluteResult.Total) :
				AbsoluteResult.Total;
			const double FinalTotalWeight = bInitialValueContributes ? (TotalWeight + (1.f - AbsoluteResult.Weight)) : TotalWeight;

			const int64 Value = (int64)((AbsoluteBlendedValue + RelativeResult.Total) / FinalTotalWeight) + TotalAdditiveResult.Total;
			OutFinalBlendResult = Value;
		}
		else if (TotalAdditiveResult.Weight != 0)
		{
			OutFinalBlendResult = TotalAdditiveResult.Total + InitialValue;
		}
		else
		{
			OutFinalBlendResult = InitialValue;
		}
	}

	void BlendResults(uint16 BlendID, int64& OutFinalBlendResult) const
	{
		FIntegerBlendResult AbsoluteResult = AccumulationBuffers.Absolute.Num() > 0 ? AccumulationBuffers.Absolute[BlendID] : FIntegerBlendResult();
		FIntegerBlendResult AdditiveResult = AccumulationBuffers.Additive.Num() > 0 ? AccumulationBuffers.Additive[BlendID] : FIntegerBlendResult();
		FIntegerBlendResult AdditiveFromBaseResult = AccumulationBuffers.AdditiveFromBase.Num() > 0 ? AccumulationBuffers.AdditiveFromBase[BlendID] : FIntegerBlendResult();

#if DO_GUARD_SLOW
		ensureMsgf(AbsoluteResult.Weight != 0.f, TEXT("Default blend combine being used for an entity that has no absolute weight. This should have an initial value and should be handled by each system, and excluded by default with UMovieSceneBlenderSystem::FinalCombineExclusionFilter ."));
#endif

		const double TotalWeight = AbsoluteResult.Weight;
		if (TotalWeight != 0)
		{
			const int64 Value = (int64)(AbsoluteResult.Total / AbsoluteResult.Weight) + AdditiveResult.Total + AdditiveFromBaseResult.Total;
			OutFinalBlendResult = Value;
		}
	}

private:

	const FIntegerAccumulationBuffers& AccumulationBuffers;
};



bool FIntegerAccumulationBuffers::IsEmpty() const
{
	return Absolute.Num() == 0 && Relative.Num() == 0 && Additive.Num() == 0 && AdditiveFromBase.Num() == 0;
}

void FIntegerAccumulationBuffers::Reset()
{
	Absolute.Reset();
	Relative.Reset();
	Additive.Reset();
	AdditiveFromBase.Reset();
}

} // namespace MovieScene
} // namespace UE

UMovieScenePiecewiseIntegerBlenderSystem::UMovieScenePiecewiseIntegerBlenderSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UIntegerChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UWeightAndEasingEvaluatorSystem::StaticClass(), GetClass());
	}
}

void UMovieScenePiecewiseIntegerBlenderSystem::OnLink()
{
}

void UMovieScenePiecewiseIntegerBlenderSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	CompactBlendChannels();

	// We allocate space for every blend even if there are gaps so we can do a straight index into each array
	const int32 MaximumNumBlends = AllocatedBlendChannels.Num();
	if (MaximumNumBlends == 0)
	{
		return;
	}

	ReinitializeAccumulationBuffers();
	if (AccumulationBuffers.IsEmpty())
	{
		return;
	}

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FTaskID ResetWeightsTask = TaskScheduler->AddMemberFunctionTask(
		FTaskParams(TEXT("Reset Integer Blender Weights")),
		this, &UMovieScenePiecewiseIntegerBlenderSystem::ZeroAccumulationBuffers);

	FTaskID SyncTask = TaskScheduler->AddNullTask();

	if (AccumulationBuffers.Absolute.Num() != 0)
	{
		FTaskID AbsoluteTask = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.Read(BuiltInComponents->IntegerResult)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult)
		.FilterAll({ BuiltInComponents->Tags.AbsoluteBlend, GetBlenderTypeTag() })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Schedule_PerAllocation<FIntegerAccumulationTask>(&Linker->EntityManager, TaskScheduler, &AccumulationBuffers.Absolute);

		TaskScheduler->AddPrerequisite(ResetWeightsTask, AbsoluteTask);
		TaskScheduler->AddPrerequisite(AbsoluteTask, SyncTask);
	}

	if (AccumulationBuffers.Relative.Num() != 0)
	{
		FTaskID RelativeTask = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.Read(BuiltInComponents->IntegerResult)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult)
		.FilterAll({ BuiltInComponents->Tags.RelativeBlend, GetBlenderTypeTag() })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Schedule_PerAllocation<FIntegerAccumulationTask>(&Linker->EntityManager, TaskScheduler, &AccumulationBuffers.Relative);

		TaskScheduler->AddPrerequisite(ResetWeightsTask, RelativeTask);
		TaskScheduler->AddPrerequisite(RelativeTask, SyncTask);
	}

	if (AccumulationBuffers.Additive.Num() != 0)
	{
		FTaskID AdditiveTask = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.Read(BuiltInComponents->IntegerResult)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult)
		.FilterAll({ BuiltInComponents->Tags.AdditiveBlend, GetBlenderTypeTag() })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Schedule_PerAllocation<FIntegerAccumulationTask>(&Linker->EntityManager, TaskScheduler, &AccumulationBuffers.Additive);

		TaskScheduler->AddPrerequisite(ResetWeightsTask, AdditiveTask);
		TaskScheduler->AddPrerequisite(AdditiveTask, SyncTask);
	}

	if (AccumulationBuffers.AdditiveFromBase.Num() != 0)
	{
		FTaskID AdditiveFromBaseTask = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.Read(BuiltInComponents->IntegerResult)
		.Read(BuiltInComponents->BaseInteger)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult)
		.FilterAll({ BuiltInComponents->Tags.AdditiveFromBaseBlend, GetBlenderTypeTag() })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Schedule_PerAllocation<FIntegerAdditiveFromBaseBlendTask>(&Linker->EntityManager, TaskScheduler, &AccumulationBuffers.AdditiveFromBase);

		TaskScheduler->AddPrerequisite(ResetWeightsTask, AdditiveFromBaseTask);
		TaskScheduler->AddPrerequisite(AdditiveFromBaseTask, SyncTask);
	}

	// Root task that performs the actual blends
	FTaskID CombineTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->BlendChannelOutput)
	.ReadOptional(TracksComponents->Integer.InitialValue)
	.Write(BuiltInComponents->IntegerResult)
	.FilterAll({ GetBlenderTypeTag() })
	.Fork_PerAllocation<FIntegerCombineBlends>(&Linker->EntityManager, TaskScheduler, &AccumulationBuffers);

	TaskScheduler->AddPrerequisite(SyncTask, CombineTask);
}

void UMovieScenePiecewiseIntegerBlenderSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	CompactBlendChannels();

	// We allocate space for every blend even if there are gaps so we can do a straight index into each array
	const int32 MaximumNumBlends = AllocatedBlendChannels.Num();
	if (MaximumNumBlends == 0)
	{
		return;
	}

	// Update cached channel data if necessary
	if (ChannelRelevancyCache.Update(Linker->EntityManager) == ECachedEntityManagerState::Stale)
	{
		ReinitializeAccumulationBuffers();
	}
	else
	{
		ZeroAccumulationBuffers();
	}

	if (AccumulationBuffers.IsEmpty())
	{
		return;
	}

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FSystemTaskPrerequisites Prereqs;
	if (AccumulationBuffers.Absolute.Num() != 0)
	{
		FGraphEventRef Task = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.Read(BuiltInComponents->IntegerResult)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult)
		.FilterAll({ BuiltInComponents->Tags.AbsoluteBlend, GetBlenderTypeTag() })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Dispatch_PerAllocation<FIntegerAccumulationTask>(&Linker->EntityManager, InPrerequisites, nullptr, &AccumulationBuffers.Absolute);

		if (Task)
		{
			Prereqs.AddRootTask(Task);
		}
	}

	if (AccumulationBuffers.Relative.Num() != 0)
	{
		FGraphEventRef Task = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.Read(BuiltInComponents->IntegerResult)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult)
		.FilterAll({ BuiltInComponents->Tags.RelativeBlend, GetBlenderTypeTag() })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Dispatch_PerAllocation<FIntegerAccumulationTask>(&Linker->EntityManager, InPrerequisites, nullptr, &AccumulationBuffers.Relative);

		if (Task)
		{
			Prereqs.AddRootTask(Task);
		}
	}

	if (AccumulationBuffers.Additive.Num() != 0)
	{
		FGraphEventRef Task = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.Read(BuiltInComponents->IntegerResult)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult)
		.FilterAll({ BuiltInComponents->Tags.AdditiveBlend, GetBlenderTypeTag() })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Dispatch_PerAllocation<FIntegerAccumulationTask>(&Linker->EntityManager, InPrerequisites, nullptr, &AccumulationBuffers.Additive);

		if (Task)
		{
			Prereqs.AddRootTask(Task);
		}
	}

	if (AccumulationBuffers.AdditiveFromBase.Num() != 0)
	{
		FGraphEventRef Task = FEntityTaskBuilder()
		.Read(BuiltInComponents->BlendChannelInput)
		.Read(BuiltInComponents->IntegerResult)
		.Read(BuiltInComponents->BaseInteger)
		.ReadOptional(BuiltInComponents->WeightAndEasingResult)
		.FilterAll({ BuiltInComponents->Tags.AdditiveFromBaseBlend, GetBlenderTypeTag() })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.Dispatch_PerAllocation<FIntegerAdditiveFromBaseBlendTask>(&Linker->EntityManager, InPrerequisites, nullptr, &AccumulationBuffers.AdditiveFromBase);

		if (Task)
		{
			Prereqs.AddRootTask(Task);
		}
	}

	// Root task that performs the actual blends
	FEntityTaskBuilder()
	.Read(BuiltInComponents->BlendChannelOutput)
	.ReadOptional(TracksComponents->Integer.InitialValue)
	.Write(BuiltInComponents->IntegerResult)
	.FilterAll({ GetBlenderTypeTag() })
	.Dispatch_PerAllocation<FIntegerCombineBlends>(&Linker->EntityManager, Prereqs, &Subsequents, &AccumulationBuffers);
}

void UMovieScenePiecewiseIntegerBlenderSystem::ReinitializeAccumulationBuffers()
{
	using namespace UE::MovieScene;

	const int32 MaximumNumBlends = AllocatedBlendChannels.Num();

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	AccumulationBuffers.Reset();

	// Find if we have any integer results to blend using any supported blend types.
	const TComponentTypeID<int64> Component = BuiltInComponents->IntegerResult;

	const bool bHasAbsolutes         = Linker->EntityManager.Contains(FEntityComponentFilter().All({ GetBlenderTypeTag(), Component, BuiltInComponents->BlendChannelInput, BuiltInComponents->Tags.AbsoluteBlend }));
	const bool bHasRelatives         = Linker->EntityManager.Contains(FEntityComponentFilter().All({ GetBlenderTypeTag(), Component, BuiltInComponents->BlendChannelInput, BuiltInComponents->Tags.RelativeBlend }));
	const bool bHasAdditives         = Linker->EntityManager.Contains(FEntityComponentFilter().All({ GetBlenderTypeTag(), Component, BuiltInComponents->BlendChannelInput, BuiltInComponents->Tags.AdditiveBlend }));
	const bool bHasAdditivesFromBase = Linker->EntityManager.Contains(FEntityComponentFilter().All({ GetBlenderTypeTag(), Component, BuiltInComponents->BlendChannelInput, BuiltInComponents->Tags.AdditiveFromBaseBlend }));

	if (bHasAbsolutes)
	{
		AccumulationBuffers.Absolute.SetNum(MaximumNumBlends);
	}
	if (bHasRelatives)
	{
		AccumulationBuffers.Relative.SetNum(MaximumNumBlends);
	}
	if (bHasAdditives)
	{
		AccumulationBuffers.Additive.SetNum(MaximumNumBlends);
	}
	if (bHasAdditivesFromBase)
	{
		AccumulationBuffers.AdditiveFromBase.SetNum(MaximumNumBlends);
	}

	ZeroAccumulationBuffers();
}

void UMovieScenePiecewiseIntegerBlenderSystem::ZeroAccumulationBuffers()
{
	using namespace UE::MovieScene;

	if (AccumulationBuffers.Absolute.Num() > 0)
	{
		FMemory::Memzero(AccumulationBuffers.Absolute.GetData(), sizeof(FIntegerBlendResult) * AccumulationBuffers.Absolute.Num());
	}
	if (AccumulationBuffers.Relative.Num() > 0)
	{
		FMemory::Memzero(AccumulationBuffers.Relative.GetData(), sizeof(FIntegerBlendResult) * AccumulationBuffers.Relative.Num());
	}
	if (AccumulationBuffers.Additive.Num() > 0)
	{
		FMemory::Memzero(AccumulationBuffers.Additive.GetData(), sizeof(FIntegerBlendResult) * AccumulationBuffers.Additive.Num());
	}
	if (AccumulationBuffers.AdditiveFromBase.Num() > 0)
	{
		FMemory::Memzero(AccumulationBuffers.AdditiveFromBase.GetData(), sizeof(FIntegerBlendResult) * AccumulationBuffers.AdditiveFromBase.Num());
	}
}


