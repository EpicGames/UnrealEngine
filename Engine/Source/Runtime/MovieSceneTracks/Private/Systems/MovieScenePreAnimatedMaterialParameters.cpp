// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieScenePreAnimatedMaterialParameters.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "Materials/MaterialParameterCollectionInstance.h"
#include "Materials/MaterialInstanceDynamic.h"
#include "Materials/MaterialInterface.h"

namespace UE::MovieScene
{

TAutoRegisterPreAnimatedStorageID<FPreAnimatedScalarMaterialParameterStorage> FPreAnimatedScalarMaterialParameterStorage::StorageID;
TAutoRegisterPreAnimatedStorageID<FPreAnimatedVectorMaterialParameterStorage> FPreAnimatedVectorMaterialParameterStorage::StorageID;

FMaterialParameterKey::FMaterialParameterKey(const FObjectComponent& InBoundMaterial, const FName& InParameterName)
	: BoundMaterial(InBoundMaterial.GetObject())
	, ParameterInfo(InParameterName)
{}

FMaterialParameterKey::FMaterialParameterKey(const FObjectComponent& InBoundMaterial, const FMaterialParameterInfo& InParameterInfo)
	: BoundMaterial(InBoundMaterial.GetObject())
	, ParameterInfo(InParameterInfo)
{}

uint32 GetTypeHash(const FMaterialParameterKey& InKey)
{
	return GetTypeHash(InKey.BoundMaterial) ^ GetTypeHash(InKey.ParameterInfo);
}

bool operator==(const FMaterialParameterKey& A, const FMaterialParameterKey& B)
{
	return A.BoundMaterial == B.BoundMaterial && A.ParameterInfo == B.ParameterInfo;
}

void FMaterialParameterCollectionScalarTraits::ReplaceObject(FMaterialParameterKey& InOutKey, const FObjectKey& NewObject)
{
	InOutKey.BoundMaterial = NewObject.ResolveObjectPtr();
}

float FMaterialParameterCollectionScalarTraits::CachePreAnimatedValue(const FObjectComponent& InBoundMaterial, const FName& ParameterName)
{
	return CachePreAnimatedValue(InBoundMaterial, FMaterialParameterInfo(ParameterName));
}

float FMaterialParameterCollectionScalarTraits::CachePreAnimatedValue(const FObjectComponent& InBoundMaterial, const FMaterialParameterInfo& ParameterInfo)
{
	UObject* BoundMaterial = InBoundMaterial.GetObject();

	float ParameterValue = 0.f;
	if (UMaterialParameterCollectionInstance* MPCI = Cast<UMaterialParameterCollectionInstance>(BoundMaterial))
	{
		MPCI->GetScalarParameterValue(ParameterInfo.Name, ParameterValue);
	}
	else if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(BoundMaterial))
	{
		MaterialInterface->GetScalarParameterValue(ParameterInfo, ParameterValue);
	}
	return ParameterValue;
}

void FMaterialParameterCollectionScalarTraits::RestorePreAnimatedValue(const FMaterialParameterKey& InKey, float OldValue, const FRestoreStateParams& Params)
{
	UObject* BoundObject = InKey.BoundMaterial.ResolveObjectPtr();

	if (UMaterialParameterCollectionInstance* MPCI = Cast<UMaterialParameterCollectionInstance>(BoundObject))
	{
		MPCI->SetScalarParameterValue(InKey.ParameterInfo.Name, OldValue);
	}
	else if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(BoundObject))
	{
		MID->SetScalarParameterValueByInfo(InKey.ParameterInfo, OldValue);
	}
}

void FMaterialParameterCollectionVectorTraits::ReplaceObject(FMaterialParameterKey& InOutKey, const FObjectKey& NewObject)
{
	InOutKey.BoundMaterial = NewObject.ResolveObjectPtr();
}

FLinearColor FMaterialParameterCollectionVectorTraits::CachePreAnimatedValue(const FObjectComponent& InBoundMaterial, const FName& ParameterName)
{
	return CachePreAnimatedValue(InBoundMaterial, FMaterialParameterInfo(ParameterName));
}

FLinearColor FMaterialParameterCollectionVectorTraits::CachePreAnimatedValue(const FObjectComponent& InBoundMaterial, const FMaterialParameterInfo& ParameterInfo)
{
	UObject* BoundMaterial = InBoundMaterial.GetObject();

	FLinearColor ParameterValue = FLinearColor::White;
	if (UMaterialParameterCollectionInstance* MPCI = Cast<UMaterialParameterCollectionInstance>(BoundMaterial))
	{
		MPCI->GetVectorParameterValue(ParameterInfo.Name, ParameterValue);
	}
	else if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(BoundMaterial))
	{
		MaterialInterface->GetVectorParameterValue(ParameterInfo, ParameterValue);
	}
	return ParameterValue;
}

void FMaterialParameterCollectionVectorTraits::RestorePreAnimatedValue(const FMaterialParameterKey& InKey, const FLinearColor& OldValue, const FRestoreStateParams& Params)
{
	UObject* BoundObject = InKey.BoundMaterial.ResolveObjectPtr();

	if (UMaterialParameterCollectionInstance* MPCI = Cast<UMaterialParameterCollectionInstance>(BoundObject))
	{
		MPCI->SetVectorParameterValue(InKey.ParameterInfo.Name, OldValue);
	}
	else if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(BoundObject))
	{
		MID->SetVectorParameterValueByInfo(InKey.ParameterInfo, OldValue);
	}
}


} // namespace UE::MovieScene