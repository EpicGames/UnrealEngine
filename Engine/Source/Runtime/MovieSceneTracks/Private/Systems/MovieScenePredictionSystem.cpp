// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieScenePredictionSystem.h"
#include "Systems/MovieSceneEventSystems.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/FloatChannelEvaluatorSystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneInitialValueCache.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"

#include "Sections/MovieScene3DTransformSection.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTracksComponentTypes.h"
#include "MovieSceneSequence.h"
#include "MovieSceneSequencePlayer.h"
#include "MovieSceneSpawnableAnnotation.h"

#include "Compilation/MovieSceneCompiledDataManager.h"

#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Evaluation/MovieSceneEvaluationTree.h"

#include "Engine/World.h"
#include "GameFramework/Actor.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePredictionSystem)

DECLARE_CYCLE_STAT(TEXT("Prediction Initialization"), MovieSceneEval_PredictionIntialization, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Prediction Finalization"),   MovieSceneEval_PredictionFinalization,  STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Prediction Report Results"), MovieSceneEval_PredictionReportResults, STATGROUP_MovieSceneECS);

UMovieSceneAsyncAction_SequencePrediction* UMovieSceneAsyncAction_SequencePrediction::PredictWorldTransformAtTime(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, float TimeInSeconds)
{
	return MakePredictionImpl(Player, TargetComponent, TimeInSeconds, true);
}

UMovieSceneAsyncAction_SequencePrediction* UMovieSceneAsyncAction_SequencePrediction::PredictLocalTransformAtTime(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, float TimeInSeconds)
{
	return MakePredictionImpl(Player, TargetComponent, TimeInSeconds, false);
}

UMovieSceneAsyncAction_SequencePrediction* UMovieSceneAsyncAction_SequencePrediction::PredictWorldTransformAtFrame(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, FFrameTime FrameTime)
{
	if (!Player || !TargetComponent)
	{
		return nullptr;
	}

	UMovieSceneSequence* RootSequence = Player->GetSequence();
	if (!RootSequence || !RootSequence->GetMovieScene())
	{
		return nullptr;
	}

	FFrameTime TickResolutionTime = ConvertFrameTime(FrameTime, RootSequence->GetMovieScene()->GetDisplayRate(), RootSequence->GetMovieScene()->GetTickResolution());
	return MakePredictionImpl(Player, TargetComponent, TickResolutionTime, true);
}

UMovieSceneAsyncAction_SequencePrediction* UMovieSceneAsyncAction_SequencePrediction::PredictLocalTransformAtFrame(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, FFrameTime FrameTime)
{
	if (!Player || !TargetComponent)
	{
		return nullptr;
	}

	UMovieSceneSequence* RootSequence = Player->GetSequence();
	if (!RootSequence || !RootSequence->GetMovieScene())
	{
		return nullptr;
	}

	FFrameTime TickResolutionTime = ConvertFrameTime(FrameTime, RootSequence->GetMovieScene()->GetDisplayRate(), RootSequence->GetMovieScene()->GetTickResolution());
	return MakePredictionImpl(Player, TargetComponent, TickResolutionTime, false);
}

UMovieSceneAsyncAction_SequencePrediction* UMovieSceneAsyncAction_SequencePrediction::MakePredictionImpl(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, float TimeInSeconds, bool bInWorldSpace)
{
	if (!Player || !TargetComponent)
	{
		return nullptr;
	}

	UMovieSceneSequence* RootSequence = Player->GetSequence();
	if (!RootSequence || !RootSequence->GetMovieScene())
	{
		return nullptr;
	}

	FFrameTime TickResolutionTime = TimeInSeconds * RootSequence->GetMovieScene()->GetTickResolution();
	return MakePredictionImpl(Player, TargetComponent, TickResolutionTime, bInWorldSpace);
}

UMovieSceneAsyncAction_SequencePrediction* UMovieSceneAsyncAction_SequencePrediction::MakePredictionImpl(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, FFrameTime TickResolutionTime, bool bInWorldSpace)
{
	check(Player && TargetComponent);

	UMovieSceneEntitySystemLinker* Linker           = static_cast<IMovieScenePlayer*>(Player)->GetEvaluationTemplate().GetEntitySystemLinker();
	UMovieScenePredictionSystem*   PredictionSystem = Linker->LinkSystem<UMovieScenePredictionSystem>();

	UMovieSceneAsyncAction_SequencePrediction* NewPrediction = NewObject<UMovieSceneAsyncAction_SequencePrediction>(PredictionSystem);

	NewPrediction->SequencePlayer      = Player;
	NewPrediction->RootPredictedTime   = TickResolutionTime;
	NewPrediction->InterrogationIndex  = PredictionSystem->MakeNewInterrogation(NewPrediction->RootPredictedTime);
	NewPrediction->SceneComponent      = TargetComponent;
	NewPrediction->bWorldSpace         = bInWorldSpace;

	PredictionSystem->AddPendingPrediction(NewPrediction);
	return NewPrediction;
}

void UMovieSceneAsyncAction_SequencePrediction::ReportResult(UE::MovieScene::FInterrogationChannels* Channels, const TSparseArray<TArray<FTransform>>& AllResults)
{
	using namespace UE::MovieScene;

	if (InterrogationIndex != INDEX_NONE)
	{
		FInterrogationChannel Channel = Channels->FindChannel(SceneComponent);
		if (Channel && AllResults.IsValidIndex(Channel.AsIndex()))
		{
			if (AllResults[Channel.AsIndex()].IsValidIndex(InterrogationIndex))
			{
				FTransform Transform = AllResults[Channel.AsIndex()][InterrogationIndex];
				Result.Broadcast(Transform);
				return;
			}
		}
	}

	Failure.Broadcast();
}

void UMovieSceneAsyncAction_SequencePrediction::ReportResult(UE::MovieScene::FInterrogationChannels* Channels, const TSparseArray<TArray<UE::MovieScene::FIntermediate3DTransform>>& AllResults)
{
	using namespace UE::MovieScene;

	if (InterrogationIndex != INDEX_NONE)
	{
		FInterrogationChannel Channel = Channels->FindChannel(SceneComponent);
		if (Channel && AllResults.IsValidIndex(Channel.AsIndex()))
		{
			if (AllResults[Channel.AsIndex()].IsValidIndex(InterrogationIndex))
			{
				FIntermediate3DTransform Transform = AllResults[Channel.AsIndex()][InterrogationIndex];
				Result.Broadcast(FTransform(Transform.GetRotation(), Transform.GetTranslation(), Transform.GetScale()));
				return;
			}
		}
	}

	Failure.Broadcast();
}

void UMovieSceneAsyncAction_SequencePrediction::Reset(UMovieSceneEntitySystemLinker* Linker)
{
	using namespace UE::MovieScene;

	FComponentTypeID NeedsUnlink = FBuiltInComponentTypes::Get()->Tags.NeedsUnlink;

	for (FMovieSceneEntityID EntityID : ImportedEntities)
	{
		Linker->EntityManager.AddComponent(EntityID, NeedsUnlink, EEntityRecursion::Full);
	}
	ImportedEntities.Empty();
}

void UMovieSceneAsyncAction_SequencePrediction::ImportEntities(UE::MovieScene::FInterrogationChannels* Channels)
{
	if (SceneComponent)
	{
		ImportTransformHierarchy(Channels, SceneComponent);
	}
}

int32 UMovieSceneAsyncAction_SequencePrediction::ImportTransformEntities(UMovieSceneEntitySystemLinker* Linker, const UE::MovieScene::FEntityImportSequenceParams& ImportParams, const FGuid& ObjectGuid, FFrameTime PredictedTime, const FMovieSceneEntityComponentField* ComponentField, const UE::MovieScene::FInterrogationKey& InterrogationKey)
{
	using namespace UE::MovieScene;

	int32 NumEntities = 0;

	auto QueryCallback = [this, Linker, ImportParams, ObjectGuid, ComponentField, InterrogationKey, &NumEntities](const FMovieSceneEvaluationFieldEntityQuery& InQuery)
	{
		UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(InQuery.Entity.Key.EntityOwner.Get());
		if (TransformSection == nullptr)
		{
			return true;
		}

		if (!MovieSceneHelpers::IsSectionKeyable(TransformSection))
		{
			return true;
		}

		const FMovieSceneEvaluationFieldSharedEntityMetaData* SharedMetaData = ComponentField->FindSharedMetaData(InQuery);
		if (!SharedMetaData || SharedMetaData->ObjectBindingID != ObjectGuid)
		{
			return true;
		}

		UObject* EntityOwner = InQuery.Entity.Key.EntityOwner.Get();
		IMovieSceneEntityProvider* Provider = Cast<IMovieSceneEntityProvider>(EntityOwner);
		if (!Provider)
		{
			return true;
		}

		FEntityImportParams Params;
		Params.Sequence = ImportParams;
		Params.EntityID = InQuery.Entity.Key.EntityID;
		Params.EntityMetaData = ComponentField->FindMetaData(InQuery);
		Params.SharedMetaData = SharedMetaData;
		Params.InterrogationKey = InterrogationKey;

		FImportedEntity ImportedEntity;
		Provider->InterrogateEntity(Linker, Params, &ImportedEntity);

		if (!ImportedEntity.IsEmpty())
		{
			TransformSection->BuildDefaultComponents(Linker, Params, &ImportedEntity);

			const FMovieSceneEntityID NewEntityID = ImportedEntity.Manufacture(Params, &Linker->EntityManager);
			this->ImportedEntities.Add(NewEntityID);

			++NumEntities;
		}

		return true;
	};

	// Query any transform track entities at the specified time that relate to the object binding ID
	TRange<FFrameNumber> Unused;
	ComponentField->QueryPersistentEntities(PredictedTime.FrameNumber, QueryCallback, Unused);

	return NumEntities;
}


int32 UMovieSceneAsyncAction_SequencePrediction::ImportTransformEntities(UObject* PredicateObject, UObject* ObjectContext, const UE::MovieScene::FInterrogationKey& InterrogationKey)
{
	using namespace UE::MovieScene;

	FEntityImportSequenceParams ImportParams;

	int32 TotalNumEntities = 0;
	const FMovieSceneRootEvaluationTemplateInstance& RootTemplate = static_cast<IMovieScenePlayer*>(SequencePlayer)->GetEvaluationTemplate();

	TOptional<FMovieSceneSpawnableAnnotation> SpawnableAnnotation;

	if (AActor* Actor = Cast<AActor>(PredicateObject))
	{
		SpawnableAnnotation = FMovieSceneSpawnableAnnotation::Find(Actor);
		if (SpawnableAnnotation)
		{
			FMovieSceneSpawnRegister& SpawnRegister = static_cast<IMovieScenePlayer*>(SequencePlayer)->GetSpawnRegister();

			// Verify that the spawnable came from this sequence
			if (SpawnRegister.FindSpawnedObject(SpawnableAnnotation->ObjectBindingID, SpawnableAnnotation->SequenceID).Get() != Actor)
			{
				SpawnableAnnotation.Reset();
			}
		}
	}

	UMovieSceneEntitySystemLinker*  Linker              = RootTemplate.GetEntitySystemLinker();
	UMovieSceneCompiledDataManager* CompiledDataManager = RootTemplate.GetCompiledDataManager();
	FMovieSceneCompiledDataID       CompiledDataID      = RootTemplate.GetCompiledDataID();

	const FMovieSceneCompiledDataEntry& CompiledEntry = CompiledDataManager->GetEntryRef(CompiledDataID);

	if (UMovieSceneSequence* Sequence = CompiledEntry.GetSequence())
	{
		const bool bIsSpawnable = SpawnableAnnotation && SpawnableAnnotation->SequenceID == MovieSceneSequenceID::Root;
		FGuid FoundID = bIsSpawnable ? SpawnableAnnotation->ObjectBindingID : Sequence->FindBindingFromObject(PredicateObject, SequencePlayer->GetSharedPlaybackState());

		if (FoundID.IsValid())
		{
			const FMovieSceneEntityComponentField* ComponentField = CompiledDataManager->FindEntityComponentField(CompiledDataID);
			if (ComponentField)
			{
				TotalNumEntities += ImportTransformEntities(Linker, ImportParams, FoundID, RootPredictedTime, ComponentField, InterrogationKey);
			}
		}
	}

	// Also check all the sub-sequences at the predicted time
	const FMovieSceneSequenceHierarchy* Hierarchy = CompiledDataManager->FindHierarchy(CompiledDataID);
	if (Hierarchy)
	{
		FMovieSceneEvaluationTreeRangeIterator NodeIt = Hierarchy->GetTree().IterateFromTime(RootPredictedTime.FrameNumber);
		TMovieSceneEvaluationTreeDataIterator<FMovieSceneSubSequenceTreeEntry> SubSequenceIt = Hierarchy->GetTree().GetAllData(NodeIt.Node());
		for ( ; SubSequenceIt; ++SubSequenceIt)
		{
			if (SubSequenceIt->Flags == ESectionEvaluationFlags::None)
			{
				const FMovieSceneSubSequenceData* SubData = Hierarchy->FindSubData(SubSequenceIt->SequenceID);
				if (SubData)
				{
					if (UMovieSceneSequence* SubSequence = SubData->GetSequence())
					{
						FMovieSceneCompiledDataID SubDataID = CompiledDataManager->FindDataID(SubSequence);
						if (SubDataID.IsValid())
						{
							const bool bIsSpawnable = SpawnableAnnotation && SpawnableAnnotation->SequenceID == SubSequenceIt->SequenceID;
							FGuid FoundID = bIsSpawnable ? SpawnableAnnotation->ObjectBindingID : SubSequence->FindBindingFromObject(PredicateObject, SequencePlayer->GetSharedPlaybackState());

							if (FoundID.IsValid())
							{
								const FMovieSceneEntityComponentField* ComponentField = CompiledDataManager->FindEntityComponentField(SubDataID);
								if (ComponentField)
								{
									ImportParams.HierarchicalBias = SubData->HierarchicalBias;

									FFrameTime SubPredictedTime = RootPredictedTime * SubData->RootToSequenceTransform;
									TotalNumEntities += ImportTransformEntities(Linker, ImportParams, FoundID, SubPredictedTime, ComponentField, InterrogationKey);
								}
							}
						}
					}
				}
			}
		}
	}

	return TotalNumEntities;
}

void UMovieSceneAsyncAction_SequencePrediction::ImportLocalTransforms(UE::MovieScene::FInterrogationChannels* Channels, USceneComponent* InSceneComponent)
{
	using namespace UE::MovieScene;

	check(InSceneComponent);

	FInterrogationChannel ParentChannel;
	if (USceneComponent* AttachParent = InSceneComponent->GetAttachParent())
	{
		ParentChannel = Channels->FindChannel(AttachParent);
	}

	FInterrogationChannel Channel = Channels->FindChannel(InSceneComponent);
	if (!Channel.IsValid())
	{
		Channel = Channels->AllocateChannel(InSceneComponent, ParentChannel, FMovieScenePropertyBinding("Transform", TEXT("Transform")));
	}

	if (!Channel.IsValid())
	{
		return;
	}

	FInterrogationKey Key{ Channel, InterrogationIndex };

	// Query for any transform tracks relating to the scene component at the predicted time
	int32 NumEntities = ImportTransformEntities(InSceneComponent, InSceneComponent->GetOwner(), Key);

	// Also blend in any transforms that exist for this scene component's actor as well (if it is the root)
	AActor* Owner = InSceneComponent->GetOwner();
	if (Owner && InSceneComponent == Owner->GetRootComponent())
	{
		NumEntities += ImportTransformEntities(Owner, Owner->GetWorld(), Key);
	}

	if (NumEntities > 0)
	{
		Channels->ActivateChannel(Channel);
	}
}

void UMovieSceneAsyncAction_SequencePrediction::ImportTransformHierarchy(UE::MovieScene::FInterrogationChannels* Channels, USceneComponent* InSceneComponent)
{
	check(InSceneComponent);

	// Ensure the parent is imported first
	if (USceneComponent* AttachParent = InSceneComponent->GetAttachParent())
	{
		ImportTransformHierarchy(Channels, AttachParent);
	}

	ImportLocalTransforms(Channels, InSceneComponent);
}

UMovieScenePredictionSystem::UMovieScenePredictionSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Spawn | ESystemPhase::Instantiation | ESystemPhase::Finalization;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UFloatChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieScenePostEvalEventSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneInterrogatedPropertyInstantiatorSystem::StaticClass(), GetClass());

		DefineComponentConsumer(GetClass(), UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->ComponentTransform.PropertyTag);
	}
}

bool UMovieScenePredictionSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	// This system is always relevant as long as it has pending or processing predictions in-flight
	return PendingPredictions.Num() + ProcessingPredictions.Num() != 0;
}

void UMovieScenePredictionSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	ESystemPhase CurrentPhase = Runner->GetCurrentPhase();

	// ------------------------------------------------------------------
	// Instantiation Phase - import the necessary entities to interrogate
	// pending predictions
	if (CurrentPhase == ESystemPhase::Spawn && PendingPredictions.Num())
	{
		SCOPE_CYCLE_COUNTER(MovieSceneEval_PredictionIntialization)

		ensure(ProcessingPredictions.Num() == 0);

		Swap(ProcessingPredictions, PendingPredictions);
		PendingPredictions.Empty();

		for (int32 Index = 0; Index < ProcessingPredictions.Num(); ++Index)
		{
			UMovieSceneAsyncAction_SequencePrediction* Pending = ProcessingPredictions[Index];
			Pending->ImportEntities(&InterrogationChannels);
		}

		// Add mutual components for any interrogation entities
		Linker->EntityManager.AddMutualComponents(FEntityComponentFilter().All({ FBuiltInComponentTypes::Get()->Interrogation.InputKey }));
	}
	else if (CurrentPhase == ESystemPhase::Instantiation && ProcessingPredictions.Num())
	{
		// Initialize evaluation times
		TArrayView<const FInterrogationParams> Interrogations = InterrogationChannels.GetInterrogations();
		FEntityTaskBuilder()
		.Read(FBuiltInComponentTypes::Get()->Interrogation.InputKey)
		.Write(FBuiltInComponentTypes::Get()->EvalTime)
		.FilterNone({ FBuiltInComponentTypes::Get()->Tags.FixedTime })
		.Iterate_PerEntity(&Linker->EntityManager, [&Interrogations](const FInterrogationKey& InterrogationKey, FFrameTime& OutEvalTime) { OutEvalTime = Interrogations[InterrogationKey.InterrogationIndex].Time; });

		// This code is a bit of a hack, but is necessary to ensure that interrogation entities have the correct initial values
		// for any entities that need them
		{
			TMultiMap<FInterrogationChannel, FIntermediate3DTransform*> KeyToInitialValue;

			FEntityTaskBuilder()
			.Read(FBuiltInComponentTypes::Get()->Interrogation.OutputKey)
			.Write(FMovieSceneTracksComponentTypes::Get()->ComponentTransform.InitialValue)
			.Iterate_PerEntity(&Linker->EntityManager, [&KeyToInitialValue](const FInterrogationKey& InterrogationKey, FIntermediate3DTransform& OutInitialValue)
			{
				KeyToInitialValue.Add(InterrogationKey.Channel, &OutInitialValue);
			});

			if (KeyToInitialValue.Num() > 0)
			{
				FEntityTaskBuilder()
				.Read(FBuiltInComponentTypes::Get()->BoundObject)
				.Read(FMovieSceneTracksComponentTypes::Get()->ComponentTransform.InitialValue)
				.FilterAll({ FBuiltInComponentTypes::Get()->BlendChannelOutput })
				.Iterate_PerEntity(&Linker->EntityManager, [&](UObject* BoundObject, const FIntermediate3DTransform& InInitialValue)
				{
					FInterrogationChannel Channel = InterrogationChannels.FindChannel(BoundObject);
					if (Channel)
					{
						for (auto It = KeyToInitialValue.CreateKeyIterator(Channel); It; ++It)
						{
							*It.Value() = InInitialValue;
							It.RemoveCurrent();
						}
					}
				});

				const FSparseInterrogationChannelInfo& SparseChannelInfo = InterrogationChannels.GetSparseChannelInfo();
				for (auto It = KeyToInitialValue.CreateIterator(); It; ++It)
				{
					const FInterrogationChannelInfo& ChannelInfo = SparseChannelInfo.Get(It.Key());
					if (USceneComponent* SceneComponent = Cast<USceneComponent>(ChannelInfo.WeakObject.Get()))
					{
						*It.Value() = FIntermediate3DTransform(SceneComponent->GetRelativeLocation(), SceneComponent->GetRelativeRotation(), SceneComponent->GetRelativeScale3D());
					}
				}
			}
		}
	}
	else if (CurrentPhase == ESystemPhase::Finalization && ProcessingPredictions.Num())
	{
		bool bNeedsWorldSpace = false, bNeedsLocalSpace = false;

		for (UMovieSceneAsyncAction_SequencePrediction* Prediction : ProcessingPredictions)
		{
			bNeedsWorldSpace |= Prediction->bWorldSpace;
			bNeedsLocalSpace |= !Prediction->bWorldSpace;
		}

		TSparseArray<TArray<FTransform>> WorldTransforms;
		TSparseArray<TArray<UE::MovieScene::FIntermediate3DTransform>> LocalTransforms;
		{
			SCOPE_CYCLE_COUNTER(MovieSceneEval_PredictionFinalization)

			if (bNeedsLocalSpace)
			{
				InterrogationChannels.QueryLocalSpaceTransforms(Linker, LocalTransforms);
			}
			if (bNeedsWorldSpace)
			{
				InterrogationChannels.QueryWorldSpaceTransforms(Linker, WorldTransforms);
			}
		}

		{
			SCOPE_CYCLE_COUNTER(MovieSceneEval_PredictionReportResults)

			for (UMovieSceneAsyncAction_SequencePrediction* Prediction : ProcessingPredictions)
			{
				if (Prediction->bWorldSpace)
				{
					Prediction->ReportResult(&InterrogationChannels, WorldTransforms);
				}
				else
				{
					Prediction->ReportResult(&InterrogationChannels, LocalTransforms);
				}

				Prediction->Reset(Linker);
			}
		}

		Linker->EntityManager.MimicStructureChanged();
		ProcessingPredictions.Empty();

		InterrogationChannels.Reset();
	}
}

void UMovieScenePredictionSystem::AddPendingPrediction(UMovieSceneAsyncAction_SequencePrediction* Prediction)
{
	PendingPredictions.Add(Prediction);

	// Mimic structure changed to ensure that the instantiation phase runs
	Linker->EntityManager.MimicStructureChanged();
}

int32 UMovieScenePredictionSystem::MakeNewInterrogation(FFrameTime InTime)
{
	UE::MovieScene::FInterrogationParams Params{ InTime };
	return InterrogationChannels.AddInterrogation(Params);
}
