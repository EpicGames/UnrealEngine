// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieScenePropertyInstantiator.h"

#include "Algo/AllOf.h"
#include "Algo/IndexOf.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieSceneEntityGroupingSystem.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieScenePropertyBinding.h"
#include "EntitySystem/MovieScenePropertyRegistry.h"
#include "EntitySystem/MovieSceneInitialValueSystem.h"
#include "ProfilingDebugging/CountersTrace.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePropertyInstantiator)

DECLARE_CYCLE_STAT(TEXT("DiscoverInvalidatedProperties"), MovieSceneEval_DiscoverInvalidatedProperties, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("ProcessInvalidatedProperties"), MovieSceneEval_ProcessInvalidatedProperties, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("InitializePropertyMetaData"), MovieSceneEval_InitializePropertyMetaData, STATGROUP_MovieSceneECS);

namespace UE::MovieScene
{

struct FPropertyInstantiatorGroupingPolicy
{
	using GroupKeyType = TTuple<UObject*, FName>;

	bool GetGroupKey(UObject* Object, const FMovieScenePropertyBinding& PropertyBinding, GroupKeyType& OutGroupKey)
	{
		OutGroupKey = MakeTuple(Object, PropertyBinding.PropertyPath);
		return true;
	}

#if WITH_EDITOR
	bool OnObjectsReplaced(GroupKeyType& InOutKey, const TMap<UObject*, UObject*>& ReplacementMap)
	{
		if (UObject* const * NewObject = ReplacementMap.Find(InOutKey.Key))
		{
			InOutKey.Key = *NewObject;
			return true;
		}
		return false;
	}
#endif
};

} // namespace UE::MovieScene


UMovieScenePropertyInstantiatorSystem::FContributorKey UMovieScenePropertyInstantiatorSystem::FPropertyParameters::MakeContributorKey() const
{
	return FContributorKey(PropertyInfoIndex);
}

UMovieScenePropertyInstantiatorSystem::UMovieScenePropertyInstantiatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	BuiltInComponents = FBuiltInComponentTypes::Get();

	RecomposerImpl.OnGetPropertyInfo = FOnGetPropertyRecomposerPropertyInfo::CreateUObject(
				this, &UMovieScenePropertyInstantiatorSystem::FindPropertyFromSource);

	SystemCategories = FSystemInterrogator::GetExcludedFromInterrogationCategory();
	RelevantComponent = BuiltInComponents->PropertyBinding;
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);
		DefineComponentConsumer(GetClass(), BuiltInComponents->Group);
		DefineComponentConsumer(GetClass(), BuiltInComponents->HierarchicalBlendTarget);
		DefineComponentConsumer(GetClass(), BuiltInComponents->Tags.Ignored);

		DefineComponentProducer(GetClass(), BuiltInComponents->BlendChannelInput);
		DefineComponentProducer(GetClass(), BuiltInComponents->SymbolicTags.CreatesEntities);

		DefineImplicitPrerequisite(GetClass(), UMovieSceneInitialValueSystem::StaticClass());
	}
}

UE::MovieScene::FPropertyStats UMovieScenePropertyInstantiatorSystem::GetStatsForProperty(UE::MovieScene::FCompositePropertyTypeID PropertyID) const
{
	const int32 Index = PropertyID.AsIndex();
	if (PropertyStats.IsValidIndex(Index))
	{
		return PropertyStats[Index];
	}

	return UE::MovieScene::FPropertyStats();
}

void UMovieScenePropertyInstantiatorSystem::OnLink()
{
	using namespace UE::MovieScene;

	CleanFastPathMask.Reset();
	CleanFastPathMask.SetAll({ BuiltInComponents->FastPropertyOffset, BuiltInComponents->SlowProperty, BuiltInComponents->CustomPropertyIndex });
	CleanFastPathMask.CombineWithBitwiseOR(Linker->EntityManager.GetComponents()->GetMigrationMask(), EBitwiseOperatorFlags::MaxSize);

	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->LinkSystem<UMovieSceneEntityGroupingSystem>();
	PropertyGroupingKey = GroupingSystem->AddGrouping(
			FPropertyInstantiatorGroupingPolicy(),
			BuiltInComponents->BoundObject, BuiltInComponents->PropertyBinding);
			
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.AddUObject(this, &UMovieScenePropertyInstantiatorSystem::OnObjectsReplaced);
#endif
}

void UMovieScenePropertyInstantiatorSystem::OnUnlink()
{
	using namespace UE::MovieScene;

	const bool bAllPropertiesClean = (
				ResolvedProperties.Num() == 0 &&
				Contributors.Num() == 0 &&
				NewContributors.Num() == 0);
	if (!ensure(bAllPropertiesClean))
	{
		ResolvedProperties.Reset();
		Contributors.Reset();
		NewContributors.Reset();
		PropertyStats.Reset();
	}

	const bool bAllPropertiesGone = Algo::AllOf(
			PropertyStats, [](const FPropertyStats& Item) 
			{
				return Item.NumProperties == 0 && Item.NumPartialProperties == 0;
			});
	if (!ensure(bAllPropertiesGone))
	{
		PropertyStats.Reset();
	}

	const bool bAllTasksDone = (
			InitializePropertyMetaDataTasks.Num() == 0 &&
			SaveGlobalStateTasks.Num() == 0);
	if (!ensure(bAllTasksDone))
	{
		InitializePropertyMetaDataTasks.Reset();
		SaveGlobalStateTasks.Reset();
	}

	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->FindSystem<UMovieSceneEntityGroupingSystem>();
	if (ensure(GroupingSystem))
	{
		GroupingSystem->RemoveGrouping(PropertyGroupingKey);
	}
	PropertyGroupingKey = FEntityGroupingPolicyKey();
	
#if WITH_EDITOR
	FCoreUObjectDelegates::OnObjectsReplaced.RemoveAll(this);
#endif
}

void UMovieScenePropertyInstantiatorSystem::OnCleanTaggedGarbage()
{
	using namespace UE::MovieScene;

	// Only process expired properties for this GC pass to ensure we don't end up creating any new entities
	DiscoverExpiredProperties(PendingInvalidatedProperties);

	TArrayView<const FPropertyDefinition> Properties = this->BuiltInComponents->PropertyRegistry.GetProperties();

	bool bAnyDestroyed = false;
	// Look through our resolved properties to detect any outputs that have been destroyed
	for (int32 Index = 0; Index < ResolvedProperties.GetMaxIndex(); ++Index)
	{
		if (!ResolvedProperties.IsAllocated(Index))
		{
			continue;
		}

		FObjectPropertyInfo& PropertyInfo = ResolvedProperties[Index];

		FContributorKey ContributorKey(Index);

		const bool bFastPathOutputBeingDestroyed = PropertyInfo.PreviousFastPathID && Linker->EntityManager.HasComponent(PropertyInfo.PreviousFastPathID, BuiltInComponents->Tags.NeedsUnlink);

		// If the fast path is being destroyed, we need to copy over any initial values from that fast path entity to
		// any additional contributors that might still be alive. This can happen if a sequence gets GC'd and removes its entities while another is still alive animating the same thing.
		if (bFastPathOutputBeingDestroyed)
		{
			FMovieSceneEntityID TemporaryFastPathEntity;
			for (auto It = Contributors.CreateConstKeyIterator(ContributorKey); It; ++It)
			{
				Linker->EntityManager.AddComponent(It->Value, BuiltInComponents->Tags.NeedsLink);
				TemporaryFastPathEntity = It->Value;
			}

			if (TemporaryFastPathEntity)
			{
				const FPropertyDefinition& PropertyDefinition = Properties[PropertyInfo.PropertyDefinitionIndex];
				FComponentMask CopyMask;
				CopyMask.Set(PropertyDefinition.InitialValueType);
				CopyMask.Set(BuiltInComponents->Tags.HasAssignedInitialValue);
				for (FComponentTypeID Component : PropertyDefinition.MetaDataTypes)
				{
					CopyMask.Set(Component);
				}

				Linker->EntityManager.CopyComponents(PropertyInfo.PreviousFastPathID, TemporaryFastPathEntity, CopyMask);
				PropertyInfo.PreviousFastPathID = TemporaryFastPathEntity;
			}
			else
			{
				// There is nothing else animating this - destroy the property entirely
				DestroyStaleProperty(Index);
				bAnyDestroyed = true;

				if (PendingInvalidatedProperties.IsValidIndex(Index) && PendingInvalidatedProperties[Index] == true)
				{
					// This property index is no longer valid at all
					PendingInvalidatedProperties[Index] = false;
				}
			}
		}
		else if (PropertyInfo.FinalBlendOutputID && Linker->EntityManager.HasComponent(PropertyInfo.FinalBlendOutputID, BuiltInComponents->Tags.NeedsUnlink))
		{
			// Really we shouldn't have any contributors any more if the output is being destroyed since the target object must be going away
			//    (which means all the contributors that reference that target object must also be going away)
			if (!ensureMsgf(!Contributors.Contains(ContributorKey), TEXT("Blend output is being destroyed while there are still contributors.")))
			{
				// We still have contributors?? Shouldn't happen, but it's recoverable by just re-resolving all the contributors
				for (auto It = Contributors.CreateKeyIterator(ContributorKey); It; ++It)
				{
					Linker->EntityManager.AddComponent(It->Value, BuiltInComponents->Tags.NeedsLink);
					It.RemoveCurrent();
				}
			}

			DestroyStaleProperty(Index);
			bAnyDestroyed = true;

			if (PendingInvalidatedProperties.IsValidIndex(Index) && PendingInvalidatedProperties[Index] == true)
			{
				// This property index is no longer valid at all
				PendingInvalidatedProperties[Index] = false;
			}
		}
	}

	if (bAnyDestroyed)
	{
		PostDestroyStaleProperties();
	}
}

void UMovieScenePropertyInstantiatorSystem::OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap)
{
#if WITH_EDITOR
	for (auto It = ResolvedProperties.CreateIterator(); It; ++It)
	{
		FObjectPropertyInfo& ResolvedProperty = (*It);
		if (UObject* const* NewObject = ReplacementMap.Find(ResolvedProperty.BoundObject))
		{
			ResolvedProperty.BoundObject = *NewObject;
		}
	}
#endif
}

void UMovieScenePropertyInstantiatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	// Discover any newly created or expiring property entities
	DiscoverInvalidatedProperties(PendingInvalidatedProperties);

	if (PendingInvalidatedProperties.Num() != 0)
	{
		UpgradeFloatToDoubleProperties(PendingInvalidatedProperties);
		ProcessInvalidatedProperties(PendingInvalidatedProperties);

		PendingInvalidatedProperties.Empty();
	}

	if (InitializePropertyMetaDataTasks.Find(true) != INDEX_NONE)
	{
		InitializePropertyMetaData(InPrerequisites, Subsequents);
	}
}

void UMovieScenePropertyInstantiatorSystem::DiscoverInvalidatedProperties(TBitArray<>& OutInvalidatedProperties)
{
	using namespace UE::MovieScene;

	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_DiscoverInvalidatedProperties);

	TArrayView<const FPropertyDefinition> Properties = BuiltInComponents->PropertyRegistry.GetProperties();

	PropertyStats.SetNum(Properties.Num());

	DiscoverNewProperties(OutInvalidatedProperties);
	DiscoverExpiredProperties(OutInvalidatedProperties);
}

void UMovieScenePropertyInstantiatorSystem::DiscoverNewProperties(TBitArray<>&OutInvalidatedProperties)
{
	using namespace UE::MovieScene;

	TArrayView<const FPropertyDefinition> Properties = this->BuiltInComponents->PropertyRegistry.GetProperties();

	auto VisitNewProperties = [this, Properties, &OutInvalidatedProperties](FEntityAllocationIteratorItem AllocationItem, const FMovieSceneEntityID* EntityIDs, UObject* const * ObjectPtrs, const FMovieScenePropertyBinding* PropertyPtrs, const FEntityGroupID* GroupIDs)
	{
		const FEntityAllocation* Allocation     = AllocationItem.GetAllocation();
		const FComponentMask&    AllocationType = AllocationItem.GetAllocationType();

		const int32 PropertyDefinitionIndex = Algo::IndexOfByPredicate(Properties, [=](const FPropertyDefinition& InDefinition){ return Allocation->HasComponent(InDefinition.PropertyType); });
		if (PropertyDefinitionIndex == INDEX_NONE)
		{
			return;
		}

		const FPropertyDefinition& PropertyDefinition = Properties[PropertyDefinitionIndex];

		FCustomAccessorView CustomAccessors = PropertyDefinition.CustomPropertyRegistration ? PropertyDefinition.CustomPropertyRegistration->GetAccessors() : FCustomAccessorView();

		const bool bIgnored = AllocationType.Contains(BuiltInComponents->Tags.Ignored);
		if (bIgnored)
		{
			// Ignored properties should no longer contribute
			for (int32 Index = 0; Index < Allocation->Num(); ++Index)
			{
				const int32 PropertyIndex = GroupIDs[Index].GroupIndex;
				if (PropertyIndex != INDEX_NONE)
				{
					OutInvalidatedProperties.PadToNum(PropertyIndex + 1, false);
					OutInvalidatedProperties[PropertyIndex] = true;

					this->Contributors.Remove(PropertyIndex, EntityIDs[Index]);
				}
			}
		}
		else
		{
			for (int32 Index = 0; Index < Allocation->Num(); ++Index)
			{
				const bool bResolved = this->ResolveProperty(CustomAccessors, ObjectPtrs[Index], PropertyPtrs[Index], GroupIDs[Index], PropertyDefinitionIndex);
				if (bResolved)
				{
					const int32 PropertyIndex = GroupIDs[Index].GroupIndex;

					FContributorKey Key { PropertyIndex };
					this->Contributors.Add(Key, EntityIDs[Index]);
					this->NewContributors.Add(Key, EntityIDs[Index]);

					OutInvalidatedProperties.PadToNum(PropertyIndex + 1, false);
					OutInvalidatedProperties[PropertyIndex] = true;
				}
			}
		}
	};

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->BoundObject)
	.Read(BuiltInComponents->PropertyBinding)
	.Read(BuiltInComponents->Group)
	.FilterNone({ BuiltInComponents->BlendChannelOutput })
	.FilterAll({ BuiltInComponents->Tags.NeedsLink })
	.Iterate_PerAllocation(&Linker->EntityManager, VisitNewProperties);
}

void UMovieScenePropertyInstantiatorSystem::DiscoverExpiredProperties(TBitArray<>& OutInvalidatedProperties)
{
	using namespace UE::MovieScene;

	auto VisitExpiredEntities = [this, &OutInvalidatedProperties](FMovieSceneEntityID EntityID, const FEntityGroupID& GroupID)
	{
		const int32 PropertyIndex = GroupID.GroupIndex;
		if (PropertyIndex != INDEX_NONE)
		{
			OutInvalidatedProperties.PadToNum(PropertyIndex + 1, false);
			OutInvalidatedProperties[PropertyIndex] = true;

			this->Contributors.Remove(PropertyIndex, EntityID);
		}
	};

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->Group)
	.FilterNone({ BuiltInComponents->BlendChannelOutput })
	.FilterAll({ BuiltInComponents->BoundObject, BuiltInComponents->PropertyBinding, BuiltInComponents->Tags.NeedsUnlink })
	.Iterate_PerEntity(&Linker->EntityManager, VisitExpiredEntities);
}

void UMovieScenePropertyInstantiatorSystem::UpgradeFloatToDoubleProperties(const TBitArray<>& InvalidatedProperties)
{
	using namespace UE::MovieScene;

	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_ProcessInvalidatedProperties);

	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	TArrayView<const FPropertyDefinition> Properties = BuiltInComponents->PropertyRegistry.GetProperties();

	for (TConstSetBitIterator<> It(InvalidatedProperties); It; ++It)
	{
		const int32 PropertyIndex = It.GetIndex();
		if (!ResolvedProperties.IsValidIndex(PropertyIndex))
		{
			continue;
		}

		FObjectPropertyInfo& PropertyInfo = ResolvedProperties[PropertyIndex];
		const bool bHadConversionInfo = PropertyInfo.ConvertedFromPropertyDefinitionIndex.IsSet();

		// The first time we encounter a specific property, we need to figure out if it needs type conversion or not.
		if (!bHadConversionInfo)
		{
			// Don't do anything if the property isn't of the kind of type we need to care about. Right now, we only
			// support dealing with float->double and FVectorXf->FVectorXd.
			const FPropertyDefinition& PropertyDefinition = Properties[PropertyInfo.PropertyDefinitionIndex];
			if (PropertyDefinition.PropertyType != TrackComponents->Float.PropertyTag &&
				PropertyDefinition.PropertyType != TrackComponents->FloatVector.PropertyTag)
			{
				PropertyInfo.ConvertedFromPropertyDefinitionIndex = INDEX_NONE;
				continue;
			}

			FTrackInstancePropertyBindings Bindings(PropertyInfo.PropertyBinding.PropertyName, PropertyInfo.PropertyBinding.PropertyPath.ToString());
			FProperty* BoundProperty = Bindings.GetProperty(*PropertyInfo.BoundObject);
			if (!BoundProperty)
			{
				continue;
			}

			// Patch the resolved property info to point to the double-precision property definition.
			PropertyInfo.ConvertedFromPropertyDefinitionIndex = INDEX_NONE;
			if (PropertyDefinition.PropertyType == TrackComponents->Float.PropertyTag)
			{
				const bool bIsDouble = BoundProperty->IsA<FDoubleProperty>();
				if (bIsDouble)
				{
					const int32 DoublePropertyDefinitionIndex = Properties.IndexOfByPredicate(
						[TrackComponents](const FPropertyDefinition& Item) { return Item.PropertyType == TrackComponents->Double.PropertyTag; });
					ensure(DoublePropertyDefinitionIndex != INDEX_NONE);
					PropertyInfo.ConvertedFromPropertyDefinitionIndex = PropertyInfo.PropertyDefinitionIndex;
					PropertyInfo.PropertyDefinitionIndex = DoublePropertyDefinitionIndex;
				}
			}
			else if (PropertyDefinition.PropertyType == TrackComponents->FloatVector.PropertyTag)
			{
				const UScriptStruct* BoundStructProperty = CastField<FStructProperty>(BoundProperty)->Struct;
				const bool bIsDouble = (
					(BoundStructProperty == TBaseStructure<FVector2D>::Get()
						) ||
					(
						BoundStructProperty == TBaseStructure<FVector>::Get() ||
						BoundStructProperty == TVariantStructure<FVector3d>::Get()
						) ||
					(
						BoundStructProperty == TBaseStructure<FVector4>::Get() ||
						BoundStructProperty == TVariantStructure<FVector4d>::Get()
						));
				if (bIsDouble)
				{
					const int32 DoublePropertyDefinitionIndex = Properties.IndexOfByPredicate(
						[TrackComponents](const FPropertyDefinition& Item) { return Item.PropertyType == TrackComponents->DoubleVector.PropertyTag; });
					ensure(DoublePropertyDefinitionIndex != INDEX_NONE);
					PropertyInfo.ConvertedFromPropertyDefinitionIndex = PropertyInfo.PropertyDefinitionIndex;
					PropertyInfo.PropertyDefinitionIndex = DoublePropertyDefinitionIndex;
				}
			}
			else
			{
				check(false);
			}
		}

		const int32 OldPropertyDefinitionIndex = PropertyInfo.ConvertedFromPropertyDefinitionIndex.Get(INDEX_NONE);
		if (OldPropertyDefinitionIndex == INDEX_NONE)
		{
			continue;
		}

		// Now we need to patch the contributors so that they have double-precision components.
		// We only need to do it for the *new* contributors discovered this frame.
		FComponentTypeID OldPropertyTag, NewPropertyTag;
		const FPropertyDefinition& PropertyDefinition = Properties[PropertyInfo.PropertyDefinitionIndex];
		if (PropertyDefinition.PropertyType == TrackComponents->Double.PropertyTag)
		{
			OldPropertyTag = TrackComponents->Float.PropertyTag;
			NewPropertyTag = TrackComponents->Double.PropertyTag;
		}
		else if (PropertyDefinition.PropertyType == TrackComponents->DoubleVector.PropertyTag)
		{
			OldPropertyTag = TrackComponents->FloatVector.PropertyTag;
			NewPropertyTag = TrackComponents->DoubleVector.PropertyTag;
		}

		if (ensure(OldPropertyTag && NewPropertyTag))
		{
			// Swap out the property tag
			FContributorKey Key(PropertyIndex);
			for (auto ContribIt = NewContributors.CreateKeyIterator(Key); ContribIt; ++ContribIt)
			{
				const FMovieSceneEntityID CurID(ContribIt.Value());
				FComponentMask EntityType = Linker->EntityManager.GetEntityType(CurID);
				EntityType.Remove(OldPropertyTag);
				EntityType.Set(NewPropertyTag);
				Linker->EntityManager.ChangeEntityType(CurID, EntityType);
			}

			// Update contributor info and stats. This is only done the first time this property is encountered.
			if (!bHadConversionInfo)
			{
				--PropertyStats[OldPropertyDefinitionIndex].NumProperties;
				++PropertyStats[PropertyInfo.PropertyDefinitionIndex].NumProperties;
			}
		}
	}
}

void UMovieScenePropertyInstantiatorSystem::ProcessInvalidatedProperties(const TBitArray<>& InvalidatedProperties)
{
	using namespace UE::MovieScene;

	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_ProcessInvalidatedProperties);

	TBitArray<> StaleProperties;

	TArrayView<const FPropertyDefinition> Properties = BuiltInComponents->PropertyRegistry.GetProperties();

	FPropertyParameters Params;

	// This is all random access at this point :(
	for (TConstSetBitIterator<> It(InvalidatedProperties); It; ++It)
	{
		const int32 PropertyIndex = It.GetIndex();
		if (!ResolvedProperties.IsValidIndex(PropertyIndex))
		{
			continue;
		}

		// Update our view of how this property is animated
		Params.PropertyInfo = &ResolvedProperties[PropertyIndex];
		Params.PropertyDefinition = &Properties[Params.PropertyInfo->PropertyDefinitionIndex];
		Params.PropertyInfoIndex = PropertyIndex;

		UpdatePropertyInfo(Params);

		// Does it have anything at all contributing to it anymore?
		if (!Contributors.Contains(PropertyIndex))
		{
			StaleProperties.PadToNum(PropertyIndex + 1, false);
			StaleProperties[PropertyIndex] = true;
		}
		// Does it support fast path?
		else if (Params.PropertyInfo->bSupportsFastPath)
		{
			InitializeFastPath(Params);
		}
		// Else use the (slightly more) expensive blend path
		else
		{
			InitializeBlendPath(Params);
		}
	}

	// Restore and destroy stale properties
	if (StaleProperties.Find(true) != INDEX_NONE)
	{
		for (TConstSetBitIterator<> It(StaleProperties); It; ++It)
		{
			DestroyStaleProperty(It.GetIndex());
		}

		PostDestroyStaleProperties();
	}

	NewContributors.Empty();
}

void UMovieScenePropertyInstantiatorSystem::DestroyStaleProperty(int32 PropertyIndex)
{
	FObjectPropertyInfo* PropertyInfo = &ResolvedProperties[PropertyIndex];

	if (PropertyInfo->BlendChannel != INVALID_BLEND_CHANNEL)
	{
		if (UMovieSceneBlenderSystem* Blender = PropertyInfo->Blender.Get())
		{
			const FMovieSceneBlendChannelID BlendChannelID(Blender->GetBlenderSystemID(), PropertyInfo->BlendChannel);
			Blender->ReleaseBlendChannel(BlendChannelID);
		}
		Linker->EntityManager.AddComponents(PropertyInfo->FinalBlendOutputID, BuiltInComponents->FinishedMask);
	}

	if (PropertyInfo->bIsPartiallyAnimated)
	{
		--PropertyStats[PropertyInfo->PropertyDefinitionIndex].NumPartialProperties;
	}

	--PropertyStats[PropertyInfo->PropertyDefinitionIndex].NumProperties;
	ResolvedProperties.RemoveAt(PropertyIndex);

	// PropertyInfo is now garbage
}

void UMovieScenePropertyInstantiatorSystem::PostDestroyStaleProperties()
{
	ResolvedProperties.Shrink();
}

void UMovieScenePropertyInstantiatorSystem::UpdatePropertyInfo(const FPropertyParameters& Params)
{
	using namespace UE::MovieScene;

	TArrayView<const FPropertyCompositeDefinition> Composites = BuiltInComponents->PropertyRegistry.GetComposites(*Params.PropertyDefinition);

	// This function updates the meta-data associated with a property for each hbias that it is animated from
	// There are 3 possible 'modes' for hbias to be considered:
	//    - Blended means that greater biases are allowed to override lower biases, but still blend with them when the weight is < 1
	//    - Non-blended hbias simply disables contribution from any lower biases
	//    - Entities tagged with IgnoreHierarchicalBias will always be relevant and contribute with the highest bias


	// Channel masks for all entities, entities within the active hbias, and within the 'ignored hbias' buckets
	// Set bits denote channels that are not animated by entities in these contexts
	FChannelMask ActiveBiasEmptyChannels(true, Params.PropertyDefinition->CompositeSize);

	// Key that visits any contributor to this property regardless of hbias
	FContributorKey AnyContributor(Params.PropertyInfoIndex);

	int32 NumContributors = 0;
	int16 HBias = 0;
	bool bSupportsFastPath = true;
	bool bWantsRestoreState = false;
	bool bNeedsInitialValue = false;
	bool bBlendHierarchicalBias = false;

	// Iterate all contributors for this property to re-generate the meta-data
	for (auto ContributorIt = Contributors.CreateConstKeyIterator(AnyContributor); ContributorIt; ++ContributorIt)
	{
		FMovieSceneEntityID Contributor = ContributorIt.Value();

		const FComponentMask& Type = Linker->EntityManager.GetEntityType(Contributor);

		if (Type.Contains(BuiltInComponents->HierarchicalBias))
		{
			HBias = Linker->EntityManager.ReadComponentChecked(Contributor, BuiltInComponents->HierarchicalBias);
		}

		// Update the various empty channel masks
		for (int32 CompositeIndex = 0; CompositeIndex < Params.PropertyDefinition->CompositeSize; ++CompositeIndex)
		{
			const bool bCheckChannel = ActiveBiasEmptyChannels[CompositeIndex] == true;

			if (bCheckChannel)
			{
				FComponentTypeID ThisChannel = Composites[CompositeIndex].ComponentTypeID;
				if (ThisChannel && Type.Contains(ThisChannel))
				{
					ActiveBiasEmptyChannels[CompositeIndex] = false;
				}
			}
		}

		++NumContributors;

		// Update whether this meta-data entry wants restore state
		if (!bWantsRestoreState && Type.Contains(BuiltInComponents->Tags.RestoreState))
		{
			bWantsRestoreState = true;
		}

		// Update whether this meta-data entry needs an initial value or not
		if (!bNeedsInitialValue && Type.Contains(BuiltInComponents->Tags.AlwaysCacheInitialValue))
		{
			bNeedsInitialValue = true;
		}


		if (!bBlendHierarchicalBias && Type.Contains(BuiltInComponents->Tags.BlendHierarchicalBias))
		{
			bBlendHierarchicalBias = true;
			bSupportsFastPath = false;
		}

		// Update whether this property supports fast path
		if (bSupportsFastPath)
		{
			if (NumContributors > 1)
			{
				bSupportsFastPath = false;
			}
			else
			{
				if (Type.Contains(BuiltInComponents->Tags.RelativeBlend) || 
						Type.Contains(BuiltInComponents->Tags.AdditiveBlend) || 
						Type.Contains(BuiltInComponents->Tags.OverrideBlend) ||
						Type.Contains(BuiltInComponents->Tags.AdditiveFromBaseBlend) || 
						Type.Contains(BuiltInComponents->WeightAndEasingResult))
				{
					bSupportsFastPath = false;
				}
			}
		}
	}

	// Reset the restore state status of the property if we still have contributors
	// We do not do this if there are no contributors to ensure that stale properties are restored correctly
	Params.PropertyInfo->EmptyChannels = ActiveBiasEmptyChannels;

	const bool bWasPartial = Params.PropertyInfo->bIsPartiallyAnimated;
	const bool bIsPartial  = Params.PropertyInfo->EmptyChannels.Find(true) != INDEX_NONE;

	if (bWasPartial != bIsPartial)
	{
		const int32 StatIndex = Params.PropertyInfo->PropertyDefinitionIndex;
		PropertyStats[StatIndex].NumPartialProperties += bIsPartial ? 1 : -1;
	}

	Params.PropertyInfo->bIsPartiallyAnimated   = bIsPartial;
	Params.PropertyInfo->bMaxHBiasHasChanged    = Params.PropertyInfo->HBias != HBias;
	Params.PropertyInfo->HBias                  = HBias;
	Params.PropertyInfo->bSupportsFastPath      = bSupportsFastPath;
	Params.PropertyInfo->bWantsRestoreState     = bWantsRestoreState;
	Params.PropertyInfo->bNeedsInitialValue     = bNeedsInitialValue;
}

void UMovieScenePropertyInstantiatorSystem::InitializeFastPath(const FPropertyParameters& Params)
{
	using namespace UE::MovieScene;

	// Find the sole contributor with the specific property info and hbias
	const int16 ActiveHBias = Params.PropertyInfo->HBias;
	FContributorKey AnyContributor(Params.PropertyInfoIndex);
	for (auto ContributorIt = Contributors.CreateConstKeyIterator(AnyContributor); ContributorIt; ++ContributorIt)
	{
		FMovieSceneEntityID Contributor = ContributorIt.Value();

		FTypelessMutation SoleContributorMutation;
		SoleContributorMutation.RemoveMask.SetAll({ BuiltInComponents->BlendChannelInput, BuiltInComponents->HierarchicalBlendTarget });

		if (Params.PropertyInfo->bNeedsInitialValue)
		{
			SoleContributorMutation.AddMask.Set(Params.PropertyDefinition->InitialValueType);
		}

		if (Params.PropertyDefinition->MetaDataTypes.Num() > 0)
		{
			InitializePropertyMetaDataTasks.PadToNum(Params.PropertyInfo->PropertyDefinitionIndex+1, false);
			InitializePropertyMetaDataTasks[Params.PropertyInfo->PropertyDefinitionIndex] = true;

			for (FComponentTypeID Component : Params.PropertyDefinition->MetaDataTypes)
			{
				SoleContributorMutation.AddMask.Set(Component);
			}
		}

		// Ensure the sole contributor is set up to apply the property as a final output
		switch (Params.PropertyInfo->Property.GetIndex())
		{
		case 0:
			FEntityBuilder()
			.Add(BuiltInComponents->FastPropertyOffset, Params.PropertyInfo->Property.template Get<uint16>())
			.MutateExisting(&Linker->EntityManager, Contributor, SoleContributorMutation);
			break;
		case 1:
			FEntityBuilder()
			.Add(BuiltInComponents->CustomPropertyIndex, Params.PropertyInfo->Property.template Get<FCustomPropertyIndex>())
			.MutateExisting(&Linker->EntityManager, Contributor, SoleContributorMutation);
			break;
		case 2:
			FEntityBuilder()
			.Add(BuiltInComponents->SlowProperty, Params.PropertyInfo->Property.template Get<FSlowPropertyPtr>())
			.MutateExisting(&Linker->EntityManager, Contributor, SoleContributorMutation);
			break;
		}

		// Copy initial values and meta-data back off our old blend output
		FMovieSceneEntityID OldOutput = Params.PropertyInfo->PreviousFastPathID ? Params.PropertyInfo->PreviousFastPathID : Params.PropertyInfo->FinalBlendOutputID;
		if (OldOutput)
		{
			FComponentMask CopyMask;
			CopyMask.Set(Params.PropertyDefinition->InitialValueType);
			CopyMask.Set(BuiltInComponents->Tags.HasAssignedInitialValue);
			for (FComponentTypeID Component : Params.PropertyDefinition->MetaDataTypes)
			{
				CopyMask.Set(Component);
			}

			Linker->EntityManager.CopyComponents(OldOutput, Contributor, CopyMask);
		}

		Params.PropertyInfo->PreviousFastPathID = Contributor;
	}

	// If this was previously blended, destroy the blend output
	if (Params.PropertyInfo->FinalBlendOutputID)
	{
		check(!Linker->EntityManager.HasComponent(Params.PropertyInfo->FinalBlendOutputID, BuiltInComponents->BlendChannelInput));

		UMovieSceneBlenderSystem* Blender = Params.PropertyInfo->Blender.Get();
		if (Blender && Params.PropertyInfo->BlendChannel != INVALID_BLEND_CHANNEL)
		{
			const FMovieSceneBlendChannelID BlendChannelID(Blender->GetBlenderSystemID(), Params.PropertyInfo->BlendChannel);
			Blender->ReleaseBlendChannel(BlendChannelID);

			Params.PropertyInfo->BlendChannel = INVALID_BLEND_CHANNEL;
			Params.PropertyInfo->Blender = nullptr;
		}

		Linker->EntityManager.AddComponent(Params.PropertyInfo->FinalBlendOutputID, BuiltInComponents->Tags.NeedsUnlink);
		Params.PropertyInfo->FinalBlendOutputID = FMovieSceneEntityID();
	}
}

UMovieScenePropertyInstantiatorSystem::FSetupBlenderSystemResult UMovieScenePropertyInstantiatorSystem::SetupBlenderSystem(const FPropertyParameters& Params)
{
	using namespace UE::MovieScene;

	UClass* NewBlenderClass = nullptr;
	int32 BlenderClassPriority = TNumericLimits<int32>::Lowest();

	// Iterate all the contributors to locate the correct blender system by-priority
	FContributorKey ContributorKey = Params.MakeContributorKey();
	for (auto ContributorIt = Contributors.CreateConstKeyIterator(ContributorKey); ContributorIt; ++ContributorIt)
	{
		FMovieSceneEntityID Contributor = ContributorIt.Value();
		TOptionalComponentReader<TSubclassOf<UMovieSceneBlenderSystem>> BlenderTypeComponent = Linker->EntityManager.ReadComponent(Contributor, BuiltInComponents->BlenderType);
		if (!BlenderTypeComponent)
		{
			continue;
		}

		UClass* ProspectiveBlenderClass = BlenderTypeComponent->Get();
		if (NewBlenderClass == ProspectiveBlenderClass)
		{
			// If it's already the same, don't waste time getting the CDO or anything like that
			continue;
		}

		const UMovieSceneBlenderSystem* CDO = GetDefault<UMovieSceneBlenderSystem>(ProspectiveBlenderClass);

		if (!NewBlenderClass || CDO->GetSelectionPriority() > BlenderClassPriority)
		{
			NewBlenderClass = ProspectiveBlenderClass;
			BlenderClassPriority = CDO->GetSelectionPriority();
		}
		else
		{
#if DO_CHECK
			ensureMsgf(CDO->GetSelectionPriority() != BlenderClassPriority,
				TEXT("Encountered 2 different blender classes being used with the same priority - this is undefined behavior. Please check the system classes to ensure they have different priorities (%s and %s)."),
				*ProspectiveBlenderClass->GetName(), *NewBlenderClass->GetName());
#endif
		}
	}

	if (!NewBlenderClass)
	{
		NewBlenderClass = Params.PropertyDefinition->BlenderSystemClass;
	}

	if (!NewBlenderClass)
	{
		return FSetupBlenderSystemResult();
	}

	FComponentTypeID BlenderTypeTag = GetDefault<UMovieSceneBlenderSystem>(NewBlenderClass)->GetBlenderTypeTag();
	ensureMsgf(BlenderTypeTag, TEXT("Encountered a blender system (%s) with an invalid type tag."), *NewBlenderClass->GetName());

	FBlenderSystemInfo NewBlenderInfo{ NewBlenderClass, BlenderTypeTag };
	FBlenderSystemInfo OldBlenderInfo;

	UMovieSceneBlenderSystem* ExistingBlender = Params.PropertyInfo->Blender.Get();
	if (ExistingBlender)
	{
		UClass* OldBlenderClass = ExistingBlender->GetClass();
		if (OldBlenderClass != NewBlenderClass)
		{
			OldBlenderInfo = FBlenderSystemInfo{ OldBlenderClass, ExistingBlender->GetBlenderTypeTag() };
		}
		else
		{
			// It's the same - keep the same info
			OldBlenderInfo = NewBlenderInfo;
		}
	}

	return FSetupBlenderSystemResult{NewBlenderInfo, OldBlenderInfo};
}



void UMovieScenePropertyInstantiatorSystem::InitializeBlendPath(const FPropertyParameters& Params)
{
	using namespace UE::MovieScene;

	TArrayView<const FPropertyCompositeDefinition> Composites = BuiltInComponents->PropertyRegistry.GetComposites(*Params.PropertyDefinition);

	FSetupBlenderSystemResult SetupResult = SetupBlenderSystem(Params);

	// -----------------------------------------------------------------------------------------------------
	// Situation 0: there is no blender system for this type of property. Normally, this means we can't
	// end up trying to blend it, but it may happen if that property is inside a sub-sequence with hierarchical
	// easing.
	if (!SetupResult.CurrentInfo.BlenderSystemClass)
	{
		InitializeFastPath(Params);
		return;
	}

	// -----------------------------------------------------------------------------------------------------
	// Situation 1: New or modified contributors (inputs) but we're already set up for blending using the same system
	if (Params.PropertyInfo->FinalBlendOutputID && SetupResult.CurrentInfo.BlenderSystemClass == SetupResult.PreviousInfo.BlenderSystemClass)
	{
		UMovieSceneBlenderSystem* Blender = Params.PropertyInfo->Blender.Get();
		check(Blender);

		// Ensure the output entity still matches the correct set of channels of all inputs
		FComponentMask NewEntityType;
		Params.MakeOutputComponentType(Linker->EntityManager, Composites, NewEntityType);
		Linker->EntityManager.ChangeEntityType(Params.PropertyInfo->FinalBlendOutputID, NewEntityType);

		const FMovieSceneBlendChannelID BlendChannel(Blender->GetBlenderSystemID(), Params.PropertyInfo->BlendChannel);

		// Change new contributors to include the blend input components
		FContributorKey ContributorKey = Params.MakeContributorKey();

		TMultiMap<FContributorKey, FMovieSceneEntityID>::TConstKeyIterator ContributorIt = Params.PropertyInfo->bMaxHBiasHasChanged
			? Contributors.CreateConstKeyIterator(ContributorKey)
			: NewContributors.CreateConstKeyIterator(ContributorKey);

		for (; ContributorIt; ++ContributorIt)
		{
			FEntityBuilder()
			.Add(BuiltInComponents->BlendChannelInput, BlendChannel)
			.AddTag(SetupResult.CurrentInfo.BlenderTypeTag)
			.MutateExisting(&Linker->EntityManager, ContributorIt.Value());
		}

		check(!Linker->EntityManager.HasComponent(Params.PropertyInfo->FinalBlendOutputID, BuiltInComponents->BlendChannelInput));

		// Nothing more to do
		return;
	}

	// -----------------------------------------------------------------------------------------------------
	// Situation 2: Never used blending before, or the blender type has changed
	UMovieSceneBlenderSystem* OldBlender = Params.PropertyInfo->Blender.Get();
	UMovieSceneBlenderSystem* NewBlender = CastChecked<UMovieSceneBlenderSystem>(Linker->LinkSystem(SetupResult.CurrentInfo.BlenderSystemClass));

	const FMovieSceneBlendChannelID NewBlendChannel = NewBlender->AllocateBlendChannel();

	FTypelessMutation InputMutation;
	// Clean any previously-fast-path entities
	InputMutation.RemoveMask = CleanFastPathMask;
	InputMutation.RemoveMask.Set(Params.PropertyDefinition->InitialValueType);
	InputMutation.RemoveMask.Set(BuiltInComponents->Tags.HasAssignedInitialValue);
	for (FComponentTypeID Component : Params.PropertyDefinition->MetaDataTypes)
	{
		InputMutation.RemoveMask.Set(Component);
	}

	// -----------------------------------------------------------------------------------------------------
	// Situation 2.1: We're already set up for blending, but with a different blender system
	if (OldBlender)
	{
		const FMovieSceneBlendChannelID OldBlendChannel(OldBlender->GetBlenderSystemID(), Params.PropertyInfo->BlendChannel);
		OldBlender->ReleaseBlendChannel(OldBlendChannel);

		Params.PropertyInfo->Blender = NewBlender;
		Params.PropertyInfo->BlendChannel = NewBlendChannel.ChannelID;

		// Change the output entity by adding the new blend channel and tag, while simultaneously
		// updating the channels and restore state flags etc added by MakeOutputComponentType
		{
			FTypelessMutation OutputMutation;
			OutputMutation.RemoveAll();

			Params.MakeOutputComponentType(Linker->EntityManager, Composites, OutputMutation.AddMask);

			// Remove the old blender type tag before add the new one
			OutputMutation.AddMask.Remove({ SetupResult.PreviousInfo.BlenderTypeTag });

			FEntityBuilder()
			.Add(BuiltInComponents->BlendChannelOutput, NewBlendChannel)
			.AddTag(SetupResult.CurrentInfo.BlenderTypeTag)
			.MutateExisting(&Linker->EntityManager, Params.PropertyInfo->FinalBlendOutputID, OutputMutation);
		}

		// Ensure that the old blend tag is removed from inputs
		InputMutation.Remove({ SetupResult.PreviousInfo.BlenderTypeTag });
	}

	// -----------------------------------------------------------------------------------------------------
	// Situation 2.2: Never encountered blending before - need to create a new output entity to receive the blend result
	else
	{
		Params.PropertyInfo->Blender = NewBlender;
		Params.PropertyInfo->BlendChannel = NewBlendChannel.ChannelID;

		FComponentMask NewMask;
		NewMask.Set(Params.PropertyDefinition->InitialValueType);

		if (Params.PropertyDefinition->MetaDataTypes.Num() > 0)
		{
			InitializePropertyMetaDataTasks.PadToNum(Params.PropertyInfo->PropertyDefinitionIndex+1, false);
			InitializePropertyMetaDataTasks[Params.PropertyInfo->PropertyDefinitionIndex] = true;

			for (FComponentTypeID Component : Params.PropertyDefinition->MetaDataTypes)
			{
				NewMask.Set(Component);
			}
		}

		for (int32 Index = 0; Index < Composites.Num(); ++Index)
		{
			if (Params.PropertyInfo->EmptyChannels[Index] == false)
			{
				NewMask.Set(Composites[Index].ComponentTypeID);
			}
		}
		NewMask.Set(Params.PropertyDefinition->PropertyType);

		FMovieSceneEntityID NewOutputEntityID;

		auto NewOutputEntity = FEntityBuilder()
		.Add(BuiltInComponents->BlendChannelOutput,      NewBlendChannel)
		.Add(BuiltInComponents->PropertyBinding,         Params.PropertyInfo->PropertyBinding)
		.Add(BuiltInComponents->BoundObject,             Params.PropertyInfo->BoundObject)
		.AddTagConditional(BuiltInComponents->Tags.RestoreState, Params.PropertyInfo->bWantsRestoreState)
		.AddTag(SetupResult.CurrentInfo.BlenderTypeTag)
		.AddTag(BuiltInComponents->Tags.NeedsLink)
		.AddMutualComponents();

		switch (Params.PropertyInfo->Property.GetIndex())
		{
		// Never seen this property before
		case 0:
			NewOutputEntityID = NewOutputEntity
			.Add(BuiltInComponents->FastPropertyOffset, Params.PropertyInfo->Property.template Get<uint16>())
			.CreateEntity(&Linker->EntityManager, NewMask);
			break;

		case 1:
			NewOutputEntityID = NewOutputEntity
			.Add(BuiltInComponents->CustomPropertyIndex, Params.PropertyInfo->Property.template Get<FCustomPropertyIndex>())
			.CreateEntity(&Linker->EntityManager, NewMask);
			break;

		case 2:
			NewOutputEntityID = NewOutputEntity
			.Add(BuiltInComponents->SlowProperty, Params.PropertyInfo->Property.template Get<FSlowPropertyPtr>())
			.CreateEntity(&Linker->EntityManager, NewMask);
			break;
		}

		if (Params.PropertyInfo->PreviousFastPathID)
		{
			// If this contributor has the initial values on it, we copy its initial values and meta-data components
			FComponentMask CopyMask = Linker->EntityManager.GetComponents()->GetCopyAndMigrationMask();

			CopyMask.Set(Params.PropertyDefinition->InitialValueType);
			CopyMask.Set(BuiltInComponents->Tags.HasAssignedInitialValue);
			for (FComponentTypeID Component : Params.PropertyDefinition->MetaDataTypes)
			{
				CopyMask.Set(Component);
			}

			Linker->EntityManager.CopyComponents(Params.PropertyInfo->PreviousFastPathID, NewOutputEntityID, CopyMask);
		}

		// The property entity ID is now the blend output entity
		Params.PropertyInfo->FinalBlendOutputID = NewOutputEntityID;
		Params.PropertyInfo->PreviousFastPathID = FMovieSceneEntityID();

		check(!Linker->EntityManager.HasComponent(Params.PropertyInfo->FinalBlendOutputID, BuiltInComponents->BlendChannelInput));
	}

	// Change *all* contributors (not just new ones because the old ones will have the old blender's channel and tag on them)
	// to include the new blend channel input, and remove the old blender type. No need to remove the clean fast path mask because
	// that will have already happened as part of the 'completely new blending' branch below
	FContributorKey ContributorKey(Params.PropertyInfoIndex);
	for (auto ContributorIt = Contributors.CreateConstKeyIterator(ContributorKey); ContributorIt; ++ContributorIt)
	{
		FMovieSceneEntityID Contributor = ContributorIt.Value();

		FEntityBuilder()
		.Add(BuiltInComponents->BlendChannelInput, NewBlendChannel)
		.AddTag(SetupResult.CurrentInfo.BlenderTypeTag)
		.MutateExisting(&Linker->EntityManager, Contributor, InputMutation);
	}

	check(!Linker->EntityManager.HasComponent(Params.PropertyInfo->FinalBlendOutputID, BuiltInComponents->BlendChannelInput));
}

bool UMovieScenePropertyInstantiatorSystem::ResolveProperty(UE::MovieScene::FCustomAccessorView CustomAccessors, UObject* Object, const FMovieScenePropertyBinding& PropertyBinding, const UE::MovieScene::FEntityGroupID& GroupID, int32 PropertyDefinitionIndex)
{
	using namespace UE::MovieScene;

	if (ResolvedProperties.IsValidIndex(GroupID.GroupIndex))
	{
#if !UE_BUILD_SHIPPING
		const FObjectPropertyInfo& ResolvedProperty = ResolvedProperties[GroupID.GroupIndex];
		ensure(ResolvedProperty.BoundObject == Object);
		ensure(ResolvedProperty.PropertyBinding.PropertyPath == PropertyBinding.PropertyPath);
#endif
		return true;
	}

	TOptional<FResolvedProperty> ResolvedProperty = FPropertyRegistry::ResolveProperty(Object, PropertyBinding, CustomAccessors);
	if (!ResolvedProperty.IsSet())
	{
		UE_LOGF(LogMovieScene, Warning, "Unable to resolve property '%ls' from '%ls' instance '%ls'", *PropertyBinding.PropertyPath.ToString(), *Object->GetClass()->GetName(), *Object->GetName());
		return false;
	}

	ResolvedProperties.EmplaceAt(GroupID.GroupIndex, MoveTemp(ResolvedProperty.GetValue()));
	FObjectPropertyInfo& NewInfo = ResolvedProperties[GroupID.GroupIndex];

	NewInfo.BoundObject = Object;
	NewInfo.PropertyBinding = PropertyBinding;
	NewInfo.PropertyDefinitionIndex = PropertyDefinitionIndex;

	++PropertyStats[PropertyDefinitionIndex].NumProperties;

	return true;
}

UE::MovieScene::FPropertyRecomposerPropertyInfo UMovieScenePropertyInstantiatorSystem::FindPropertyFromSource(FMovieSceneEntityID EntityID, UObject* Object) const
{
	using namespace UE::MovieScene;

	TOptionalComponentReader<FMovieScenePropertyBinding> PropertyBinding = Linker->EntityManager.ReadComponent(EntityID, BuiltInComponents->PropertyBinding);
	if (!PropertyBinding)
	{
		return FPropertyRecomposerPropertyInfo::Invalid();
	}

	TOptionalComponentReader<FEntityGroupID> GroupID = Linker->EntityManager.ReadComponent(EntityID, BuiltInComponents->Group);
	if (!GroupID)
	{
		// If this Entity didn't have a group, see if any of its children are bound to the same object and use that
		Linker->EntityManager.IterateImmediateChildren(EntityID, [this, Object, &GroupID](FMovieSceneEntityID ChildEntityID){
			TOptionalComponentReader<UObject*>       BoundObject = this->Linker->EntityManager.ReadComponent(ChildEntityID, BuiltInComponents->BoundObject);
			TOptionalComponentReader<FEntityGroupID> ChildGroup  = this->Linker->EntityManager.ReadComponent(ChildEntityID, BuiltInComponents->Group);
			if (BoundObject && *BoundObject == Object && ChildGroup)
			{
				GroupID = MoveTemp(ChildGroup);
			}
		});
	}

	if (!GroupID)
	{
		return FPropertyRecomposerPropertyInfo::Invalid();
	}

	const int32 PropertyIndex = GroupID->GroupIndex;
	if (PropertyIndex != INDEX_NONE && ResolvedProperties.IsValidIndex(PropertyIndex))
	{
		const uint16 BlendChannel = ResolvedProperties[PropertyIndex].BlendChannel;
		const FObjectPropertyInfo& PropertyInfo = ResolvedProperties[PropertyIndex];
		return FPropertyRecomposerPropertyInfo { BlendChannel, PropertyInfo.Blender.Get(), PropertyInfo.FinalBlendOutputID };
	}

	return FPropertyRecomposerPropertyInfo::Invalid();
}

void UMovieScenePropertyInstantiatorSystem::InitializePropertyMetaData(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	MOVIESCENE_DETAILED_SCOPE_CYCLE_COUNTER(MovieSceneEval_InitializePropertyMetaData);

	for (TConstSetBitIterator<> TypesToCache(InitializePropertyMetaDataTasks); TypesToCache; ++TypesToCache)
	{
		FCompositePropertyTypeID PropertyID = FCompositePropertyTypeID::FromIndex(TypesToCache.GetIndex());

		const FPropertyDefinition& Definition = BuiltInComponents->PropertyRegistry.GetDefinition(PropertyID);
		Definition.Handler->DispatchInitializePropertyMetaDataTasks(Definition, InPrerequisites, Subsequents, Linker);
	}

	InitializePropertyMetaDataTasks.Empty();
}

void UMovieScenePropertyInstantiatorSystem::FPropertyParameters::MakeOutputComponentType(
	const UE::MovieScene::FEntityManager& EntityManager,
	TArrayView<const UE::MovieScene::FPropertyCompositeDefinition> Composites,
	UE::MovieScene::FComponentMask& OutComponentType) const
{
	using namespace UE::MovieScene;

	// Get the existing type
	if (PropertyInfo->FinalBlendOutputID)
	{
		OutComponentType = EntityManager.GetEntityType(PropertyInfo->FinalBlendOutputID);
	}

	// Ensure the property has only the exact combination of channels that constitute its animation
	for (int32 Index = 0; Index < Composites.Num(); ++Index)
	{
		FComponentTypeID Composite = Composites[Index].ComponentTypeID;
		if (PropertyInfo->EmptyChannels[Index] != true)
		{
			OutComponentType.Set(Composite);
		}
		else
		{
			OutComponentType.Remove(Composite);
		}
	}
	OutComponentType.Set(PropertyDefinition->PropertyType);

	// Set the restore state tag appropriately
	if (PropertyInfo->bWantsRestoreState)
	{
		OutComponentType.Set(FBuiltInComponentTypes::Get()->Tags.RestoreState);
	}
	else
	{
		OutComponentType.Remove(FBuiltInComponentTypes::Get()->Tags.RestoreState);
	}
}

