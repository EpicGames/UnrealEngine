// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieScenePropertyNotificationSystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePropertyNotificationSystem)

namespace UE::MovieScene
{

struct FCallNotificationsTask
{
	void ForEachEntity(UObject* const BoundObject, FPropertyNotifyComponentData& PropertyNotify)
	{
		EnsureNotifyFunctionCached(BoundObject, PropertyNotify);

		UFunction* NotifyFunction = PropertyNotify.WeakNotifyFunction.Get();
		const FNotification Notification{ BoundObject, NotifyFunction };
		if (NotifyFunction && !ProcessedNotifications.Contains(Notification))
		{
			BoundObject->ProcessEvent(NotifyFunction, nullptr);
			ProcessedNotifications.Add(Notification);
		}
	}

private:

	void EnsureNotifyFunctionCached(UObject* const BoundObject, FPropertyNotifyComponentData& PropertyNotify)
	{
		if (PropertyNotify.bNotifyFunctionCached)
		{
			return;
		}

		const FName NotifyFuncName = PropertyNotify.NotifyFunctionName;
		if (UFunction* NotifyFunction = BoundObject->FindFunction(NotifyFuncName))
		{
			if (NotifyFunction->NumParms == 0 && NotifyFunction->ReturnValueOffset == MAX_uint16)
			{
				PropertyNotify.WeakNotifyFunction = NotifyFunction;
			}
			else
			{
				UE_LOGF(LogMovieScene, Warning, 
						"Notification function '%ls' on object class '%ls' must take no parameters and return no value.",
						*NotifyFuncName.ToString(), *BoundObject->GetClass()->GetName());
			}
		}
		else
		{
			UE_LOGF(LogMovieScene, Warning, 
					"Notification function '%ls' not found on object class '%ls'.",
					*NotifyFuncName.ToString(), *BoundObject->GetClass()->GetName());
		}

		PropertyNotify.bNotifyFunctionCached = true;
	}

private:

	using FNotification = TTuple<UObject*, UFunction*>;
	TSet<FNotification> ProcessedNotifications;
};

}  // namespace UE::MovieScene

UMovieScenePropertyNotificationSystem::UMovieScenePropertyNotificationSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	SystemCategories = EEntitySystemCategory::Core;
	Phase = ESystemPhase::Finalization;
	RelevantComponent = TracksComponents->PropertyNotify;
}

void UMovieScenePropertyNotificationSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FCallNotificationsTask CallNotificationsTask;

	FEntityTaskBuilder()
	.Read(BuiltInComponents->BoundObject)
	.Write(TracksComponents->PropertyNotify)
	.FilterAny({ BuiltInComponents->CustomPropertyIndex, BuiltInComponents->FastPropertyOffset, BuiltInComponents->SlowProperty })
	.FilterNone({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->Tags.Ignored })
	.RunInline_PerEntity(&Linker->EntityManager, CallNotificationsTask);
}

