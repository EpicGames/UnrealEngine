// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieScenePropertySystem.h"
#include "Systems/MovieScenePropertyInstantiator.h"

#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSources.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePropertySystem)

UMovieScenePropertySystem::UMovieScenePropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Scheduling;
	SystemCategories = EEntitySystemCategory::PropertySystems | FSystemInterrogator::GetExcludedFromInterrogationCategory();
}

void UMovieScenePropertySystem::OnLink()
{
	using namespace UE::MovieScene;

	// Never apply properties during evaluation. This code is necessary if derived types do support interrogation.
	InstantiatorSystem = Linker->LinkSystemIfAllowed<UMovieScenePropertyInstantiatorSystem>();
	if (InstantiatorSystem)
	{
		Linker->SystemGraph.AddReference(this, InstantiatorSystem);
	}
}

void UMovieScenePropertySystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	// Never apply properties during evaluation. This code is necessary if derived types do support interrogation.
	if (!InstantiatorSystem)
	{
		return;
	}

	FPropertyStats Stats = InstantiatorSystem->GetStatsForProperty(CompositePropertyID);
	if (Stats.NumProperties > 0)
	{
		const FPropertyRegistry&   PropertyRegistry = FBuiltInComponentTypes::Get()->PropertyRegistry;
		const FPropertyDefinition& Definition       = PropertyRegistry.GetDefinition(CompositePropertyID);

		Definition.Handler->ScheduleSetterTasks(Definition, PropertyRegistry.GetComposites(Definition), Stats, TaskScheduler, Linker);
	}
}

void UMovieScenePropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	// Never apply properties during evaluation. This code is necessary if derived types do support interrogation.
	if (!InstantiatorSystem)
	{
		return;
	}

	FPropertyStats Stats = InstantiatorSystem->GetStatsForProperty(CompositePropertyID);
	if (Stats.NumProperties > 0)
	{
		const FPropertyRegistry&   PropertyRegistry = FBuiltInComponentTypes::Get()->PropertyRegistry;
		const FPropertyDefinition& Definition       = PropertyRegistry.GetDefinition(CompositePropertyID);

		Definition.Handler->DispatchSetterTasks(Definition, PropertyRegistry.GetComposites(Definition), Stats, InPrerequisites, Subsequents, Linker);
	}
}

void UMovieScenePropertySystem::SavePreAnimatedState(const FPreAnimationParameters& InParameters)
{
	using namespace UE::MovieScene;

	using FThreeWayAccessor  = TMultiReadOptional<FCustomPropertyIndex, uint16, TSharedPtr<FTrackInstancePropertyBindings>>;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	const FPropertyDefinition& Definition = BuiltInComponents->PropertyRegistry.GetDefinition(CompositePropertyID);

	TSharedPtr<IPreAnimatedStorage> PreAnimatedStorage = Definition.Handler->GetPreAnimatedStateStorage(Definition, InParameters.CacheExtension);
	if (!PreAnimatedStorage)
	{
		return;
	}

	PreAnimatedStorageID = PreAnimatedStorage->GetStorageType();

	FComponentMask IncludeMask({ Definition.PropertyType });
	if (!InParameters.CacheExtension->AreEntriesInvalidated())
	{
		IncludeMask.Set(BuiltInComponents->Tags.NeedsLink);
	}

	FComponentMask ExcludeMask({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->Tags.Finished, BuiltInComponents->Tags.Ignored });

	if (IPreAnimatedObjectEntityStorage* ObjectStorage = PreAnimatedStorage->AsObjectStorage())
	{
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(BuiltInComponents->BoundObject)
		.FilterAll(IncludeMask)
		.FilterNone(ExcludeMask)
		.Iterate_PerAllocation(&Linker->EntityManager,
			[ObjectStorage](FEntityAllocationIteratorItem Item, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> RootInstanceHandles, TRead<UObject*> BoundObjects)
			{
				ObjectStorage->BeginTrackingEntities(Item, EntityIDs, RootInstanceHandles, BoundObjects);
			}
		);

		FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.FilterAll(IncludeMask)
		.FilterNone(ExcludeMask)
		.Iterate_PerAllocation(&Linker->EntityManager,
			[ObjectStorage](FEntityAllocationIteratorItem Item, TRead<UObject*> Objects)
			{
				FCachePreAnimatedValueParams Params;
				ObjectStorage->CachePreAnimatedValues(Params, Objects.AsArray(Item.GetAllocation()->Num()));
			}
		);
	}
	else if (IPreAnimatedObjectPropertyStorage* PropertyStorage = PreAnimatedStorage->AsPropertyStorage())
	{
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(BuiltInComponents->BoundObject)
		.Read(BuiltInComponents->PropertyBinding)
		.FilterAll(IncludeMask)
		.FilterNone(ExcludeMask)
		.Iterate_PerAllocation(&Linker->EntityManager,
			[PropertyStorage](FEntityAllocationIteratorItem Item, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> RootInstanceHandles, TRead<UObject*> BoundObjects, TRead<FMovieScenePropertyBinding> PropertyBindings)
			{
				PropertyStorage->BeginTrackingEntities(Item, EntityIDs, RootInstanceHandles, BoundObjects, PropertyBindings);
			}
		);

		FEntityTaskBuilder()
		.Read(BuiltInComponents->BoundObject)
		.Read(BuiltInComponents->PropertyBinding)
		.ReadOneOf(BuiltInComponents->CustomPropertyIndex, BuiltInComponents->FastPropertyOffset, BuiltInComponents->SlowProperty)
		.FilterAll(IncludeMask)
		.FilterNone(ExcludeMask)
		.Iterate_PerAllocation(&Linker->EntityManager,
			[PropertyStorage](FEntityAllocationIteratorItem Item, TRead<UObject*> Objects, TRead<FMovieScenePropertyBinding> PropertyBindings, FThreeWayAccessor ResolvedProperties)
			{
				FCachePreAnimatedValueParams Params;
				PropertyStorage->CachePreAnimatedValues(Params, Item, Objects, PropertyBindings, ResolvedProperties);
			}
		);
	}
}

void UMovieScenePropertySystem::RestorePreAnimatedState(const FPreAnimationParameters& InParameters)
{

}

