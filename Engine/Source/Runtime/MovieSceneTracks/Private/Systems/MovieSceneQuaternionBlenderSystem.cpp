// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneQuaternionBlenderSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntityManager.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/MovieSceneBaseValueEvaluatorSystem.h"
#include "Systems/MovieSceneQuaternionInterpolationRotationSystem.h"
#include "Systems/WeightAndEasingEvaluatorSystem.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneQuaternionBlenderSystem)


namespace UE::MovieScene
{

struct FResetBuffers
{
	FQuaternionBlenderAccumulationBuffers* Buffers;

	void Run(FEntityAllocationWriteContext WriteContext) const
	{
		Buffers->AbsoluteWeights.Reset();

		for (int32 Index = 0; Index < Buffers->Absolutes.Num(); ++Index)
		{
			Buffers->Absolutes[Index] = FQuatTransform{
				FVector::ZeroVector,
				FQuat::Identity,
				FVector::ZeroVector
			};
		}
		for (int32 Index = 0; Index < Buffers->Additives.Num(); ++Index)
		{
			Buffers->Additives[Index] = FQuatTransform{ 
				FVector::ZeroVector,
				FQuat::Identity,
				FVector::ZeroVector
			};
		}
	}
};

struct FAccumulateAbsoluteWeights
{
	FQuaternionBlenderAccumulationBuffers* Buffers;

	FAccumulateAbsoluteWeights(FQuaternionBlenderAccumulationBuffers* InBuffers)
		: Buffers(InBuffers)
	{}

	void ForEachEntity(FMovieSceneBlendChannelID BlendChannel, double WeightResult) const
	{
		if (!Buffers->AbsoluteWeights.IsValidIndex(BlendChannel.ChannelID))
		{
			Buffers->AbsoluteWeights.Insert(BlendChannel.ChannelID, WeightResult);
		}
		else
		{
			Buffers->AbsoluteWeights[BlendChannel.ChannelID] += WeightResult;
		}
	}
};


/** Task for accumulating all weighted blend inputs into arrays based on BlendID. Will be run for Absolute, Additive and Relative blend modes*/
struct FAbsoluteAccumulationTask
{
	FQuaternionBlenderAccumulationBuffers* Buffers;

	FAbsoluteAccumulationTask(FQuaternionBlenderAccumulationBuffers* InBuffers)
		: Buffers(InBuffers)
	{}

	void ForEachEntity(
		FMovieSceneBlendChannelID BlendChannel,
		double LocationX, double LocationY, double LocationZ,
		double RotationRoll, double RotationPitch, double RotationYaw,
		double ScaleX, double ScaleY, double ScaleZ,
		const double* WeightResult
		) const
	{
		double Weight = WeightResult ? *WeightResult : 1.0;
		if (Weight == 0.0)
		{
			return;
		}

		const double TotalWeight = Buffers->AbsoluteWeights.IsValidIndex(BlendChannel.ChannelID)
			? Buffers->AbsoluteWeights[BlendChannel.ChannelID]
			: 1.0;

		FQuatTransform& Transform = Buffers->Absolutes[BlendChannel.ChannelID];

		FQuat ThisRotation = FRotator(RotationPitch, RotationYaw, RotationRoll).Quaternion();
		FVector ThisTranslation(LocationX, LocationY, LocationZ);
		FVector ThisScale(ScaleX, ScaleY, ScaleZ);

		const double NormalizedWeight = Weight / TotalWeight;

		Transform.Translation += ThisTranslation * NormalizedWeight;
		Transform.Rotation    *= FQuat::Slerp(FQuat::Identity, ThisRotation, NormalizedWeight);
		Transform.Scale       += ThisScale * NormalizedWeight;
	}
};

struct FAdditiveAccumulationTask
{
	FQuaternionBlenderAccumulationBuffers* Buffers;

	FAdditiveAccumulationTask(FQuaternionBlenderAccumulationBuffers* InBuffers)
		: Buffers(InBuffers)
	{}

	void ForEachEntity(
		FMovieSceneBlendChannelID BlendChannel,
		double LocationX, double LocationY, double LocationZ,
		double RotationRoll, double RotationPitch, double RotationYaw,
		double ScaleX, double ScaleY, double ScaleZ,
		const double* WeightResult
		) const
	{
		FQuatTransform& Transform = Buffers->Additives[BlendChannel.ChannelID];

		const double Weight = WeightResult ? *WeightResult : 1.0;
		Transform.Translation += FVector(LocationX, LocationY, LocationZ) * Weight;
		Transform.Rotation    *= FQuat::Slerp(FQuat::Identity, FRotator(RotationPitch, RotationYaw, RotationRoll).Quaternion(), Weight);
		Transform.Scale       += FVector(ScaleX, ScaleY, ScaleZ) * Weight;
	}
};

/** Task that combines all accumulated blends for any tracked property type that has blend inputs/outputs */
struct FCombineTask
{
	FQuaternionBlenderAccumulationBuffers* Buffers;

	FCombineTask(FQuaternionBlenderAccumulationBuffers* InBuffers)
		: Buffers(InBuffers)
	{}

	void ForEachEntity(
		FMovieSceneBlendChannelID BlendChannel,
		double& LocationX, double& LocationY, double& LocationZ,
		double& RotationRoll, double& RotationPitch, double& RotationYaw,
		double& ScaleX, double& ScaleY, double& ScaleZ,
		const FIntermediate3DTransform* InitialValue
		) const
	{
		const uint16 ChannelID = BlendChannel.ChannelID;
		const double TotalWeight = Buffers->AbsoluteWeights.IsValidIndex(ChannelID) ? Buffers->AbsoluteWeights[ChannelID] : 1.0;

		FVector  ResultTranslation = Buffers->Absolutes[ChannelID].Translation;
		FQuat    ResultRotation    = Buffers->Absolutes[ChannelID].Rotation;
		FVector  ResultScale       = Buffers->Absolutes[ChannelID].Scale;

		if (TotalWeight >= 0.0 && TotalWeight < 1.0)
		{
			if (ensure(InitialValue))
			{
				ResultTranslation = FMath::Lerp(InitialValue->GetTranslation(),            ResultTranslation, TotalWeight);
				ResultRotation    = FQuat::Slerp(InitialValue->GetRotation().Quaternion(), ResultRotation,    TotalWeight);
				ResultScale       = FMath::Lerp(InitialValue->GetScale(),                  ResultScale,       TotalWeight);
			}
		}

 		ResultTranslation += Buffers->Additives[ChannelID].Translation;
		ResultRotation    *= Buffers->Additives[ChannelID].Rotation;
		ResultScale       += Buffers->Additives[ChannelID].Scale;

		FRotator RotatorResult(ResultRotation);

		LocationX = ResultTranslation.X;
		LocationY = ResultTranslation.Y;
		LocationZ = ResultTranslation.Z;
		RotationRoll = RotatorResult.Roll;
		RotationPitch = RotatorResult.Pitch;
		RotationYaw = RotatorResult.Yaw;
		ScaleX = ResultScale.X;
		ScaleY = ResultScale.Y;
		ScaleZ = ResultScale.Z;
	}
};


} // namespace UE::MovieScene

UMovieSceneQuaternionBlenderSystem::UMovieSceneQuaternionBlenderSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SelectionPriority = 1000;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UFloatChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneBaseValueEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneQuaternionInterpolationRotationSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UWeightAndEasingEvaluatorSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneQuaternionBlenderSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	Buffers.Absolutes.SetNumZeroed(AllocatedBlendChannels.Num());
	Buffers.Additives.SetNumZeroed(AllocatedBlendChannels.Num());

	FTaskID ResetBufferTask = TaskScheduler->AddTask<FResetBuffers>(FTaskParams(TEXT("Reset Quaternion Blender Buffers")), &Buffers);

	// Kick off additives first because they don't need normalized weighting
	// Not handling additive from base yet because ideally we could do those in the AdditivesTask without caring about the base
	FTaskID AdditivesTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->BlendChannelInput)
	.ReadAllOf(BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2]) // Translation components
	.ReadAllOf(BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5]) // Rotation components
	.ReadAllOf(BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8]) // Scale components
	.ReadOptional(BuiltInComponents->WeightAndEasingResult)
	.FilterAny({ BuiltInComponents->Tags.AdditiveBlend, BuiltInComponents->Tags.AdditiveFromBaseBlend })
	.FilterAll({ GetBlenderTypeTag() })
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.Schedule_PerEntity<FAdditiveAccumulationTask>(&Linker->EntityManager, TaskScheduler, &Buffers);

	TaskScheduler->AddPrerequisite(ResetBufferTask, AdditivesTask);

	// Accumulate all weights so we can do a normalized lerp.
	// Everything downstream depends on this task
	FTaskID CollectWeightsTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->BlendChannelInput)
	.Read(BuiltInComponents->WeightAndEasingResult)
	.FilterAll({ BuiltInComponents->Tags.AbsoluteBlend, GetBlenderTypeTag() })
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.Schedule_PerEntity<FAccumulateAbsoluteWeights>(&Linker->EntityManager, TaskScheduler, &Buffers);

	TaskScheduler->AddPrerequisite(ResetBufferTask, CollectWeightsTask);

	// Next we accumulate normalized transforms
	FTaskID AbsolutesTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->BlendChannelInput)
	.ReadAllOf(BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2]) // Translation components
	.ReadAllOf(BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5]) // Rotation components
	.ReadAllOf(BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8]) // Scale components
	.ReadOptional(BuiltInComponents->WeightAndEasingResult)
	.FilterAll({ BuiltInComponents->Tags.AbsoluteBlend, GetBlenderTypeTag() })
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.Schedule_PerEntity<FAbsoluteAccumulationTask>(&Linker->EntityManager, TaskScheduler, &Buffers);

	TaskScheduler->AddPrerequisite(CollectWeightsTask, AbsolutesTask);

	// Now blend the results
	FTaskID BlendTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->BlendChannelOutput)
	.WriteAllOf(BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2]) // Translation components
	.WriteAllOf(BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5]) // Rotation components
	.WriteAllOf(BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8]) // Scale components
	.ReadOptional(TracksComponents->ComponentTransform.InitialValue)
	.FilterAll({ GetBlenderTypeTag() })
	.Fork_PerEntity<FCombineTask>(&Linker->EntityManager, TaskScheduler, &Buffers);

	TaskScheduler->AddPrerequisite(AdditivesTask, BlendTask);
	TaskScheduler->AddPrerequisite(CollectWeightsTask, BlendTask);
	TaskScheduler->AddPrerequisite(AbsolutesTask, BlendTask);
}

void UMovieSceneQuaternionBlenderSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	Buffers.Absolutes.SetNumZeroed(AllocatedBlendChannels.Num());
	Buffers.Additives.SetNumZeroed(AllocatedBlendChannels.Num());

	FSystemTaskPrerequisites Prereqs = InPrerequisites;

	// Kick off additives first because they don't need normalized weighting
	// Not handling additive from base yet because ideally we could do those in the AdditivesTask without caring about the base
	FGraphEventRef AdditivesTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->BlendChannelInput)
	.ReadAllOf(BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2]) // Translation components
	.ReadAllOf(BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5]) // Rotation components
	.ReadAllOf(BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8]) // Scale components
	.ReadOptional(BuiltInComponents->WeightAndEasingResult)
	.FilterAny({ BuiltInComponents->Tags.AdditiveBlend, BuiltInComponents->Tags.AdditiveFromBaseBlend })
	.FilterAll({ GetBlenderTypeTag() })
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.template Dispatch_PerEntity<FAdditiveAccumulationTask>(&Linker->EntityManager, InPrerequisites, nullptr, &Buffers);

	// Accumulate all weights so we can do a normalized lerp.
	// Everything downstream depends on this task
	FGraphEventRef CollectWeightsTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->BlendChannelInput)
	.Read(BuiltInComponents->WeightAndEasingResult)
	.FilterAll({ BuiltInComponents->Tags.AbsoluteBlend, GetBlenderTypeTag() })
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.template Dispatch_PerEntity<FAccumulateAbsoluteWeights>(&Linker->EntityManager, InPrerequisites, nullptr, &Buffers);

	if (CollectWeightsTask)
	{
		InPrerequisites.AddRootTask(CollectWeightsTask);
	}

	// Next we accumulate normalized transforms
	FGraphEventRef AbsolutesTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->BlendChannelInput)
	.ReadAllOf(BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2]) // Translation components
	.ReadAllOf(BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5]) // Rotation components
	.ReadAllOf(BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8]) // Scale components
	.ReadOptional(BuiltInComponents->WeightAndEasingResult)
	.FilterAll({ BuiltInComponents->Tags.AbsoluteBlend, GetBlenderTypeTag() })
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.template Dispatch_PerEntity<FAbsoluteAccumulationTask>(&Linker->EntityManager, InPrerequisites, nullptr, &Buffers);

	if (AbsolutesTask)
	{
		InPrerequisites.AddRootTask(AbsolutesTask);
	}
	if (AdditivesTask)
	{
		InPrerequisites.AddRootTask(AdditivesTask);
	}

	// Now blend the results
	FEntityTaskBuilder()
	.Read(BuiltInComponents->BlendChannelOutput)
	.WriteAllOf(BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2]) // Translation components
	.WriteAllOf(BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5]) // Rotation components
	.WriteAllOf(BuiltInComponents->DoubleResult[6], BuiltInComponents->DoubleResult[7], BuiltInComponents->DoubleResult[8]) // Scale components
	.ReadOptional(TracksComponents->ComponentTransform.InitialValue)
	.FilterAll({ GetBlenderTypeTag() })
	.template Dispatch_PerEntity<FCombineTask>(&Linker->EntityManager, InPrerequisites, &Subsequents, &Buffers);
}

FGraphEventRef UMovieSceneQuaternionBlenderSystem::DispatchDecomposeTask(const UE::MovieScene::FValueDecompositionParams& Params, UE::MovieScene::FAlignedDecomposedValue* Output)
{
	return nullptr;
}