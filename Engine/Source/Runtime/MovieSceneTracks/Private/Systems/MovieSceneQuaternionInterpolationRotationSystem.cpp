// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneQuaternionInterpolationRotationSystem.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneQuaternionInterpolationRotationSystem)

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate quat-interp-rot channels"), MovieSceneEval_EvaluateQuatInterpRotChannelTask, STATGROUP_MovieSceneECS);

namespace UE
{
namespace MovieScene
{

struct FEvaluateQuaternionInterpolationRotationChannels
{
	// Find the closest next/previous times to the current time inside a list of keyframe times.
	static void SetFrameRange(const FFrameTime FrameTime, const TArrayView<const FFrameNumber> &Times, TRange<FFrameNumber> &FrameRange)
	{
		int32 Index1, Index2;
		Index2 = 0;
		Index2 = Algo::UpperBound(Times, FrameTime.FrameNumber);
		Index1 = Index2 - 1;
		Index1 = Index1 >= 0 ? Index1 : INDEX_NONE;
		Index2 = Index2 < Times.Num() ? Index2 : INDEX_NONE;
		if (Index1 != INDEX_NONE && Index2 != INDEX_NONE)
		{
			if (Times[Index1] > FrameRange.GetLowerBoundValue()) 
			{
				FrameRange.SetLowerBoundValue(Times[Index1]);
			}
			if (Times[Index2] != FrameTime.FrameNumber && Times[Index2] < FrameRange.GetUpperBoundValue())
			{
				FrameRange.SetUpperBoundValue(Times[Index2]);
			}
		}
	}

	void ForEachAllocation(
			const FEntityAllocation* Allocation,
			TRead<FFrameTime> FrameTimes, 
			TMultiReadOptional<FSourceDoubleChannel, FSourceDoubleChannel, FSourceDoubleChannel> RotChannelAccessors,
			double* OutResultXs, double* OutResultYs, double* OutResultZs) const
	{
		const FSourceDoubleChannel* RotationXs = RotChannelAccessors.Get<0>();
		const FSourceDoubleChannel* RotationYs = RotChannelAccessors.Get<1>();
		const FSourceDoubleChannel* RotationZs = RotChannelAccessors.Get<2>();

		check(
			(OutResultXs != nullptr) == (RotationXs != nullptr) &&
			(OutResultYs != nullptr) == (RotationYs != nullptr) &&
			(OutResultZs != nullptr) == (RotationZs != nullptr) );
		
		const int32 AllocationSize = Allocation->Num();
		for (int32 Index = 0; Index < AllocationSize; ++Index)
		{
			const FFrameTime FrameTime = FrameTimes[Index];

			const FSourceDoubleChannel* RotationX = RotationXs ? &RotationXs[Index] : nullptr;
			const FSourceDoubleChannel* RotationY = RotationYs ? &RotationYs[Index] : nullptr;
			const FSourceDoubleChannel* RotationZ = RotationZs ? &RotationZs[Index] : nullptr;

			// Find the closest keyframes before/after the current time on the 3 rotation channels.
			TRange<FFrameNumber> FrameRange(TNumericLimits<FFrameNumber>::Min(), TNumericLimits<FFrameNumber>::Max());
			if (RotationX)
			{
				SetFrameRange(FrameTime, RotationX->Source->GetTimes(), FrameRange);
			}
			if (RotationY)
			{
				SetFrameRange(FrameTime, RotationY->Source->GetTimes(), FrameRange);
			}
			if (RotationZ)
			{
				SetFrameRange(FrameTime, RotationZ->Source->GetTimes(), FrameRange);
			}

			const FFrameNumber LowerBound = FrameRange.GetLowerBoundValue();
			const FFrameNumber UpperBound = FrameRange.GetUpperBoundValue();
			if (LowerBound != TNumericLimits<FFrameNumber>::Min() && UpperBound != TNumericLimits<FFrameNumber>::Max())
			{
				double Value;
				FVector FirstRot(0.0f, 0.0f, 0.0f);
				FVector SecondRot(0.0f, 0.0f, 0.0f);
				double U = (FrameTime.AsDecimal() - (double) FrameRange.GetLowerBoundValue().Value) /
					double(FrameRange.GetUpperBoundValue().Value - FrameRange.GetLowerBoundValue().Value);
				U = FMath::Clamp(U, 0.0, 1.0);

				if (RotationX)
				{
					if (RotationX->Source->Evaluate(LowerBound, Value))
					{
						FirstRot[0] = Value;
					}
					if (RotationX->Source->Evaluate(UpperBound, Value))
					{
						SecondRot[0] = Value;
					}
				}
				if (RotationY)
				{
					if (RotationY->Source->Evaluate(LowerBound, Value))
					{
						FirstRot[1] = Value;
					}
					if (RotationY->Source->Evaluate(UpperBound, Value))
					{
						SecondRot[1] = Value;
					}
				}
				if (RotationZ)
				{
					if (RotationZ->Source->Evaluate(LowerBound, Value))
					{
						FirstRot[2] = Value;
					}
					if (RotationZ->Source->Evaluate(UpperBound, Value))
					{
						SecondRot[2] = Value;
					}
				}

				const FQuat Key1Quat = FQuat::MakeFromEuler(FirstRot);
				const FQuat Key2Quat = FQuat::MakeFromEuler(SecondRot);

				const FQuat SlerpQuat = FQuat::Slerp(Key1Quat, Key2Quat, U);
				FVector Euler = FRotator(SlerpQuat).Euler();
				if (RotationX)
				{
					OutResultXs[Index] = Euler[0];
				}
				if (RotationY)
				{
					OutResultYs[Index] = Euler[1];
				}
				if (RotationZ)
				{
					OutResultZs[Index] = Euler[2];
				}
			}
			else  // no range found: default to regular, but still do RotToQuat
			{
				double Value;
				FVector CurrentRot(0.0f, 0.0f, 0.0f);
				if (RotationX && RotationX->Source->Evaluate(FrameTime, Value))
				{
					CurrentRot[0] = Value;
				}
				if (RotationY && RotationY->Source->Evaluate(FrameTime, Value))
				{
					CurrentRot[1] = Value;
				}
				if (RotationZ && RotationZ->Source->Evaluate(FrameTime, Value))
				{
					CurrentRot[2] = Value;
				}
				FQuat Quat = FQuat::MakeFromEuler(CurrentRot);
				FVector Euler = FRotator(Quat).Euler();
				if (RotationX)
				{
					OutResultXs[Index] = Euler[0];
				}
				if (RotationY)
				{
					OutResultYs[Index] = Euler[1];
				}
				if (RotationZ)
				{
					OutResultZs[Index] = Euler[2];
				}
			}
		}
	}
};

} // namespace MovieScene
} // namespace UE

UMovieSceneQuaternionInterpolationRotationSystem::UMovieSceneQuaternionInterpolationRotationSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());

		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		DefineComponentProducer(GetClass(), BuiltInComponents->DoubleResult[3]);
		DefineComponentProducer(GetClass(), BuiltInComponents->DoubleResult[4]);
		DefineComponentProducer(GetClass(), BuiltInComponents->DoubleResult[5]);
	}
}

bool UMovieSceneQuaternionInterpolationRotationSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	return InLinker->EntityManager.ContainsAnyComponent({ 
			TrackComponents->QuaternionRotationChannel[0],
			TrackComponents->QuaternionRotationChannel[1],
			TrackComponents->QuaternionRotationChannel[2]
		});
}

void UMovieSceneQuaternionInterpolationRotationSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->EvalTime)
	.ReadOneOrMoreOf(
			TrackComponents->QuaternionRotationChannel[0],
			TrackComponents->QuaternionRotationChannel[1],
			TrackComponents->QuaternionRotationChannel[2])
	.WriteOptional(BuiltInComponents->DoubleResult[3])
	.WriteOptional(BuiltInComponents->DoubleResult[4])
	.WriteOptional(BuiltInComponents->DoubleResult[5])
	.SetStat(GET_STATID(MovieSceneEval_EvaluateQuatInterpRotChannelTask))
	.Fork_PerAllocation<FEvaluateQuaternionInterpolationRotationChannels>(&Linker->EntityManager, TaskScheduler);
}

void UMovieSceneQuaternionInterpolationRotationSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityTaskBuilder()
		.Read(BuiltInComponents->EvalTime)
		.ReadOneOrMoreOf(
				TrackComponents->QuaternionRotationChannel[0],
				TrackComponents->QuaternionRotationChannel[1],
				TrackComponents->QuaternionRotationChannel[2])
		.WriteOptional(BuiltInComponents->DoubleResult[3])
		.WriteOptional(BuiltInComponents->DoubleResult[4])
		.WriteOptional(BuiltInComponents->DoubleResult[5])
		.SetStat(GET_STATID(MovieSceneEval_EvaluateQuatInterpRotChannelTask))
		.Dispatch_PerAllocation<FEvaluateQuaternionInterpolationRotationChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents);
}

