// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneRotatorPropertySystem.h"

#include "MovieSceneTracksComponentTypes.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneRotatorPropertySystem)

UMovieSceneRotatorPropertySystem::UMovieSceneRotatorPropertySystem(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Rotator);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneRotatorPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& InSubsequents)
{
	Super::OnRun(InPrerequisites, InSubsequents);
}
