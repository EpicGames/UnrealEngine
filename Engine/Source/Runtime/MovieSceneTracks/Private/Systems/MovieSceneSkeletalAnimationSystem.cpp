// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneSkeletalAnimationSystem.h"

#include "Decorations/MovieSceneScalingAnchors.h"
#include "Async/TaskGraphInterfaces.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntityGroupingSystem.h"
#include "IMovieScenePlayer.h"
#include "MovieScene.h"
#include "MovieSceneExecutionToken.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"

#include "Evaluation/PreAnimatedState/IMovieScenePreAnimatedStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

#include "AnimCustomInstanceHelper.h"
#include "AnimSequencerInstance.h"
#include "AnimSequencerInstanceProxy.h"
#include "Animation/AnimMontage.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Animation/AnimSingleNodeInstance.h"
#include "Components/SkeletalMeshComponent.h"
#include "Engine/SkeletalMesh.h"
#include "SkeletalMeshRestoreState.h"

#include "Rendering/MotionVectorSimulation.h"
#include "Systems/MovieSceneMotionVectorSimulationSystem.h"
#include "Systems/MovieSceneComponentTransformSystem.h"
#include "Systems/MovieSceneQuaternionInterpolationRotationSystem.h"
#include "Systems/WeightAndEasingEvaluatorSystem.h"
#include "Systems/MovieSceneObjectPropertySystem.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"

#include "Components/SkeletalMeshComponent.h"
#include "Animation/AnimationPoseData.h"
#include "BonePose.h"
#include "Animation/BuiltInAttributeTypes.h"

#include "SequencerAnimationOverride.h"

#if WITH_EDITOR
#include "Editor.h"
#include "Editor/UnrealEdEngine.h"
#include "UnrealEdGlobals.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSkeletalAnimationSystem)

DECLARE_CYCLE_STAT(TEXT("Gather skeletal animations"), MovieSceneEval_GatherSkeletalAnimations, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Evaluate skeletal animations"), MovieSceneEval_EvaluateSkeletalAnimations, STATGROUP_MovieSceneECS);

namespace UE::MovieScene
{

MOVIESCENETRACKS_API bool (*ShouldUseLegacyControlRigTemplate)() = nullptr;

bool GAnimationUIFlickerFix = false;
bool AnimationUIFlickerFixActive()
{
	 return GAnimationUIFlickerFix == true || (ShouldUseLegacyControlRigTemplate && !ShouldUseLegacyControlRigTemplate());
}
FAutoConsoleVariableRef CVarAnimationUIFlickerFix(
	TEXT("Sequencer.Animation.UIFlickerFix"),
	GAnimationUIFlickerFix,
	TEXT("(Default: true. Fixes pre-animated state ordering that was causing excessive UI flicker. Known to cause issues when animating Anim Class so should be disabled if a crash is encountered.")
	);

/** Helper function to get our sequencer animation node from a skeletal mesh component */
UAnimSequencerInstance* GetAnimSequencerInstance(USkeletalMeshComponent* SkeletalMeshComponent)
{
	ISequencerAnimationSupport* SeqInterface = Cast<ISequencerAnimationSupport>(SkeletalMeshComponent->GetAnimInstance());
	if (SeqInterface)
	{
		return Cast<UAnimSequencerInstance>(SeqInterface->GetSourceAnimInstance());
	}

	return nullptr;
}

/** ------------------------------------------------------------------------- */

/** Pre-animated state for skeletal animations */
struct FPreAnimatedSkeletalAnimationState
{
	EAnimationMode::Type AnimationMode;
	TWeakObjectPtr<UAnimInstance> CachedAnimInstance;
	FSkeletalMeshRestoreState SkeletalMeshRestoreState;
};

/** Pre-animation traits for skeletal animations */
struct FPreAnimatedSkeletalAnimationTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType     = FObjectKey;
	using StorageType = FPreAnimatedSkeletalAnimationState;

	FPreAnimatedSkeletalAnimationState CachePreAnimatedValue(const KeyType& Object)
	{
		FPreAnimatedSkeletalAnimationState OutCachedValue;
		USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(Object.ResolveObjectPtr());
		if (ensure(Component))
		{
			OutCachedValue.AnimationMode = Component->GetAnimationMode();
			OutCachedValue.CachedAnimInstance = Component->AnimScriptInstance;
			OutCachedValue.SkeletalMeshRestoreState.SaveState(Component);
		}
		return OutCachedValue;
	}

	void RestorePreAnimatedValue(const KeyType& Object, StorageType& InOutCachedValue, const FRestoreStateParams& Params)
	{
		USkeletalMeshComponent* Component = Cast<USkeletalMeshComponent>(Object.ResolveObjectPtr());
		if (!Component || !Component->IsRegistered())
		{
			return;
		}

		ISequencerAnimationSupport* SequencerInst = Cast<ISequencerAnimationSupport>(GetAnimSequencerInstance(Component));
		if (SequencerInst)
		{
			SequencerInst->ResetPose();
			SequencerInst->ResetNodes();
		}

		FAnimCustomInstanceHelper::UnbindFromSkeletalMeshComponent<UAnimSequencerInstance>(Component);

		// Restore LOD before reinitializing anim instance
		InOutCachedValue.SkeletalMeshRestoreState.RestoreLOD();

		if (Component->GetAnimationMode() != InOutCachedValue.AnimationMode)
		{
			// this SetAnimationMode reinitializes even if the mode is same
			// if we're using same anim blueprint, we don't want to keep reinitializing it. 
			Component->SetAnimationMode(InOutCachedValue.AnimationMode);
		}

		TStrongObjectPtr<UAnimInstance> PreviousAnimInstance = InOutCachedValue.CachedAnimInstance.Pin();
		if (PreviousAnimInstance && IsValid(PreviousAnimInstance.Get()) && PreviousAnimInstance->GetSkelMeshComponent() == Component)
		{
			bool bNeedsReInit = false;

			if (Component->AnimScriptInstance != PreviousAnimInstance.Get())
			{
				Component->AnimScriptInstance = PreviousAnimInstance.Get();
				bNeedsReInit = true;
			}

			InOutCachedValue.CachedAnimInstance.Reset();

			if (Component->AnimScriptInstance)
			{
				//the skeleton may have changed so need to recalc required bones as needed.
				if (Component->GetSkeletalMeshAsset() && Component->AnimScriptInstance->CurrentSkeleton != Component->GetSkeletalMeshAsset()->GetSkeleton())
				{
					Component->AnimScriptInstance->CurrentSkeleton = Component->GetSkeletalMeshAsset()->GetSkeleton();
					bNeedsReInit = true;
				}
			}

			if (bNeedsReInit)
			{
				// When Changing anim instances or skeletons, RecalcRequiredBones and UpdateMorphTargets need to be called, and InitializeAnimation needs to be called regardless, when changing script instances
				// Playing montages does not work on uninitialized anim script instances with the EarlyOutMontageWhenUninitialized cvar enabled.
				Component->InitializeAnimScriptInstance(true);
			}
		}

		// Restore pose after unbinding to force the restored pose
		Component->SetUpdateAnimationInEditor(true);
		Component->SetUpdateClothInEditor(true);
		if (!Component->IsPostEvaluatingAnimation())
		{
			Component->TickAnimation(0.f, false);
			Component->RefreshBoneTransforms();
			Component->RefreshFollowerComponents();
			Component->UpdateComponentToWorld();
			Component->FinalizeBoneTransform();
			Component->MarkRenderTransformDirty();
			Component->MarkRenderDynamicDataDirty();
		}

		// Reset the mesh component update flag and animation mode to what they were before we animated the object
		InOutCachedValue.SkeletalMeshRestoreState.RestoreState();

		// if not game world, don't clean this up
		if (Component->GetWorld() != nullptr && Component->GetWorld()->IsGameWorld() == false)
		{
			Component->ClearMotionVector();
		}
	}
};

/** Pre-animation storage for skeletal animations */
struct FPreAnimatedSkeletalAnimationStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedSkeletalAnimationTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedSkeletalAnimationStorage> StorageID;
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedSkeletalAnimationStorage> FPreAnimatedSkeletalAnimationStorage::StorageID;

/** ------------------------------------------------------------------------- */

/** Pre-animated state for a sequencer montage node */
struct FPreAnimatedSkeletalAnimationMontageState
{
	TWeakObjectPtr<UAnimInstance> WeakInstance;
	int32 MontageInstanceId;
};

/** Pre-animated traits for a sequencer montage node */
struct FPreAnimatedSkeletalAnimationMontageTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType     = FObjectKey;
	using StorageType = FPreAnimatedSkeletalAnimationMontageState;

	FPreAnimatedSkeletalAnimationMontageState CachePreAnimatedValue(const FObjectKey& Object)
	{
		// Should be unused, as we always cache state with captured values.
		return FPreAnimatedSkeletalAnimationMontageState();
	}

	void RestorePreAnimatedValue(const FObjectKey& Object, FPreAnimatedSkeletalAnimationMontageState& InOutCachedValue, const FRestoreStateParams& Params)
	{
		UAnimInstance* AnimInstance = InOutCachedValue.WeakInstance.Get();
		if (AnimInstance)
		{
			FAnimMontageInstance* MontageInstance = AnimInstance->GetMontageInstanceForID(InOutCachedValue.MontageInstanceId);
			if (MontageInstance)
			{
				MontageInstance->Stop(FAlphaBlend(0.f), false);
			}
		}
	}
};

/** Pre-animated storage for a sequencer montage node */
struct FPreAnimatedSkeletalAnimationMontageStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedSkeletalAnimationMontageTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedSkeletalAnimationMontageStorage> StorageID;
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedSkeletalAnimationMontageStorage> FPreAnimatedSkeletalAnimationMontageStorage::StorageID;

/** ------------------------------------------------------------------------- */

/** Pre-animated traits for a sequencer animation node */
struct FPreAnimatedSkeletalAnimationAnimInstanceTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType     = FObjectKey;
	using StorageType = bool; // We actually don't need any state, so this will be a dummy value

	bool CachePreAnimatedValue(const FObjectKey& Object)
	{
		// Nothing to do, we just need the object pointer to restore state.
		return true;
	}

	void RestorePreAnimatedValue(const FObjectKey& Object, bool& Unused, const FRestoreStateParams& Params)
	{
		if (UObject* ObjectPtr = Object.ResolveObjectPtr())
		{
			ISequencerAnimationSupport* SequencerAnimationSupport = Cast<ISequencerAnimationSupport>(ObjectPtr);
			if (ensure(SequencerAnimationSupport))
			{
				SequencerAnimationSupport->ResetNodes();
			}
		}
	}
};

/** Pre-animated storage for a sequencer animation node */
struct FPreAnimatedSkeletalAnimationAnimInstanceStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedSkeletalAnimationAnimInstanceTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedSkeletalAnimationAnimInstanceStorage> StorageID;
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedSkeletalAnimationAnimInstanceStorage> FPreAnimatedSkeletalAnimationAnimInstanceStorage::StorageID;

/** ------------------------------------------------------------------------- */

/** FBoneTransformFinalizeData used for setting up root motion*/

FBoneTransformFinalizeData::FBoneTransformFinalizeData() : SkeletalMeshComponent(nullptr), SwapRootBone(ESwapRootBone::SwapRootBone_None)
{
}

void FBoneTransformFinalizeData::Register(USkeletalMeshComponent* InSkeleletalMeshCompononent, ESwapRootBone InSwapRootBone, FTransform& InMeshRelativeRootMotionTransform,TOptional<FTransform> InInitialActorTransform)
{
	if (OnBoneTransformsFinalizedHandle.IsValid() == false)
	{
		SkeletalMeshComponent = InSkeleletalMeshCompononent;
		SwapRootBone = InSwapRootBone;
		MeshRelativeRootMotionTransform = InMeshRelativeRootMotionTransform;
		InitialActorTransform = InInitialActorTransform;
		// Also store the inverse relative rotation between SkeletalMeshComponent and the Actor if it's not the root component
		if (SkeletalMeshComponent.IsValid() && SkeletalMeshComponent.Get() != SkeletalMeshComponent->GetOwner()->GetRootComponent())
		{
			InverseMeshToActorRotation = SkeletalMeshComponent->GetOwner()->GetRootComponent()->GetComponentTransform().GetRelativeTransformReverse(SkeletalMeshComponent->GetComponentTransform()).GetRotation();
		}

		OnBoneTransformsFinalizedHandle = SkeletalMeshComponent->RegisterOnBoneTransformsFinalizedDelegate(
			FOnBoneTransformsFinalizedMultiCast::FDelegate::CreateLambda([this]()
				{
					BoneTransformFinalized();
				}));
	}

#if WITH_EDITOR
	if (OnBeginActorMovementHandle.IsValid() == false)
	{
		OnBeginActorMovementHandle = GEditor->OnBeginObjectMovement().Add(UEditorEngine::FOnBeginTransformObject::FDelegate::CreateLambda([this](UObject&)
				{
					bActorBeingMoved = true;
				}));
	}

	if (OnEndActorMovementHandle.IsValid() == false)
	{
		OnEndActorMovementHandle = GEditor->OnEndObjectMovement().Add(UEditorEngine::FOnEndTransformObject::FDelegate::CreateLambda([this](UObject&)
			{
				bActorBeingMoved = false;
			}));
	}
#endif
}

void FBoneTransformFinalizeData::Unregister()
{
	if (SkeletalMeshComponent.IsValid())
	{
		SkeletalMeshComponent->UnregisterOnBoneTransformsFinalizedDelegate(OnBoneTransformsFinalizedHandle);
	}
	OnBoneTransformsFinalizedHandle.Reset();
	InitialActorTransform.Reset();
	InverseMeshToActorRotation.Reset();

#if WITH_EDITOR
	GEditor->OnBeginObjectMovement().Remove(OnBeginActorMovementHandle);
	OnBeginActorMovementHandle.Reset();
	GEditor->OnEndObjectMovement().Remove(OnEndActorMovementHandle);
	OnEndActorMovementHandle.Reset();
#endif
}

void FBoneTransformFinalizeData::BoneTransformFinalized()
{
#if WITH_EDITOR
	if (bActorBeingMoved)
	{
		return;
	}
#endif

	if (SkeletalMeshComponent.IsValid() && SwapRootBone != ESwapRootBone::SwapRootBone_None)
	{
		FTransform RelativeTransform = MeshRelativeRootMotionTransform;

		if (InitialActorTransform.IsSet())
		{
			RelativeTransform = RelativeTransform * InitialActorTransform.GetValue();
		}

		if (SwapRootBone == ESwapRootBone::SwapRootBone_Component)
		{
			SkeletalMeshComponent->SetRelativeLocationAndRotation(RelativeTransform.GetLocation(), RelativeTransform.GetRotation().Rotator());
		}
		else if (SwapRootBone == ESwapRootBone::SwapRootBone_Actor)
		{
			AActor* Actor = SkeletalMeshComponent->GetOwner();
			if (Actor && Actor->GetRootComponent())
			{
				// Compensate for any mesh component rotation
				if (InitialActorTransform.IsSet() && InverseMeshToActorRotation.IsSet())
				{
					RelativeTransform = MeshRelativeRootMotionTransform;
					RelativeTransform.SetTranslation(InverseMeshToActorRotation.GetValue() * RelativeTransform.GetTranslation());
					RelativeTransform = RelativeTransform * InitialActorTransform.GetValue();
				}

				Actor->GetRootComponent()->SetRelativeLocationAndRotation(RelativeTransform.GetLocation(), RelativeTransform.GetRotation().Rotator());
			}
		}
	}
}

void FSkeletalAnimationSystemData::ResetSkeletalAnimations()
{
	//clear out the delegates
	for (TTuple<TWeakObjectPtr<USkeletalMeshComponent>, FBoundObjectActiveSkeletalAnimations>& Pair : SkeletalAnimations)
	{
		Pair.Value.BoneTransformFinalizeData.Unregister();
	}
	SkeletalAnimations.Reset();
}

/** ------------------------------------------------------------------------- */
/** Task for gathering active skeletal animations */
struct FGatherSkeletalAnimations
{
	const FInstanceRegistry* InstanceRegistry;
	FSkeletalAnimationSystemData* SystemData;

	FGatherSkeletalAnimations(const FInstanceRegistry* InInstanceRegistry, FSkeletalAnimationSystemData* InSystemData)
		: InstanceRegistry(InInstanceRegistry)
		, SystemData(InSystemData)
	{
	}

	void PreTask() const
	{
		// Start fresh every frame, gathering all active skeletal animations.
		SystemData->ResetSkeletalAnimations();
	}

	void ForEachAllocation(
			const FEntityAllocationProxy AllocationProxy, 
			TRead<FMovieSceneEntityID> EntityIDs,
			TRead<FRootInstanceHandle> RootInstanceHandles,
			TRead<FInstanceHandle> InstanceHandles,
			TRead<UObject*> BoundObjects,
			TRead<FMovieSceneSkeletalAnimationComponentData> SkeletalAnimations,
			TReadOptional<FFrameTime> OptionalEvalTimes,
			TReadOptional<double> WeightAndEasings) const
	{
		// Gather all the skeletal animations currently active in all sequences.
		// We map these animations to their bound object, which means we might blend animations from different sequences
		// that have bound to the same object.
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		const FEntityAllocation* Allocation = AllocationProxy.GetAllocation();
		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			FMovieSceneEntityID EntityID(EntityIDs[Index]);
			const FRootInstanceHandle& RootInstanceHandle(RootInstanceHandles[Index]);
			const FInstanceHandle& InstanceHandle(InstanceHandles[Index]);
			UObject* BoundObject(BoundObjects[Index]);
			const FMovieSceneSkeletalAnimationComponentData& SkeletalAnimation(SkeletalAnimations[Index]);
			const double Weight = (WeightAndEasings ? WeightAndEasings[Index] : 1.f);

			const bool bWantsRestoreState = AllocationProxy.GetAllocationType().Contains(BuiltInComponents->Tags.RestoreState);

			// Get the full context, so we can get both the current and previous evaluation times.
			const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InstanceHandle);
			const FMovieSceneContext& Context = SequenceInstance.GetContext();

			// Calculate the time at which to evaluate the animation
			const UMovieSceneSkeletalAnimationSection* AnimSection = SkeletalAnimation.Section;
			const FMovieSceneSkeletalAnimationParams& AnimParams = AnimSection->Params;

			// Get the bound skeletal mesh component.
			USkeletalMeshComponent* SkeletalMeshComponent = CastChecked<USkeletalMeshComponent>(BoundObject);
			UAnimSequenceBase* AnimSequence = AnimSection->GetPlaybackAnimation();
			if (!SkeletalMeshComponent || AnimSequence == nullptr)
			{
				continue;
			}

			FFrameTime EvalFrameTime = OptionalEvalTimes ? OptionalEvalTimes[Index] : Context.GetTime();
			FFrameTime PreviousEvalFrameTime = Context.GetPreviousTime();

			FFrameNumber SectionStartTime = AnimSection->GetInclusiveStartFrame();
			FFrameNumber SectionEndTime   = AnimSection->GetExclusiveEndFrame();

			if (AnimParams.bLinearPlaybackWhenScaled)
			{
				UMovieSceneScalingAnchors* ScalingAnchors = AnimSection->GetTypedOuter<UMovieScene>()->FindDecoration<UMovieSceneScalingAnchors>();

				if (ScalingAnchors)
				{
					TOptional<FFrameTime> UnwarpedTime = ScalingAnchors->InverseRemapTimeCycled(EvalFrameTime, EvalFrameTime, FInverseTransformTimeParams());
					if (UnwarpedTime.IsSet())
					{
						EvalFrameTime = UnwarpedTime.GetValue();
					}
					TOptional<FFrameTime> PreviousUnwarpedTime = ScalingAnchors->InverseRemapTimeCycled(PreviousEvalFrameTime, PreviousEvalFrameTime, FInverseTransformTimeParams());
					if (PreviousUnwarpedTime.IsSet())
					{
						PreviousEvalFrameTime = PreviousUnwarpedTime.GetValue();
					}
					TOptional<FFrameTime> UnwarpedStartTime = ScalingAnchors->InverseRemapTimeCycled(SectionStartTime, SectionStartTime, FInverseTransformTimeParams());
					if (UnwarpedStartTime.IsSet())
					{
						SectionStartTime = UnwarpedStartTime.GetValue().RoundToFrame();
					}
					TOptional<FFrameTime> UnwarpedEndTime = ScalingAnchors->InverseRemapTimeCycled(SectionEndTime, SectionEndTime, FInverseTransformTimeParams());
					if (UnwarpedEndTime.IsSet())
					{
						SectionEndTime = UnwarpedEndTime.GetValue().RoundToFrame();
					}
				}
			}

			const float EvalTime = AnimParams.MapTimeToAnimation(SectionStartTime, SectionEndTime, EvalFrameTime, Context.GetFrameRate(), AnimSequence);
			const float PreviousEvalTime = AnimParams.MapTimeToAnimation(SectionStartTime, SectionEndTime, PreviousEvalFrameTime, Context.GetFrameRate(), AnimSequence);

			const FSequenceInstance& RootInstance = InstanceRegistry->GetInstance(RootInstanceHandle);
			const FMovieSceneContext& RootContext = RootInstance.GetContext();
			const double RootDeltaTime = (RootContext.HasJumped() ? FFrameTime(0) : RootContext.GetRange().Size<FFrameTime>() ) / RootContext.GetFrameRate();

			const EMovieScenePlayerStatus::Type PlayerStatus = Context.GetStatus();

			const bool bResetDynamics = PlayerStatus == EMovieScenePlayerStatus::Stepping || 
				PlayerStatus == EMovieScenePlayerStatus::Jumping || 
				PlayerStatus == EMovieScenePlayerStatus::Scrubbing || 
				(RootDeltaTime == 0.0f && PlayerStatus != EMovieScenePlayerStatus::Stopped);

			const bool bPreviewPlayback = ShouldUsePreviewPlayback(PlayerStatus, *BoundObject);

			// If the playback status is jumping, ie. one such occurrence is setting the time for thumbnail generation, disable anim notifies updates because it could fire audio.
			// If the playback status is scrubbing, we disable notifies for now because we can't properly fire them in all cases until we get evaluation range info.
			// We now layer this with the passed in notify toggle to force a disable in this case.
			const bool bFireNotifies = !bPreviewPlayback || (PlayerStatus != EMovieScenePlayerStatus::Jumping && PlayerStatus != EMovieScenePlayerStatus::Stopped && PlayerStatus != EMovieScenePlayerStatus::Scrubbing);
			const bool bPlaying = PlayerStatus == EMovieScenePlayerStatus::Playing;

			FBoundObjectActiveSkeletalAnimations& BoundObjectAnimations = SystemData->SkeletalAnimations.FindOrAdd(SkeletalMeshComponent);

			FActiveSkeletalAnimation Animation;
			Animation.AnimSection        = AnimSection;
			Animation.Context            = Context;
			Animation.EvalFrameTime      = EvalFrameTime;
			Animation.EntityID           = EntityID;
			Animation.RootInstanceHandle = RootInstanceHandle;
			Animation.FromEvalTime       = PreviousEvalTime;
			Animation.ToEvalTime         = EvalTime;
			Animation.BlendWeight        = Weight;
			Animation.PlayerStatus       = PlayerStatus;
			Animation.bFireNotifies      = bFireNotifies;
			Animation.bPlaying           = bPlaying;
			Animation.bResetDynamics     = bResetDynamics;
			Animation.bWantsRestoreState = bWantsRestoreState;
			Animation.bPreviewPlayback   = bPreviewPlayback;

			BoundObjectAnimations.Animations.Add(Animation);

			if (FMotionVectorSimulation::IsEnabled())
			{
				const FFrameTime SimulatedTime = UE::MovieScene::GetSimulatedMotionVectorTime(Context);

				// Calculate the time at which to evaluate the animation
				const float SimulatedEvalTime = AnimParams.MapTimeToAnimation(AnimSection, SimulatedTime, Context.GetFrameRate());

				// Evaluate the weight channel and section easing at the simulation time... right now we don't benefit
				// from that being evaluated by the channel evaluators.
				float SimulatedManualWeight = 1.f;
				AnimParams.Weight.Evaluate(SimulatedTime, SimulatedManualWeight);

				const float SimulatedWeight = SimulatedManualWeight * AnimSection->EvaluateEasing(SimulatedTime);

				Animation.BlendWeight = SimulatedWeight;
				Animation.FromEvalTime = EvalTime;
				Animation.ToEvalTime = SimulatedEvalTime;
				BoundObjectAnimations.SimulatedAnimations.Add(Animation);
			}
		}
	}

private:

	static bool ShouldUsePreviewPlayback(EMovieScenePlayerStatus::Type PlayerStatus, UObject& RuntimeObject)
	{
		// We also use PreviewSetAnimPosition in PIE when not playing, as we can preview in PIE.
		bool bIsNotInPIEOrNotPlaying = (RuntimeObject.GetWorld() && !RuntimeObject.GetWorld()->HasBegunPlay()) || PlayerStatus != EMovieScenePlayerStatus::Playing;
		return GIsEditor && bIsNotInPIEOrNotPlaying;
	}
};

/** Task for evaluating skeletal animations */
struct FEvaluateSkeletalAnimations
{
private:

	UMovieSceneEntitySystemLinker* Linker;
	FSkeletalAnimationSystemData* SystemData;

	TSharedPtr<FPreAnimatedSkeletalAnimationStorage> PreAnimatedStorage;
	TSharedPtr<FPreAnimatedSkeletalAnimationMontageStorage> PreAnimatedMontageStorage;
	TSharedPtr<FPreAnimatedSkeletalAnimationAnimInstanceStorage> PreAnimatedAnimInstanceStorage;

public:

	FEvaluateSkeletalAnimations(UMovieSceneEntitySystemLinker* InLinker, FSkeletalAnimationSystemData* InSystemData)
		: Linker(InLinker)
		, SystemData(InSystemData)
	{
		PreAnimatedStorage = InLinker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedSkeletalAnimationStorage>();
		PreAnimatedMontageStorage = InLinker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedSkeletalAnimationMontageStorage>();
		PreAnimatedAnimInstanceStorage = InLinker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedSkeletalAnimationAnimInstanceStorage>();
	}

	void Run(FEntityAllocationWriteContext WriteContext) const
	{
		Run();
	}
	void Run() const
	{
		for (const TTuple<TWeakObjectPtr<USkeletalMeshComponent>, FBoundObjectActiveSkeletalAnimations>& Pair : SystemData->SkeletalAnimations)
		{
			if (Pair.Key.IsValid())
			{
				EvaluateSkeletalAnimations(Pair.Key.Get(), Pair.Value);
			}
		}
	}

private:

	void EvaluateSkeletalAnimations(USkeletalMeshComponent* SkeletalMeshComponent, const FBoundObjectActiveSkeletalAnimations& InSkeletalAnimations) const
	{
		ensureMsgf(SkeletalMeshComponent, TEXT("Attempting to evaluate an Animation track with a null object."));

		if (!SkeletalMeshComponent || !SkeletalMeshComponent->GetSkeletalMeshAsset())
		{
			return;
		}

		if (!AnimationUIFlickerFixActive())
		{
			// Cache pre-animated state for this bound object before doing anything.
			// We don't yet track what entities have already started animated vs. entities that just started this frame,
			// so we just process all the currently active ones. If they are already tracked and have already had their
			// pre-animated state saved, it these calls will just early return.
			for (const FActiveSkeletalAnimation& SkeletalAnimation : InSkeletalAnimations.Animations)
			{
				PreAnimatedStorage->BeginTrackingEntity(SkeletalAnimation.EntityID, SkeletalAnimation.bWantsRestoreState, SkeletalAnimation.RootInstanceHandle, SkeletalMeshComponent);
			}
			FCachePreAnimatedValueParams CacheParams;
			PreAnimatedStorage->CachePreAnimatedValue(CacheParams, SkeletalMeshComponent);
		}

		// Check to see if we need to switch modes on the anim instance
		if (Algo::AnyOf(InSkeletalAnimations.Animations, [&SkeletalMeshComponent](const FActiveSkeletalAnimation& ActiveAnim)
			{
				UAnimSequenceBase* Animation = ActiveAnim.AnimSection->GetPlaybackAnimation();
				if (CanPlayAnimation(SkeletalMeshComponent, Animation))
				{
					const FMovieSceneSkeletalAnimationParams& AnimParams = ActiveAnim.AnimSection->Params;
					TScriptInterface<ISequencerAnimationOverride> SequencerAnimOverride = ISequencerAnimationOverride::GetSequencerAnimOverride(SkeletalMeshComponent);
					if (AnimParams.bForceCustomMode || (SequencerAnimOverride.GetObject() && ISequencerAnimationOverride::Execute_AllowsCinematicOverride(SequencerAnimOverride.GetObject())))
					{
						return true;
					}
				}
				return false;
			}))
		{
			SkeletalMeshComponent->SetAnimationMode(EAnimationMode::AnimationCustomMode);
		}

		// Setup any needed animation nodes for sequencer playback.
		UAnimInstance* ExistingAnimInstance = GetSourceAnimInstance(SkeletalMeshComponent);
		bool bWasCreated = false;
		ISequencerAnimationSupport* SequencerInstance = FAnimCustomInstanceHelper::BindToSkeletalMeshComponent<UAnimSequencerInstance>(SkeletalMeshComponent, bWasCreated);
		if (SequencerInstance)
		{
			if (bWasCreated)
			{
				SequencerInstance->SavePose();
			}
			else
			{
				SequencerInstance->ConstructNodes();
			}
		}

		// Need to zero all weights first since we may be blending animation that are keeping state but are no longer active.
		if (SequencerInstance)
		{
			SequencerInstance->ResetNodes();
		}
		else if (ExistingAnimInstance)
		{
			for (const TPair<FObjectKey, FMontagePlayerPerSectionData >& Pair : SystemData->MontageData.FindOrAdd(SkeletalMeshComponent))
			{
				int32 InstanceId = Pair.Value.MontageInstanceId;
				FAnimMontageInstance* MontageInstanceToUpdate = ExistingAnimInstance->GetMontageInstanceForID(InstanceId);
				if (MontageInstanceToUpdate)
				{
					MontageInstanceToUpdate->SetDesiredWeight(0.0f);
					MontageInstanceToUpdate->SetWeight(0.0f);
				}
			}
		}

		if (InSkeletalAnimations.SimulatedAnimations.Num() != 0)
		{
			UMovieSceneMotionVectorSimulationSystem* MotionVectorSim = Linker->FindSystem<UMovieSceneMotionVectorSimulationSystem>();
			if (MotionVectorSim && MotionVectorSim->IsSimulationEnabled())
			{
				ApplyAnimations(SkeletalMeshComponent, InSkeletalAnimations.SimulatedAnimations);
				SkeletalMeshComponent->TickAnimation(0.f, false);
				SkeletalMeshComponent->ForceMotionVector();
		
				SimulateMotionVectors(SkeletalMeshComponent, MotionVectorSim);
			}
		}

		ApplyAnimations(SkeletalMeshComponent, InSkeletalAnimations.Animations);

		// If the skeletal component has already ticked this frame because tick prerequisites weren't set up yet or a new binding was created, forcibly tick this component to update.
		// This resolves first frame issues where the skeletal component ticks first, then the sequencer binding is resolved which sets up tick prerequisites
		// for the next frame.
		if (!SkeletalMeshComponent->IsPostEvaluatingAnimation() &&
			(SkeletalMeshComponent->PoseTickedThisFrame() || (SequencerInstance && SequencerInstance->GetSourceAnimInstance() != ExistingAnimInstance))
			)
		{
			SkeletalMeshComponent->HandleExistingParallelEvaluationTask(true, true);
			SkeletalMeshComponent->TickAnimation(0.f, false);

			SkeletalMeshComponent->RefreshBoneTransforms();
			SkeletalMeshComponent->RefreshFollowerComponents();
			SkeletalMeshComponent->UpdateComponentToWorld();
			SkeletalMeshComponent->FinalizeBoneTransform();
			SkeletalMeshComponent->MarkRenderTransformDirty();
			SkeletalMeshComponent->MarkRenderDynamicDataDirty();
		}
	}

private:

	static bool CanPlayAnimation(USkeletalMeshComponent* SkeletalMeshComponent, UAnimSequenceBase* AnimAssetBase)
	{
		return (SkeletalMeshComponent->GetSkeletalMeshAsset() &&
			SkeletalMeshComponent->GetSkeletalMeshAsset()->GetSkeleton() &&
			AnimAssetBase != nullptr &&
			AnimAssetBase->GetSkeleton() != nullptr);
	}

	static UAnimInstance* GetSourceAnimInstance(USkeletalMeshComponent* SkeletalMeshComponent)
	{
		UAnimInstance* SkelAnimInstance = SkeletalMeshComponent->GetAnimInstance();
		ISequencerAnimationSupport* SeqInterface = Cast<ISequencerAnimationSupport>(SkelAnimInstance);
		if (SeqInterface)
		{
			return SeqInterface->GetSourceAnimInstance();
		}

		return SkelAnimInstance;
	}

private:

	/** Parameter structure for setting the skeletal animation position */
	struct FSetAnimPositionParams
	{
		FMovieSceneEntityID EntityID;
		FRootInstanceHandle RootInstanceHandle;

		const UMovieSceneSkeletalAnimationSection* Section = nullptr;
		USkeletalMeshComponent* SkeletalMeshComponent = nullptr;

		FFrameTime CurrentTime;
		float FromPosition;
		float ToPosition;
		float Weight;

		bool bWantsRestoreState;
		bool bPlaying;
		bool bFireNotifies;
		bool bResetDynamics;
	};

	void SimulateMotionVectors(USkeletalMeshComponent* SkeletalMeshComponent, UMovieSceneMotionVectorSimulationSystem* MotionVectorSim) const
	{
		for (USceneComponent* Child : SkeletalMeshComponent->GetAttachChildren())
		{
			if (!Child)
			{
				continue;
			}

			FName SocketName = Child->GetAttachSocketName();
			if (SocketName != NAME_None)
			{
				FTransform SocketTransform = SkeletalMeshComponent->GetSocketTransform(SocketName, RTS_Component);
				MotionVectorSim->AddSimulatedTransform(SkeletalMeshComponent, SocketTransform, SocketName);
			}
		}
	}

	void ApplyAnimations(
		USkeletalMeshComponent* SkeletalMeshComponent,
		TArrayView<const FActiveSkeletalAnimation> SkeletalAnimations) const
	{
		for (const FActiveSkeletalAnimation& SkeletalAnimation : SkeletalAnimations)
		{
			const UMovieSceneSkeletalAnimationSection* AnimSection = SkeletalAnimation.AnimSection;
			const FMovieSceneSkeletalAnimationParams& AnimParams = AnimSection->Params;
			UMovieSceneSkeletalAnimationSection::FRootMotionParams RootMotionParams;
			AnimSection->GetRootMotion(SkeletalAnimation.EvalFrameTime.RoundToFrame(), RootMotionParams);
			//set up root motion/bone transform delegates
			if (AnimSection->Params.SwapRootBone != ESwapRootBone::SwapRootBone_None)
			{
				FTransform Transform = RootMotionParams.Transform.IsSet() ? RootMotionParams.Transform.GetValue() : FTransform::Identity;
				TOptional<FTransform> InitialActorTransform = GetCurrentTransform(AnimSection->Params.SwapRootBone, SkeletalMeshComponent);
				if (FBoundObjectActiveSkeletalAnimations* BoundObjectAnimations = SystemData->SkeletalAnimations.Find(SkeletalMeshComponent))
				{
					BoundObjectAnimations->BoneTransformFinalizeData.Register(SkeletalMeshComponent, AnimSection->Params.SwapRootBone,Transform, InitialActorTransform);
				}
			}

			// Don't fire notifies if looping around.
			bool bLooped = false;
			if (AnimParams.bReverse)
			{
				if (SkeletalAnimation.FromEvalTime <= SkeletalAnimation.ToEvalTime)
				{
					bLooped = true;
				}
			}
			else if (SkeletalAnimation.FromEvalTime >= SkeletalAnimation.ToEvalTime)
			{
				bLooped = true;
			}

			FSetAnimPositionParams SetAnimPositionParams;
			SetAnimPositionParams.EntityID = SkeletalAnimation.EntityID;
			SetAnimPositionParams.RootInstanceHandle = SkeletalAnimation.RootInstanceHandle;
			SetAnimPositionParams.Section = AnimSection;
			SetAnimPositionParams.SkeletalMeshComponent = SkeletalMeshComponent;
			SetAnimPositionParams.CurrentTime = SkeletalAnimation.EvalFrameTime;
			SetAnimPositionParams.FromPosition = SkeletalAnimation.FromEvalTime;
			SetAnimPositionParams.ToPosition = SkeletalAnimation.ToEvalTime;
			SetAnimPositionParams.Weight = SkeletalAnimation.BlendWeight;
			SetAnimPositionParams.bWantsRestoreState = SkeletalAnimation.bWantsRestoreState;
			SetAnimPositionParams.bPlaying = SkeletalAnimation.bPlaying;
			SetAnimPositionParams.bFireNotifies = (SkeletalAnimation.bFireNotifies && !AnimParams.bSkipAnimNotifiers && !bLooped);
			SetAnimPositionParams.bResetDynamics = SkeletalAnimation.bResetDynamics;

			if (SkeletalAnimation.bPreviewPlayback)
			{
				PreviewSetAnimPosition(SetAnimPositionParams);
			}
			else
			{
				SetAnimPosition(SetAnimPositionParams);
			}
		}
	}
	
	// Determines whether the bound object has a component transform property tag
	bool ContainsTransform(UObject* InBoundObject) const
	{
		using namespace UE::MovieScene;

		bool bContainsTransform = false;

		auto HarvestTransforms = [InBoundObject, &bContainsTransform](UObject* BoundObject)
		{
			if (BoundObject == InBoundObject)
			{
				bContainsTransform = true;
			}
		};
				
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		FMovieSceneTracksComponentTypes* Components = FMovieSceneTracksComponentTypes::Get();
		FEntityTaskBuilder()
			.Read(BuiltInComponents->BoundObject)
			// Only include component transforms
			.FilterAll({ Components->ComponentTransform.PropertyTag })
			// Only read things with the resolved properties on - this ensures we do not read any intermediate component transforms for blended properties
			.FilterAny({ BuiltInComponents->CustomPropertyIndex, BuiltInComponents->FastPropertyOffset, BuiltInComponents->SlowProperty })
			.Iterate_PerEntity(&Linker->EntityManager, HarvestTransforms);

		return bContainsTransform;
	}

	// Get the current transform for the component that the root bone will be swaped to
	TOptional<FTransform> GetCurrentTransform(ESwapRootBone SwapRootBone, USkeletalMeshComponent* SkeletalMeshComponent) const
	{
		TOptional<FTransform> CurrentTransform;
		if (SwapRootBone == ESwapRootBone::SwapRootBone_Component)
		{
			if (ContainsTransform(SkeletalMeshComponent))
			{
				CurrentTransform = SkeletalMeshComponent->GetRelativeTransform();
			}
		}
		else if (SwapRootBone == ESwapRootBone::SwapRootBone_Actor)
		{
			if (AActor* Actor = SkeletalMeshComponent->GetOwner())
			{
				if (USceneComponent* RootComponent = Actor->GetRootComponent())
				{
					if (ContainsTransform(RootComponent))
					{
						CurrentTransform = RootComponent->GetRelativeTransform();
					}
				}
			}
		}

		return CurrentTransform;
	}

	void SetAnimPosition(const FSetAnimPositionParams& Params) const
	{
		static const bool bLooping = false;

		const FMovieSceneSkeletalAnimationParams& AnimParams = Params.Section->Params;
		UAnimSequenceBase* Animation = Params.Section->GetPlaybackAnimation();
		if (!CanPlayAnimation(Params.SkeletalMeshComponent, Animation))
		{
			return;
		}

		UAnimSequencerInstance* SequencerInst = GetAnimSequencerInstance(Params.SkeletalMeshComponent);
		if (SequencerInst)
		{
			PreAnimatedAnimInstanceStorage->BeginTrackingEntity(Params.EntityID, Params.bWantsRestoreState, Params.RootInstanceHandle, SequencerInst);
			PreAnimatedAnimInstanceStorage->CachePreAnimatedValue(FCachePreAnimatedValueParams(), SequencerInst);
			
			TOptional <FRootMotionOverride> RootMotion;
			UMovieSceneSkeletalAnimationSection::FRootMotionParams RootMotionParams;

			Params.Section->GetRootMotion(Params.CurrentTime, RootMotionParams);
			if (RootMotionParams.Transform.IsSet())
			{
				RootMotion = FRootMotionOverride();
				RootMotion.GetValue().RootMotion = RootMotionParams.Transform.GetValue();
				RootMotion.GetValue().bBlendFirstChildOfRoot = RootMotionParams.bBlendFirstChildOfRoot;
				RootMotion.GetValue().ChildBoneIndex = RootMotionParams.ChildBoneIndex;
				RootMotion.GetValue().PreviousTransform = RootMotionParams.PreviousTransform.GetValue();
			}

			// Use the section's address as the ID for the anim sequence.
			const uint32 AnimSequenceID = GetTypeHash(Params.Section);

			// If Sequencer has a transform track, we want to set the initial transform so that root motion (if it exists) can be applied relative to that.
			TOptional<FTransform> CurrentTransform = GetCurrentTransform(Params.Section->Params.SwapRootBone, Params.SkeletalMeshComponent);

			FAnimSequencerData AnimSequencerData(
					Animation,
					AnimSequenceID,
					RootMotion, 
					Params.FromPosition,
					Params.ToPosition,
					Params.Weight, 
					Params.bFireNotifies, 
					Params.Section->Params.SwapRootBone, 
					CurrentTransform,
					Params.Section->Params.MirrorDataTable.Get());
			SequencerInst->UpdateAnimTrackWithRootMotion(AnimSequencerData);
		}
		else if (UAnimInstance* AnimInst = GetSourceAnimInstance(Params.SkeletalMeshComponent))
		{
			FMontagePlayerPerSectionData* SectionData = SystemData->MontageData.FindOrAdd(Params.SkeletalMeshComponent).Find(Params.Section);

			int32 InstanceId = (SectionData) ? SectionData->MontageInstanceId : INDEX_NONE;


			const float AssetPlayRate = FMath::IsNearlyZero(Animation->RateScale) ? 1.0f : Animation->RateScale;
			TWeakObjectPtr<UAnimMontage> WeakMontage = FAnimMontageInstance::SetSequencerMontagePosition(
					AnimParams.SlotName,
					AnimInst, 
					InstanceId, 
					Animation, 
					Params.FromPosition / AssetPlayRate, 
					Params.ToPosition / AssetPlayRate, 
					Params.Weight, 
					bLooping, 
					Params.bPlaying);

			UAnimMontage* Montage = WeakMontage.Get();
			if (Montage)
			{
				FMontagePlayerPerSectionData& DataContainer = SystemData->MontageData.FindOrAdd(Params.SkeletalMeshComponent).FindOrAdd(Params.Section);
				DataContainer.Montage = WeakMontage;
				DataContainer.MontageInstanceId = InstanceId;

				PreAnimatedMontageStorage->BeginTrackingEntity(Params.EntityID, Params.bWantsRestoreState, Params.RootInstanceHandle, Montage);
				PreAnimatedMontageStorage->CachePreAnimatedValue(FCachePreAnimatedValueParams(), Montage, [=](const FObjectKey& Unused) {
						FPreAnimatedSkeletalAnimationMontageState OutState;
						OutState.WeakInstance = AnimInst;
						OutState.MontageInstanceId = InstanceId;
						return OutState;
					});

				// Make sure it's playing if the sequence is.
				FAnimMontageInstance* Instance = AnimInst->GetMontageInstanceForID(InstanceId);
				Instance->bPlaying = Params.bPlaying;
			}
		}
	}

	void PreviewSetAnimPosition(const FSetAnimPositionParams& Params) const
	{
		using namespace UE::MovieScene;

		static const bool bLooping = false;

		const FMovieSceneSkeletalAnimationParams& AnimParams = Params.Section->Params;
		UAnimSequenceBase* Animation = Params.Section->GetPlaybackAnimation();
		if (!CanPlayAnimation(Params.SkeletalMeshComponent, Animation))
		{
			return;
		}

		UAnimSequencerInstance* SequencerInst = GetAnimSequencerInstance(Params.SkeletalMeshComponent);
		if (SequencerInst)
		{
			PreAnimatedAnimInstanceStorage->BeginTrackingEntity(Params.EntityID, Params.bWantsRestoreState, Params.RootInstanceHandle, SequencerInst);
			PreAnimatedAnimInstanceStorage->CachePreAnimatedValue(FCachePreAnimatedValueParams(), SequencerInst);

			TOptional <FRootMotionOverride> RootMotion;
			UMovieSceneSkeletalAnimationSection::FRootMotionParams RootMotionParams;
			Params.Section->GetRootMotion(Params.CurrentTime, RootMotionParams);
			if (RootMotionParams.Transform.IsSet())
			{
				RootMotion = FRootMotionOverride();
				RootMotion.GetValue().RootMotion = RootMotionParams.Transform.GetValue();
				RootMotion.GetValue().ChildBoneIndex = RootMotionParams.ChildBoneIndex;
				RootMotion.GetValue().bBlendFirstChildOfRoot = RootMotionParams.bBlendFirstChildOfRoot;
				RootMotion.GetValue().PreviousTransform = RootMotionParams.PreviousTransform.GetValue();
			}

			// Use the section's address as the ID for the anim sequence.
			const uint32 AnimSequenceID = GetTypeHash(Params.Section);

			// If Sequencer has a transform track, we want to set the initial transform so that root motion (if it exists) can be applied relative to that.
			TOptional<FTransform> CurrentTransform = GetCurrentTransform(Params.Section->Params.SwapRootBone, Params.SkeletalMeshComponent);

			FAnimSequencerData AnimSequencerData(
					Animation,
					AnimSequenceID,
					RootMotion,
					Params.FromPosition,
					Params.ToPosition,
					Params.Weight,
					Params.bFireNotifies,
					Params.Section->Params.SwapRootBone,
					CurrentTransform,
					Params.Section->Params.MirrorDataTable.Get());
			SequencerInst->UpdateAnimTrackWithRootMotion(AnimSequencerData);
		}
		else if (UAnimInstance* AnimInst = GetSourceAnimInstance(Params.SkeletalMeshComponent))
		{
			FMontagePlayerPerSectionData* SectionData = SystemData->MontageData.FindOrAdd(Params.SkeletalMeshComponent).Find(Params.Section);

			int32 InstanceId = SectionData ? SectionData->MontageInstanceId : INDEX_NONE;
		
			const float AssetPlayRate = FMath::IsNearlyZero(Animation->RateScale) ? 1.0f : Animation->RateScale;
			TWeakObjectPtr<UAnimMontage> WeakMontage = FAnimMontageInstance::PreviewSequencerMontagePosition(
					AnimParams.SlotName,
					Params.SkeletalMeshComponent,
					AnimInst,
					InstanceId,
					Animation,
					Params.FromPosition / AssetPlayRate,
					Params.ToPosition / AssetPlayRate,
					Params.Weight,
					bLooping,
					Params.bFireNotifies,
					Params.bPlaying);

			UAnimMontage* Montage = WeakMontage.Get();
			if (Montage)
			{
				FMontagePlayerPerSectionData& DataContainer = SystemData->MontageData.FindOrAdd(Params.SkeletalMeshComponent).FindOrAdd(Params.Section);
				DataContainer.Montage = WeakMontage;
				DataContainer.MontageInstanceId = InstanceId;

				PreAnimatedMontageStorage->BeginTrackingEntity(Params.EntityID, Params.bWantsRestoreState, Params.RootInstanceHandle, Montage);
				PreAnimatedMontageStorage->CachePreAnimatedValue(FCachePreAnimatedValueParams(), Montage, [=](const FObjectKey& Unused) {
						FPreAnimatedSkeletalAnimationMontageState OutState;
						OutState.WeakInstance = AnimInst;
						OutState.MontageInstanceId = InstanceId;
						return OutState;
					});

				FAnimMontageInstance* Instance = AnimInst->GetMontageInstanceForID(InstanceId);
				Instance->bPlaying = Params.bPlaying;
			}

			if (Params.bResetDynamics)
			{
				// Make sure we reset any simulations.
				AnimInst->ResetDynamics(ETeleportType::ResetPhysics);
			}
		}
	}
};

} // namespace UE::MovieScene

UMovieSceneSkeletalAnimationSystem::UMovieSceneSkeletalAnimationSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Instantiation | ESystemPhase::Scheduling;

	SystemCategories |= FSystemInterrogator::GetExcludedFromInterrogationCategory();

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UWeightAndEasingEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneComponentTransformSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneQuaternionInterpolationRotationSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieSceneObjectPropertySystem::StaticClass(), GetClass());

		DefineImplicitPrerequisite(GetClass(), UMovieSceneRestorePreAnimatedStateSystem::StaticClass());
	}
}

UObject* UMovieSceneSkeletalAnimationSystem::ResolveSkeletalMeshComponentBinding(UObject* InObject)
{
	// Check if we are bound directly to a skeletal mesh component.
	USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(InObject);
	if (SkeletalMeshComponent)
	{
		return SkeletalMeshComponent;
	}

	// Then check to see if we are controlling an actor. If so use its first skeletal mesh component.
	AActor* Actor = Cast<AActor>(InObject);
	if (!Actor)
	{
		if (UChildActorComponent* ChildActorComponent = Cast<UChildActorComponent>(InObject))
		{
			Actor = ChildActorComponent->GetChildActor();
		}
	}
	if (Actor)
	{
		return Actor->FindComponentByClass<USkeletalMeshComponent>();
	}
	return nullptr;
}

FTransform UMovieSceneSkeletalAnimationSystem::GetRootMotionOffset(UObject* InObject) const
{
	using namespace UE::MovieScene;
	
	FTransform RootMotionOffset;
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ResolveSkeletalMeshComponentBinding(InObject)))
	{
		if (const FBoundObjectActiveSkeletalAnimations* BoundObjectAnimations = SystemData.SkeletalAnimations.Find(SkeletalMeshComponent))
		{
			RootMotionOffset = BoundObjectAnimations->BoneTransformFinalizeData.MeshRelativeRootMotionTransform;
			if (BoundObjectAnimations->BoneTransformFinalizeData.SwapRootBone == ESwapRootBone::SwapRootBone_Actor
				&& BoundObjectAnimations->BoneTransformFinalizeData.InverseMeshToActorRotation.IsSet())
			{
				RootMotionOffset.SetTranslation(BoundObjectAnimations->BoneTransformFinalizeData.InverseMeshToActorRotation.GetValue() * RootMotionOffset.GetTranslation());
			}
		}
	}

	return RootMotionOffset;
}

void UMovieSceneSkeletalAnimationSystem::UpdateRootMotionOffset(UObject* InObject)
{
	using namespace UE::MovieScene;

	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ResolveSkeletalMeshComponentBinding(InObject)))
	{
		if (FBoundObjectActiveSkeletalAnimations* BoundObjectAnimations = SystemData.SkeletalAnimations.Find(SkeletalMeshComponent))
		{
			if (BoundObjectAnimations->BoneTransformFinalizeData.InitialActorTransform.IsSet())
			{
				FTransform RootMotionOffset = GetRootMotionOffset(InObject);
				FTransform CurrentTransform;
				if (BoundObjectAnimations->BoneTransformFinalizeData.SwapRootBone == ESwapRootBone::SwapRootBone_Component)
				{
					CurrentTransform = SkeletalMeshComponent->GetRelativeTransform();
				}
				else if (BoundObjectAnimations->BoneTransformFinalizeData.SwapRootBone == ESwapRootBone::SwapRootBone_Actor)
				{
					if (AActor* Actor = SkeletalMeshComponent->GetOwner())
					{
						if (USceneComponent* RootComponent = Actor->GetRootComponent())
						{
							CurrentTransform = RootComponent->GetRelativeTransform();
						}
					}
				}
				// Subtract root motion off of this
				CurrentTransform = RootMotionOffset.Inverse() * CurrentTransform;
				
				// Reset the initial transform based on this.
				BoundObjectAnimations->BoneTransformFinalizeData.InitialActorTransform = CurrentTransform;
			}
		}
	}
}

TOptional<FTransform> UMovieSceneSkeletalAnimationSystem::GetInitialActorTransform(UObject* InObject) const
{
	using namespace UE::MovieScene;

	TOptional<FTransform> InitialActorTransform;
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ResolveSkeletalMeshComponentBinding(InObject)))
	{
		if (const FBoundObjectActiveSkeletalAnimations* BoundObjectAnimations = SystemData.SkeletalAnimations.Find(SkeletalMeshComponent))
		{
			InitialActorTransform = BoundObjectAnimations->BoneTransformFinalizeData.InitialActorTransform;
		}
	}
	return InitialActorTransform;
}

TOptional<FQuat> UMovieSceneSkeletalAnimationSystem::GetInverseMeshToActorRotation(UObject* InObject) const
{
	using namespace UE::MovieScene;

	TOptional<FQuat> InverseMeshToActorRotation;
	if (USkeletalMeshComponent* SkeletalMeshComponent = Cast<USkeletalMeshComponent>(ResolveSkeletalMeshComponentBinding(InObject)))
	{
		if (const FBoundObjectActiveSkeletalAnimations* BoundObjectAnimations = SystemData.SkeletalAnimations.Find(SkeletalMeshComponent))
		{
			InverseMeshToActorRotation = BoundObjectAnimations->BoneTransformFinalizeData.InverseMeshToActorRotation;
		}
	}
	return InverseMeshToActorRotation;
}

void UMovieSceneSkeletalAnimationSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();


	// Facade task that mimics a write dependency to transform results to guarantee that
	//     skeletal animation evaluation tasks are scheduled after transforms.
	// @todo: this currently makes all skel anims dependent upon all transforms, which
	//        is not ideal, but a more granular dependency is not currently possible (or would be prohibitively complex)
	struct FTransformDependencyTask
	{
		static void ForEachAllocation(const FEntityAllocation*,
			TWriteOptional<double>,TWriteOptional<double>,TWriteOptional<double>,
			TWriteOptional<double>,TWriteOptional<double>,TWriteOptional<double>,
			TWriteOptional<double>,TWriteOptional<double>,TWriteOptional<double>)
		{}
	};

	// Facade task that mimics a write dependency to object property results to guarantee that
	//     skeletal animation evaluation tasks are scheduled after any SetMesh calls.
	// @todo: this currently makes all skel anims dependent upon all transforms, which
	//        is not ideal, but a more granular dependency is not currently possible (or would be prohibitively complex)
	struct FWriteObjectResultNoop
	{
		static void ForEachAllocation(FEntityAllocationIteratorItem, TWrite<FObjectComponent>)
		{}
	};


	// Schedule a dummy task that writes to all object results to guarantee that animation eval
	//    operates after any calls to SetMesh
	FTaskID WaitForObjectProperties = FEntityTaskBuilder()
	.Write(BuiltInComponents->ObjectResult)
	.Schedule_PerAllocation<FWriteObjectResultNoop>(&Linker->EntityManager, TaskScheduler);


	// Schedule a dummy task that does nothing but open all transform components for write
	//  this is used to ensure that previously scheduled transform setter tasks have completed before
	//  we evaluate skel animations and root motion
	FTaskID WaitForAllTransforms = FEntityTaskBuilder()
	.WriteOptional(BuiltInComponents->DoubleResult[0])
	.WriteOptional(BuiltInComponents->DoubleResult[1])
	.WriteOptional(BuiltInComponents->DoubleResult[2])
	.WriteOptional(BuiltInComponents->DoubleResult[3])
	.WriteOptional(BuiltInComponents->DoubleResult[4])
	.WriteOptional(BuiltInComponents->DoubleResult[5])
	.WriteOptional(BuiltInComponents->DoubleResult[6])
	.WriteOptional(BuiltInComponents->DoubleResult[7])
	.WriteOptional(BuiltInComponents->DoubleResult[8])
	.FilterAll({ TrackComponents->ComponentTransform.PropertyTag })
	.Schedule_PerAllocation<FTransformDependencyTask>(&Linker->EntityManager, TaskScheduler);

	// Skip gathering any anims that are tagged as anim mixer pose producer- these will be handled by the version in the anim mixer plugin
	FTaskID GatherTask = FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->InstanceHandle)
	.Read(BuiltInComponents->BoundObject)
	.Read(TrackComponents->SkeletalAnimation)
	.ReadOptional(BuiltInComponents->EvalTime)
	.ReadOptional(BuiltInComponents->WeightAndEasingResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored, TrackComponents->Tags.AnimMixerPoseProducer })
	.SetStat(GET_STATID(MovieSceneEval_GatherSkeletalAnimations))
	.Schedule_PerAllocation<FGatherSkeletalAnimations>(&Linker->EntityManager, TaskScheduler, 
			Linker->GetInstanceRegistry(), &SystemData);

	// Now evaluate gathered animations. We need to do this on the game thread (when in multi-threaded mode)
	// because this task will call into a lot of animation system code that needs to be called there.
	FTaskParams Params(GET_STATID(MovieSceneEval_EvaluateSkeletalAnimations));
	Params.ForceGameThread();
	FTaskID EvaluateTask = TaskScheduler->AddTask<FEvaluateSkeletalAnimations>(Params, Linker, &SystemData);

	TaskScheduler->AddPrerequisite(GatherTask, EvaluateTask);
	TaskScheduler->AddPrerequisite(WaitForAllTransforms, EvaluateTask);
	TaskScheduler->AddPrerequisite(WaitForObjectProperties, EvaluateTask);
}

void UMovieSceneSkeletalAnimationSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const TStatId GatherStatId = GET_STATID(MovieSceneEval_GatherSkeletalAnimations);

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents   = FMovieSceneTracksComponentTypes::Get();

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();
	if (Runner->GetCurrentPhase() == ESystemPhase::Instantiation)
	{
		if (AnimationUIFlickerFixActive())
		{
			// Begin tracking pre-animated state for all bound skel animation components
			TSharedPtr<FPreAnimatedSkeletalAnimationStorage> PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedSkeletalAnimationStorage>();

			struct FTask
			{
				FEntityComponentFilter AdditionalFilter;
			} Task;
			Task.AdditionalFilter.All({ TrackComponents->SkeletalAnimation, BuiltInComponents->Tags.NeedsLink });
			PreAnimatedStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Task, BuiltInComponents->BoundObject);
		}

		CleanSystemData();
		return;
	}

	// Skip gathering any anims that are tagged as anim mixer pose producer- these will be handled by the version in the anim mixer plugin
	FGraphEventRef GatherTask = FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->InstanceHandle)
	.Read(BuiltInComponents->BoundObject)
	.Read(TrackComponents->SkeletalAnimation)
	.ReadOptional(BuiltInComponents->EvalTime)
	.ReadOptional(BuiltInComponents->WeightAndEasingResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored, TrackComponents->Tags.AnimMixerPoseProducer })
	.SetStat(GatherStatId)
	.Dispatch_PerAllocation<FGatherSkeletalAnimations>(&Linker->EntityManager, InPrerequisites, nullptr, 
			Linker->GetInstanceRegistry(), &SystemData);

	FSystemTaskPrerequisites EvalPrereqs;
	if (GatherTask)
	{
		EvalPrereqs.AddRootTask(GatherTask);
	}

	// Now evaluate gathered animations. We need to do this on the game thread (when in multi-threaded mode)
	// because this task will call into a lot of animation system code that needs to be called there.
	FEntityTaskBuilder()
	.SetStat(GET_STATID(MovieSceneEval_EvaluateSkeletalAnimations))
	.SetDesiredThread(Linker->EntityManager.GetGatherThread())
	.Dispatch<FEvaluateSkeletalAnimations>(&Linker->EntityManager, EvalPrereqs, &Subsequents, Linker, &SystemData);
}

void UMovieSceneSkeletalAnimationSystem::OnLink()
{
	using namespace UE::MovieScene;
	
	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->LinkSystem<UMovieSceneEntityGroupingSystem>();

	GroupingKey = GroupingSystem->AddGrouping(FAnimationGroupingPolicy(), FBuiltInComponentTypes::Get()->BoundObject, FMovieSceneTracksComponentTypes::Get()->SkeletalAnimation);
}

void UMovieSceneSkeletalAnimationSystem::OnUnlink()
{
	using namespace UE::MovieScene;
	
	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->FindSystem<UMovieSceneEntityGroupingSystem>();
	if (ensure(GroupingSystem))
	{
		GroupingSystem->RemoveGrouping(GroupingKey);
	}
	GroupingKey = FEntityGroupingPolicyKey();
}

bool UMovieSceneSkeletalAnimationSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	// Must stay relevant for every entity we group, otherwise the system can unlink (removing its
	// grouping) while pose-producer entities still hold a Group component, orphaning that group. The
	// grouping filter does not exclude AnimMixerPoseProducer, so neither can this relevance filter -
	// even though gather/evaluate skip those entities.
	FEntityComponentFilter Filter;
	Filter.All({TrackComponents->SkeletalAnimation});
	return InLinker->EntityManager.Contains(Filter);
}


void UMovieSceneSkeletalAnimationSystem::CleanSystemData()
{
	SystemData.ResetSkeletalAnimations();
	// Clean-up old montage data.
	for (auto It = SystemData.MontageData.CreateIterator(); It; ++It)
	{
		if (It.Key().ResolveObjectPtr() == nullptr)
		{
			It.RemoveCurrent();
		}
	}
}

