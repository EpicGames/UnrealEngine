// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneSlomoSystem.h"

#include "Engine/World.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "GameFramework/WorldSettings.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneSlomoSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSlomoSystem)

namespace UE::MovieScene
{

struct FSlomoUtil
{
	static void ApplySlomo(UObject* PlaybackContext, double TimeDilation)
	{
		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

		if (!World || (!GIsEditor && World->GetNetMode() == NM_Client) || TimeDilation <= 0.f)
		{
			return;
		}

		AWorldSettings* WorldSettings = World->GetWorldSettings();

		if (WorldSettings)
		{
			WorldSettings->SetCinematicTimeDilation(TimeDilation);
			WorldSettings->ForceNetUpdate();
		}
	}
};

struct FPreAnimatedSlomoState
{
	TOptional<double> TimeDilation;

	static FPreAnimatedSlomoState SaveState(UObject* PlaybackContext)
	{
		UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
		if (!World)
		{
			return FPreAnimatedSlomoState();
		}

		if (AWorldSettings* WorldSettings = World->GetWorldSettings())
		{
			return FPreAnimatedSlomoState{ WorldSettings->CinematicTimeDilation };
		}
		return FPreAnimatedSlomoState();
	}

	void RestoreState(const FMovieSceneAnimTypeID& Unused, const FRestoreStateParams& Params)
	{
		TSharedPtr<const FSharedPlaybackState> SharedPlaybackState = Params.GetTerminalPlaybackState();
		if (!ensure(SharedPlaybackState))
		{
			return;
		}

		if (TimeDilation.IsSet())
		{
			FSlomoUtil::ApplySlomo(SharedPlaybackState->GetPlaybackContext(), TimeDilation.GetValue());
		}
	}
};

struct FPreAnimatedSlomoStateStorage : TSimplePreAnimatedStateStorage<FMovieSceneAnimTypeID, FPreAnimatedSlomoState>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedSlomoStateStorage> StorageID;

	FPreAnimatedStorageID GetStorageType() const override { return StorageID; }
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedSlomoStateStorage> FPreAnimatedSlomoStateStorage::StorageID;

struct FEvaluateSlomo
{
	const FInstanceRegistry* InstanceRegistry;

	FEvaluateSlomo(const FInstanceRegistry* InInstanceRegistry)
		: InstanceRegistry(InInstanceRegistry)
	{}

	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<FRootInstanceHandle> RootInstanceHandles, TRead<double> TimeDilations) const
	{
		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			FRootInstanceHandle RootInstanceHandle = RootInstanceHandles[Index];
			const FSequenceInstance& Instance = InstanceRegistry->GetInstance(RootInstanceHandle);
			UObject* PlaybackContext = Instance.GetSharedPlaybackState()->GetPlaybackContext();

			const double TimeDilation(TimeDilations[Index]);
			FSlomoUtil::ApplySlomo(PlaybackContext, TimeDilation);
		}
	}
};

} // namespace UE::MovieScene

UMovieSceneSlomoSystem::UMovieSceneSlomoSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	RelevantComponent = FMovieSceneTracksComponentTypes::Get()->Tags.Slomo;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		DefineComponentConsumer(GetClass(), BuiltInComponents->DoubleResult[0]);
	}
}

void UMovieSceneSlomoSystem::OnLink()
{
	using namespace UE::MovieScene;

	PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedSlomoStateStorage>();
}

void UMovieSceneSlomoSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->DoubleResult[0])
	.FilterAll({ TrackComponents->Tags.Slomo })
	.SetDesiredThread(Linker->EntityManager.GetGatherThread())
	.Schedule_PerAllocation<FEvaluateSlomo>(& Linker->EntityManager, TaskScheduler, Linker->GetInstanceRegistry());
}

void UMovieSceneSlomoSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->DoubleResult[0])
	.FilterAll({ TrackComponents->Tags.Slomo })
	.SetDesiredThread(Linker->EntityManager.GetGatherThread())
	.Dispatch_PerAllocation<FEvaluateSlomo>(& Linker->EntityManager, InPrerequisites, &Subsequents, Linker->GetInstanceRegistry());
}

void UMovieSceneSlomoSystem::SavePreAnimatedState(const FPreAnimationParameters& InParameters)
{
	using namespace UE::MovieScene;

	static const TMovieSceneAnimTypeID<UMovieSceneSlomoSystem> AnimTypeID;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	auto SaveSlomoStates = [this, BuiltInComponents, InstanceRegistry](
			const FEntityAllocation* Allocation, TRead<FMovieSceneEntityID> EntityIDs, TRead<FRootInstanceHandle> RootInstanceHandles)
	{
		const bool bWantsRestoreState = Allocation->HasComponent(BuiltInComponents->Tags.RestoreState);
		const FMovieSceneAnimTypeID Key = AnimTypeID;

		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			FRootInstanceHandle RootInstanceHandle = RootInstanceHandles[Index];
			const FSequenceInstance& Instance = InstanceRegistry->GetInstance(RootInstanceHandle);
			UObject* PlaybackContext = Instance.GetSharedPlaybackState()->GetPlaybackContext();

			PreAnimatedStorage->BeginTrackingEntity(EntityIDs[Index], bWantsRestoreState, RootInstanceHandle, Key);
			PreAnimatedStorage->CachePreAnimatedValue(Key, [PlaybackContext](const FMovieSceneAnimTypeID&) { return FPreAnimatedSlomoState::SaveState(PlaybackContext); });
		}
	};

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.FilterAll({ TrackComponents->Tags.Slomo })
	.Iterate_PerAllocation(&Linker->EntityManager, SaveSlomoStates);
}

