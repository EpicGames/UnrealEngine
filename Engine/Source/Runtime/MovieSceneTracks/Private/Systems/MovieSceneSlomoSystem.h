// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"

#include "MovieSceneSlomoSystem.generated.h"

namespace UE::MovieScene
{
	struct FPreAnimatedSlomoStateStorage;
}

/**
 * System for evaluating and applying world time dilation
 */
UCLASS()
class UMovieSceneSlomoSystem 
	: public UMovieSceneEntitySystem
	, public IMovieScenePreAnimatedStateSystemInterface
{
	GENERATED_BODY()

public:
	UMovieSceneSlomoSystem(const FObjectInitializer& ObjInit);

	virtual void OnLink() override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:
	virtual void SavePreAnimatedState(const FPreAnimationParameters& InParameters) override;

private:
	TSharedPtr<UE::MovieScene::FPreAnimatedSlomoStateStorage> PreAnimatedStorage;
};

