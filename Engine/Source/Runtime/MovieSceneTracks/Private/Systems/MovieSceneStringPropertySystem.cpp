// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneStringPropertySystem.h"
#include "Systems/StringChannelEvaluatorSystem.h"
#include "MovieSceneTracksComponentTypes.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneStringPropertySystem)


UMovieSceneStringPropertySystem::UMovieSceneStringPropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->String);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UStringChannelEvaluatorSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneStringPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}

