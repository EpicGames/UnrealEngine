// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneTextPropertySystem.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/TextChannelEvaluatorSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTextPropertySystem)

UMovieSceneTextPropertySystem::UMovieSceneTextPropertySystem(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->Text);
	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UTextChannelEvaluatorSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneTextPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}
