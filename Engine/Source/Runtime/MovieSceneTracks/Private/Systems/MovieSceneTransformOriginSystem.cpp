// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneTransformOriginSystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Tracks/IMovieSceneTransformOrigin.h"
#include "MovieSceneTracksComponentTypes.h"

#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieSceneQuaternionBlenderSystem.h"
#include "Systems/MovieSceneComponentTransformSystem.h"

#include "IMovieScenePlayer.h"
#include "IMovieScenePlaybackClient.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTransformOriginSystem)

namespace UE
{
namespace MovieScene
{

struct FGatherTransformOrigin
{
	TSparseArray<FTransform>* TransformOriginsByInstanceID;
	const FInstanceRegistry* InstanceRegistry;

	void Run(FEntityAllocationWriteContext WriteContext) const
	{
		TransformOriginsByInstanceID->Empty(InstanceRegistry->GetSparseInstances().Num());

		const TSparseArray<FSequenceInstance>& SparseInstances = InstanceRegistry->GetSparseInstances();
		for (int32 Index = 0; Index < SparseInstances.GetMaxIndex(); ++Index)
		{
			if (!SparseInstances.IsValidIndex(Index))
			{
				continue;
			}

			const FSequenceInstance& Instance = SparseInstances[Index];

			if(Instance.IsRootSequence())
			{
				TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();

				const IMovieScenePlaybackClient*  Client       = SharedPlaybackState->FindCapability<IMovieScenePlaybackClient>();
				const UObject*                    InstanceData = Client ? Client->GetInstanceData() : nullptr;
				const IMovieSceneTransformOrigin* RawInterface = Cast<const IMovieSceneTransformOrigin>(InstanceData);

				const bool bHasInterface = RawInterface || (InstanceData && InstanceData->GetClass()->ImplementsInterface(UMovieSceneTransformOrigin::StaticClass()));
				if (bHasInterface)
				{
					// Retrieve the current origin
					FTransform TransformOrigin = RawInterface ? RawInterface->GetTransformOrigin() : IMovieSceneTransformOrigin::Execute_BP_GetTransformOrigin(InstanceData);
					// Ignore scale. Sequencer does not apply scale, but the scale is factored into rotation, so assume always a scale of 1.
					TransformOrigin.SetScale3D(FVector::OneVector);

					TransformOriginsByInstanceID->Insert(Index, TransformOrigin);
				}
			}
		}
	}
};

struct FGatherTransformOriginsFromSubscenes
{
	// Map of child->parent instances, ordered parent-first
	TArray<FInstanceToParentPair>* InstanceHandleToParentHandle;
	
	TSparseArray<FTransform>* TransformOriginsByInstanceID;
	const FInstanceRegistry* InstanceRegistry;

	// First pass on subsequence origins. Write the transform origin for relevant section to its child instance to be pre-multiplied in the post task.
	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<FRootInstanceHandle> RootInstances, TRead<FMovieSceneSequenceID> SequenceIDs,
		TReadOptional<double> LocationX, TReadOptional<double> LocationY, TReadOptional<double> LocationZ,
		TReadOptional<double> RotationX, TReadOptional<double> RotationY, TReadOptional<double> RotationZ) const
	{
		
		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			// The subsequence section is in the parent sequence of the instance we want to apply the transforms to
			// Find the handle to the subinstance to write our origin data to.
			FInstanceHandle SubInstanceHandle = InstanceRegistry->GetInstance(RootInstances[Index]).FindSubInstance(SequenceIDs[Index]);
			
			if (!SubInstanceHandle.IsValid())
			{
				continue;
			}

			const FVector    Translation(LocationX ? LocationX[Index] : 0.f, LocationY ? LocationY[Index] : 0.f, LocationZ ? LocationZ[Index] : 0.f);
			const FRotator   Rotation(RotationY ? RotationY[Index] : 0.f, RotationZ ? RotationZ[Index] : 0.f, RotationX ? RotationX[Index] : 0.f);
			const FTransform TransformOrigin(Rotation, Translation);
			
			// Set the entry for this transform origin.
			TransformOriginsByInstanceID->Insert(SubInstanceHandle.InstanceID, TransformOrigin);
		}
	}
	
	// After all the base transforms for subsequences are gathered, multiply in their parent transform
	// This is run even if ForEachAllocation is not run, which will be the case if there are no transform overrides on a given subscene, ensuring it still gets its parent's origin.
	void PostTask()
	{
		// InstanceHandleToParentHandle mapping is sorted parent first, so each parent's transform will be valid by the time its child uses it.
		for (const FInstanceToParentPair Mapping : *InstanceHandleToParentHandle)
		{
			// If there's no parent transform there's nothing to do.
			if(!TransformOriginsByInstanceID->IsValidIndex(Mapping.Parent.InstanceID))
			{
				continue;
			}

			// If there's a child transform it needs to be multiplied in with the parent transform.
			if (TransformOriginsByInstanceID->IsValidIndex(Mapping.Child.InstanceID))
			{
				(*TransformOriginsByInstanceID)[Mapping.Child.InstanceID] *= (*TransformOriginsByInstanceID)[Mapping.Parent.InstanceID];
			}
			else
			{
				const FTransform ParentTransform = (*TransformOriginsByInstanceID)[Mapping.Parent.InstanceID];
				TransformOriginsByInstanceID->Insert(Mapping.Child.InstanceID, ParentTransform);
			}
		}
	}
};

struct FAssignTransformOrigin
{
	const TSparseArray<FTransform>* TransformOriginsByInstanceID;

	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<FInstanceHandle> Instances, TRead<UObject*> BoundObjects,
		TWriteOptional<double> LocationX, TWriteOptional<double> LocationY, TWriteOptional<double> LocationZ,
		TWriteOptional<double> RotationX, TWriteOptional<double> RotationY, TWriteOptional<double> RotationZ) const
	{
		TransformLocation(Instances, BoundObjects, LocationX, LocationY, LocationZ, RotationX, RotationY, RotationZ, Allocation->Num());
	}

	void TransformLocation(const FInstanceHandle* Instances, const UObject* const * BoundObjects,
		double* OutLocationX, double* OutLocationY, double* OutLocationZ,
		double* OutRotationX, double* OutRotationY, double* OutRotationZ,
		int32 Num) const
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			FInstanceHandle InstanceHandle = Instances[Index];
			if (!TransformOriginsByInstanceID->IsValidIndex(InstanceHandle.InstanceID))
			{
				continue;
			}

			// Do not apply transform origins to attached objects
			const USceneComponent* SceneComponent = CastChecked<const USceneComponent>(BoundObjects[Index]);
			if (SceneComponent->GetAttachParent() != nullptr)
			{
				continue;
			}

			FTransform Origin = (*TransformOriginsByInstanceID)[InstanceHandle.InstanceID];

			FVector  CurrentTranslation(OutLocationX ? OutLocationX[Index] : 0.f, OutLocationY ? OutLocationY[Index] : 0.f, OutLocationZ ? OutLocationZ[Index] : 0.f);
			FRotator CurrentRotation(OutRotationY ? OutRotationY[Index] : 0.f, OutRotationZ ? OutRotationZ[Index] : 0.f, OutRotationX ? OutRotationX[Index] : 0.f);

			FTransform NewTransform = FTransform(CurrentRotation, CurrentTranslation)*Origin;

			FVector  NewTranslation = NewTransform.GetTranslation();
			FRotator NewRotation    = NewTransform.GetRotation().Rotator();

			if (OutLocationX) { OutLocationX[Index] = NewTranslation.X; }
			if (OutLocationY) { OutLocationY[Index] = NewTranslation.Y; }
			if (OutLocationZ) { OutLocationZ[Index] = NewTranslation.Z; }

			if (OutRotationX) { OutRotationX[Index] = NewRotation.Roll; }
			if (OutRotationY) { OutRotationY[Index] = NewRotation.Pitch; }
			if (OutRotationZ) { OutRotationZ[Index] = NewRotation.Yaw; }
		}
	}
};

} // namespace MovieScene
} // namespace UE


UMovieSceneTransformOriginInstantiatorSystem::UMovieSceneTransformOriginInstantiatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Instantiation;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->SymbolicTags.CreatesEntities);

		// This must be run before the double channel evaluator
		DefineImplicitPrerequisite(GetClass(), UDoubleChannelEvaluatorSystem::StaticClass());
	}
}


void UMovieSceneTransformOriginInstantiatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	FEntityComponentFilter Filter;
	Filter.All({ TracksComponents->ComponentTransform.PropertyTag, BuiltInComponents->Tags.AbsoluteBlend, BuiltInComponents->Tags.NeedsLink });
	Filter.None({ BuiltInComponents->BlendChannelOutput });
	Filter.Any({
		BuiltInComponents->DoubleResult[0],
		BuiltInComponents->DoubleResult[1],
		BuiltInComponents->DoubleResult[2],
		BuiltInComponents->DoubleResult[3],
		BuiltInComponents->DoubleResult[4],
		BuiltInComponents->DoubleResult[5]
	});

	// Stop constant values for transforms from being optimized - we need them to be re-evaluated every frame
	// since this system will trample them all
	Linker->EntityManager.MutateAll(Filter, FAddSingleMutation(BuiltInComponents->Tags.DontOptimizeConstants));
}

UMovieSceneTransformOriginSystem::UMovieSceneTransformOriginSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	Phase = ESystemPhase::Scheduling;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	LocationAndRotationFilterResults.Any(
	{
		BuiltInComponents->DoubleResult[0],
		BuiltInComponents->DoubleResult[1],
		BuiltInComponents->DoubleResult[2],
		BuiltInComponents->DoubleResult[3],
		BuiltInComponents->DoubleResult[4],
		BuiltInComponents->DoubleResult[5]
	});

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// This system relies upon anything that creates entities
		DefineImplicitPrerequisite(GetClass(), UMovieScenePiecewiseDoubleBlenderSystem::StaticClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneQuaternionBlenderSystem::StaticClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneComponentTransformSystem::StaticClass());
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());

		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->DoubleResult[0]);
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->DoubleResult[1]);
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->DoubleResult[2]);
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->DoubleResult[3]);
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->DoubleResult[4]);
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->DoubleResult[5]);
	}
}

bool UMovieSceneTransformOriginSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;
	
	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	
	FEntityComponentFilter SubSequenceHasOriginFilter;
	SubSequenceHasOriginFilter.All({BuiltInComponents->Tags.SubInstance});
	SubSequenceHasOriginFilter.Combine(LocationAndRotationFilterResults);

	if(InLinker->EntityManager.Contains(SubSequenceHasOriginFilter))
	{
		return true;
	}

	for (const FSequenceInstance& Instance : InLinker->GetInstanceRegistry()->GetSparseInstances())
	{
		TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();

		const IMovieScenePlaybackClient*  Client       = SharedPlaybackState->FindCapability<IMovieScenePlaybackClient>();
		const UObject*                    InstanceData = Client ? Client->GetInstanceData() : nullptr;
		const IMovieSceneTransformOrigin* RawInterface = Cast<const IMovieSceneTransformOrigin>(InstanceData);

		if (RawInterface || (InstanceData && InstanceData->GetClass()->ImplementsInterface(UMovieSceneTransformOrigin::StaticClass())))
		{
			return true;
		}
	}

	return false;
}

bool UMovieSceneTransformOriginSystem::GetTransformOrigin(UE::MovieScene::FInstanceHandle InstanceHandle, FTransform& OutTransform) const
{
	if (TransformOriginsByInstanceID.IsValidIndex(InstanceHandle.InstanceID))
	{
		OutTransform = TransformOriginsByInstanceID[InstanceHandle.InstanceID];
		return true;
	}

	return false;
}

void UMovieSceneTransformOriginSystem::OnLink()
{
	UMovieSceneTransformOriginInstantiatorSystem* Instantiator = Linker->LinkSystem<UMovieSceneTransformOriginInstantiatorSystem>();
	// This system keeps the instantiator around
	Linker->SystemGraph.AddReference(this, Instantiator);

	InstanceHandleToParentHandle.Empty();
	
}

void UMovieSceneTransformOriginSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	FEntityComponentFilter AssignFilter;
	AssignFilter.All({ TracksComponents->ComponentTransform.PropertyTag, BuiltInComponents->Tags.AbsoluteBlend });
	AssignFilter.None({ BuiltInComponents->BlendChannelOutput });
	AssignFilter.Combine(LocationAndRotationFilterResults);
	
	InstanceHandleToParentHandle.Empty();
	SequenceIDToInstanceHandle.Empty();
	
	FEntityTaskBuilder()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->InstanceHandle)
	.Read(BuiltInComponents->SequenceID)
	.FilterAll({BuiltInComponents->Tags.SubInstance})
	.FilterNone({BuiltInComponents->Tags.ImportedEntity}) // filter out parent entities, otherwise we'd double-up in the InstanceHandleToParentHandle mapping.
	.Iterate_PerEntity(&Linker->EntityManager,  [this, InstanceRegistry](FRootInstanceHandle RootInstance, FInstanceHandle Instance, FMovieSceneSequenceID SequenceID)
	{
		const FInstanceHandle SubInstanceHandle = InstanceRegistry->GetInstance(RootInstance).FindSubInstance(SequenceID);
		if (InstanceRegistry->IsHandleValid(SubInstanceHandle))
		{
			const FSubSequencePath SubSequencePath = InstanceRegistry->GetInstance(SubInstanceHandle).GetSubSequencePath();
			const int32 Depth = SubSequencePath.NumGenerationsFromRoot(SequenceID);
			InstanceHandleToParentHandle.AddUnique(FInstanceToParentPair(SubInstanceHandle, Instance, Depth));
			SequenceIDToInstanceHandle.Add(SequenceID, SubInstanceHandle);
		}
		
	});

	// InstanceHandleToParentHandle is the iteration source for calculating transforms.
	// This needs to be sorted parent first to ensure parent transforms are correct for when their children's transforms are calculated down-stream.
	InstanceHandleToParentHandle.Sort();
	

	FTaskID GatherTask = TaskScheduler->AddTask<FGatherTransformOrigin>(
		FTaskParams(TEXT("Gather Transform Origins")).ForceGameThread(),
		&TransformOriginsByInstanceID,
		InstanceRegistry
	);

	FTaskID GatherSubsequencesTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->SequenceID)
	.ReadOptional(BuiltInComponents->DoubleResult[0])
	.ReadOptional(BuiltInComponents->DoubleResult[1])
	.ReadOptional(BuiltInComponents->DoubleResult[2])
	.ReadOptional(BuiltInComponents->DoubleResult[3])
	.ReadOptional(BuiltInComponents->DoubleResult[4])
	.ReadOptional(BuiltInComponents->DoubleResult[5])
	.FilterAll({BuiltInComponents->Tags.SubInstance})
	.FilterNone({BuiltInComponents->Tags.ImportedEntity})
	.CombineFilter(LocationAndRotationFilterResults)
	.SetParams(FTaskParams(TEXT("Gather Transform Origins From Subscenes")).ForcePrePostTask())
	.Schedule_PerAllocation<FGatherTransformOriginsFromSubscenes>(&Linker->EntityManager, TaskScheduler, &InstanceHandleToParentHandle, &TransformOriginsByInstanceID, InstanceRegistry);

	FTaskID AssignTask = FEntityTaskBuilder()
	.Read(BuiltInComponents->InstanceHandle)
	.Read(BuiltInComponents->BoundObject)
	.WriteOptional(BuiltInComponents->DoubleResult[0])
	.WriteOptional(BuiltInComponents->DoubleResult[1])
	.WriteOptional(BuiltInComponents->DoubleResult[2])
	.WriteOptional(BuiltInComponents->DoubleResult[3])
	.WriteOptional(BuiltInComponents->DoubleResult[4])
	.WriteOptional(BuiltInComponents->DoubleResult[5])
	.CombineFilter(AssignFilter)
	.Fork_PerAllocation<FAssignTransformOrigin>(&Linker->EntityManager, TaskScheduler, &TransformOriginsByInstanceID);

	TaskScheduler->AddPrerequisite(GatherTask, AssignTask);
	TaskScheduler->AddPrerequisite(GatherTask, GatherSubsequencesTask);
	TaskScheduler->AddPrerequisite(GatherSubsequencesTask, AssignTask);
}

void UMovieSceneTransformOriginSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	TransformOriginsByInstanceID.Empty(InstanceRegistry->GetSparseInstances().Num());

	const TSparseArray<FSequenceInstance>& SparseInstances = InstanceRegistry->GetSparseInstances();
	for (int32 Index = 0; Index < SparseInstances.GetMaxIndex(); ++Index)
	{
		if (!SparseInstances.IsValidIndex(Index))
		{
			continue;
		}

		const FSequenceInstance& Instance = SparseInstances[Index];
		TSharedRef<const FSharedPlaybackState> SharedPlaybackState = Instance.GetSharedPlaybackState();

		const IMovieScenePlaybackClient*  Client       = SharedPlaybackState->FindCapability<IMovieScenePlaybackClient>();
		const UObject*                    InstanceData = Client ? Client->GetInstanceData() : nullptr;
		const IMovieSceneTransformOrigin* RawInterface = Cast<const IMovieSceneTransformOrigin>(InstanceData);

		const bool bHasInterface = RawInterface || (InstanceData && InstanceData->GetClass()->ImplementsInterface(UMovieSceneTransformOrigin::StaticClass()));
		if (bHasInterface)
		{
			// Retrieve the current origin
			FTransform TransformOrigin = RawInterface ? RawInterface->GetTransformOrigin() : IMovieSceneTransformOrigin::Execute_BP_GetTransformOrigin(InstanceData);

			TransformOriginsByInstanceID.Insert(Index, TransformOrigin);
		}
	}

	if (TransformOriginsByInstanceID.Num() != 0)
	{
		FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		FEntityComponentFilter Filter;
		Filter.All({ TracksComponents->ComponentTransform.PropertyTag, BuiltInComponents->Tags.AbsoluteBlend });
		Filter.None({ BuiltInComponents->BlendChannelOutput });

		FEntityTaskBuilder()
		.Read(BuiltInComponents->InstanceHandle)
		.Read(BuiltInComponents->BoundObject)
		.WriteOptional(BuiltInComponents->DoubleResult[0])
		.WriteOptional(BuiltInComponents->DoubleResult[1])
		.WriteOptional(BuiltInComponents->DoubleResult[2])
		.WriteOptional(BuiltInComponents->DoubleResult[3])
		.WriteOptional(BuiltInComponents->DoubleResult[4])
		.WriteOptional(BuiltInComponents->DoubleResult[5])
		.CombineFilter(Filter)
		// Must contain at least one double result
		.FilterAny({ BuiltInComponents->DoubleResult[0], BuiltInComponents->DoubleResult[1], BuiltInComponents->DoubleResult[2],
			BuiltInComponents->DoubleResult[3], BuiltInComponents->DoubleResult[4], BuiltInComponents->DoubleResult[5] })
		.Dispatch_PerAllocation<FAssignTransformOrigin>(&Linker->EntityManager, InPrerequisites, &Subsequents, &TransformOriginsByInstanceID);
	}
}


