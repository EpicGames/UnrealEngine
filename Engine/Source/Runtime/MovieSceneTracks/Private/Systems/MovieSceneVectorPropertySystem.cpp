// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneVectorPropertySystem.h"
#include "EntitySystem/MovieSceneBoundObjectInstantiator.h"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "Systems/MovieScenePropertyInstantiator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneVectorPropertySystem)

UMovieSceneFloatVectorPropertySystem::UMovieSceneFloatVectorPropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->FloatVector);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// We need our floats correctly evaluated and blended, so we are downstream from those systems.
		DefineImplicitPrerequisite(UFloatChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneFloatVectorPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}

UMovieSceneDoubleVectorPropertySystem::UMovieSceneDoubleVectorPropertySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	BindToProperty(UE::MovieScene::FMovieSceneTracksComponentTypes::Get()->DoubleVector);

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// We need our floats correctly evaluated and blended, so we are downstream from those systems.
		DefineImplicitPrerequisite(UDoubleChannelEvaluatorSystem::StaticClass(), GetClass());
		DefineImplicitPrerequisite(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneDoubleVectorPropertySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	Super::OnRun(InPrerequisites, Subsequents);
}


