// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/MovieSceneVisibilitySystem.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "Evaluation/PreAnimatedState/IMovieScenePreAnimatedStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "Systems/MovieSceneHierarchicalBiasSystem.h"
#include "MovieSceneTracksComponentTypes.h"
#include "EntitySystem/MovieSceneEntityGroupingSystem.h"
#include "Systems/MovieScenePropertyInstantiator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneVisibilitySystem)

namespace UE::MovieScene
{

struct FPreAnimatedVisibilityState
{
	bool bHidden = false;
	bool bActorTemporarilyHiddenInEditor = false;
	bool bComponentIsVisibleInEditor = false;

	FPreAnimatedVisibilityState()
	{}
	FPreAnimatedVisibilityState(bool bInHidden, bool bInActorTemporarilyHiddenInEditor, bool bInComponentIsVisibleInEditor)
		: bHidden(bInHidden)
		, bActorTemporarilyHiddenInEditor(bInActorTemporarilyHiddenInEditor)
		, bComponentIsVisibleInEditor(bInComponentIsVisibleInEditor)
	{}
};

struct FPreAnimatedVisibilityTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType     = FObjectKey;
	using StorageType = FPreAnimatedVisibilityState;

	StorageType CachePreAnimatedValue(const KeyType& Object)
	{
		UObject* ObjectPtr = Object.ResolveObjectPtr();
		if (AActor* Actor = Cast<AActor>(ObjectPtr))
		{
			const bool bHidden = Actor->IsHidden();
			const bool bTemporarilyHiddenInEditor = 
#if WITH_EDITOR
				Actor->IsTemporarilyHiddenInEditor();
#else
				false;
#endif  // WITH_EDITOR
			return StorageType(bHidden, bTemporarilyHiddenInEditor, false);
		}
		else if (USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectPtr))
		{
			const bool bHidden = SceneComponent->bHiddenInGame;
			const bool bVisibleInEditor = SceneComponent->IsVisibleInEditor();
			return StorageType(bHidden, false, bVisibleInEditor);
		}
		else
		{
			// Dummy value, we won't use it if the bound object isn't an actor or scene component.
			return StorageType(false, false, false);
		}
	}

	void RestorePreAnimatedValue(const KeyType& Object, StorageType& InOutCachedValue, const FRestoreStateParams& Params)
	{
		UObject* ObjectPtr = Object.ResolveObjectPtr();
		if (AActor* Actor = Cast<AActor>(ObjectPtr))
		{
			Actor->SetActorHiddenInGame(InOutCachedValue.bHidden);
#if WITH_EDITOR
			Actor->SetIsTemporarilyHiddenInEditor(InOutCachedValue.bActorTemporarilyHiddenInEditor);
#endif  // WITH_EDITOR
		}
		else if (USceneComponent* SceneComponent = Cast<USceneComponent>(ObjectPtr))
		{
			SceneComponent->SetHiddenInGame(InOutCachedValue.bHidden);
			SceneComponent->SetVisibility(InOutCachedValue.bComponentIsVisibleInEditor);
		}
	}
};

struct FPreAnimatedVisibilityStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedVisibilityTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedVisibilityStorage> StorageID;
};

TAutoRegisterPreAnimatedStorageID<FPreAnimatedVisibilityStorage> FPreAnimatedVisibilityStorage::StorageID;

struct FVisibilityTask
{
	TSharedPtr<FPreAnimatedVisibilityStorage> PreAnimatedStorage;

	void ForEachAllocation(
			FEntityAllocationProxy Item,
			TRead<FMovieSceneEntityID> EntityIDs,
			TRead<FRootInstanceHandle> RootInstanceHandles,
			TRead<UObject*> BoundObjects,
			TRead<bool> BoolResults) const
	{
		const int32 Num = Item.GetAllocation()->Num();

		// Cache pre-animiated state for all bound objects in this allocation.
		FPreAnimatedTrackerParams TrackerParams(Item);
		PreAnimatedStorage->BeginTrackingEntities(TrackerParams, EntityIDs, RootInstanceHandles, BoundObjects);

		FCachePreAnimatedValueParams CacheParams;
		PreAnimatedStorage->CachePreAnimatedValues(CacheParams, BoundObjects.AsArray(Num));

		// Now make bound objects in this allocation visible/hidden according to the evaluated bool channel.
		for (int32 Index = 0; Index < Num; ++Index)
		{
			if (UObject* const BoundObject = BoundObjects[Index])
			{
				const bool bShouldBeVisible = BoolResults[Index];

				if (AActor* Actor = Cast<AActor>(BoundObject))
				{
					Actor->SetActorHiddenInGame(!bShouldBeVisible);
#if WITH_EDITOR
					if (GIsEditor && Actor->GetWorld() != nullptr && !Actor->GetWorld()->IsPlayInEditor())
					{
						Actor->SetIsTemporarilyHiddenInEditor(!bShouldBeVisible);
					}
#endif  // WITH_EDITOR
				}
				else if (USceneComponent* SceneComponent = Cast<USceneComponent>(BoundObject))
				{
					SceneComponent->SetHiddenInGame(!bShouldBeVisible);
					SceneComponent->SetVisibility(bShouldBeVisible);
				}
			}
		}
	}
};

} // namespace UE::MovieScene


UMovieSceneVisibilitySystem::UMovieSceneVisibilitySystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::Core;

	RelevantComponent = FMovieSceneTracksComponentTypes::Get()->Tags.Visibility;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->BoundObject);
		DefineComponentConsumer(GetClass(), FBuiltInComponentTypes::Get()->Tags.Ignored);
		DefineImplicitPrerequisite(UMovieSceneCachePreAnimatedStateSystem::StaticClass(), GetClass());
	}
}

void UMovieSceneVisibilitySystem::OnLink()
{
	using namespace UE::MovieScene;

	PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedVisibilityStorage>();
}

void UMovieSceneVisibilitySystem::OnUnlink()
{
	PreAnimatedStorage = nullptr;
}

void UMovieSceneVisibilitySystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TrackComponents = FMovieSceneTracksComponentTypes::Get();

	FVisibilityTask Task { PreAnimatedStorage };

	FEntityTaskBuilder()
	.ReadEntityIDs()
	.Read(BuiltInComponents->RootInstanceHandle)
	.Read(BuiltInComponents->BoundObject)
	.Read(BuiltInComponents->BoolResult)
	.FilterAll({ BuiltInComponents->Tags.NeedsLink, TrackComponents->Tags.Visibility })
	.FilterNone({BuiltInComponents->Tags.Ignored})
	.RunInline_PerAllocation(&Linker->EntityManager, Task);
}

