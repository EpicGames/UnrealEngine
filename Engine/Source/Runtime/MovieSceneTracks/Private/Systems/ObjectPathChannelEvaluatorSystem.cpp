// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/ObjectPathChannelEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "Channels/MovieSceneObjectPathChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(ObjectPathChannelEvaluatorSystem)

namespace UE::MovieScene
{
	DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate object path channels"), MovieSceneEval_EvaluateObjectPathChannelTask, STATGROUP_MovieSceneECS);

	struct FEvaluateObjectPathChannels
	{
		static void ForEachEntity(FSourceObjectPathChannel ObjectPathChannel, FFrameTime FrameTime, FObjectComponent& OutResult)
		{
			UObject* ObjectResult = nullptr;
			if (!ObjectPathChannel.Source->Evaluate(FrameTime, ObjectResult))
			{
				OutResult = FObjectComponent::Null();
			}
			else
			{
				// Strong reference to keep the object alive - it is probably an asset of some sort that we wouldn't want to keep reloading
				OutResult = FObjectComponent::Strong(ObjectResult);
			}
		}
	};
}


UObjectPathChannelEvaluatorSystem::UObjectPathChannelEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	RelevantComponent = BuiltInComponents->ObjectPathChannel;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineComponentConsumer(GetClass(), BuiltInComponents->EvalTime);
		DefineComponentProducer(GetClass(), BuiltInComponents->ObjectResult);
	}
}

void UObjectPathChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->ObjectPathChannel)
	.Read(BuiltInComponents->EvalTime)
	.Write(BuiltInComponents->ObjectResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateObjectPathChannelTask))
	.Fork_PerEntity<FEvaluateObjectPathChannels>(&Linker->EntityManager, TaskScheduler);
}

void UObjectPathChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->ObjectPathChannel)
	.Read(BuiltInComponents->EvalTime)
	.Write(BuiltInComponents->ObjectResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateObjectPathChannelTask))
	.Dispatch_PerEntity<FEvaluateObjectPathChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents);
}

