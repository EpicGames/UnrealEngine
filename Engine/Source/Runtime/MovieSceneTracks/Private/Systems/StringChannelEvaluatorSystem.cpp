// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/StringChannelEvaluatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"

#include "EntitySystem/MovieSceneEvalTimeSystem.h"
#include "Channels/MovieSceneStringChannel.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(StringChannelEvaluatorSystem)

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate string channels"), MovieSceneEval_EvaluateStringChannelTask, STATGROUP_MovieSceneECS);

namespace UE::MovieScene
{

struct FEvaluateStringChannels
{
	static void ForEachEntity(FSourceStringChannel StringChannel, FFrameTime FrameTime, FString& OutResult)
	{
		if (const FString* Value = StringChannel.Source->Evaluate(FrameTime))
		{
			OutResult = *Value;
		}
		else
		{
			OutResult = FString();
		}
	}
};


} // namespace UE::MovieScene

UStringChannelEvaluatorSystem::UStringChannelEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = EEntitySystemCategory::ChannelEvaluators;
	RelevantComponent = FBuiltInComponentTypes::Get()->StringChannel;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
	}
}


void UStringChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->StringChannel)
	.Read(BuiltInComponents->EvalTime)
	.Write(BuiltInComponents->StringResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateStringChannelTask))
	.Fork_PerEntity<FEvaluateStringChannels>(&Linker->EntityManager, TaskScheduler);
}

void UStringChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
	.Read(BuiltInComponents->StringChannel)
	.Read(BuiltInComponents->EvalTime)
	.Write(BuiltInComponents->StringResult)
	.FilterNone({ BuiltInComponents->Tags.Ignored })
	.SetStat(GET_STATID(MovieSceneEval_EvaluateStringChannelTask))
	.Dispatch_PerEntity<FEvaluateStringChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents);
}

