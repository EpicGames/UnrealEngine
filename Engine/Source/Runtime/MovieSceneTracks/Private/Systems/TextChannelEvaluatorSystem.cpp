// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/TextChannelEvaluatorSystem.h"
#include "Channels/MovieSceneTextChannel.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEvalTimeSystem.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(TextChannelEvaluatorSystem)

DECLARE_CYCLE_STAT(TEXT("MovieScene: Evaluate text channels"), MovieSceneEval_EvaluateTextChannelTask, STATGROUP_MovieSceneECS);

namespace UE::MovieScene::Private
{
	struct FEvaluateTextChannels
	{
		static void ForEachEntity(FSourceTextChannel TextChannel, FFrameTime FrameTime, FText& OutResult)
		{
			if (const FText* Value = TextChannel.Source->Evaluate(FrameTime))
			{
				OutResult = *Value;
			}
			else
			{
				OutResult = FText::GetEmpty();
			}
		}
	};
}

UTextChannelEvaluatorSystem::UTextChannelEvaluatorSystem(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	using namespace UE::MovieScene;

	SystemCategories  = EEntitySystemCategory::ChannelEvaluators;
	RelevantComponent = FBuiltInComponentTypes::Get()->TextChannel;
	Phase             = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
	}
}

void UTextChannelEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
		.Read(BuiltInComponents->TextChannel)
		.Read(BuiltInComponents->EvalTime)
		.Write(BuiltInComponents->TextResult)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(GET_STATID(MovieSceneEval_EvaluateTextChannelTask))
		.Fork_PerEntity<Private::FEvaluateTextChannels>(&Linker->EntityManager, TaskScheduler);
}

void UTextChannelEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityTaskBuilder()
		.Read(BuiltInComponents->TextChannel)
		.Read(BuiltInComponents->EvalTime)
		.Write(BuiltInComponents->TextResult)
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.SetStat(GET_STATID(MovieSceneEval_EvaluateTextChannelTask))
		.Dispatch_PerEntity<Private::FEvaluateTextChannels>(&Linker->EntityManager, InPrerequisites, &Subsequents);
}
