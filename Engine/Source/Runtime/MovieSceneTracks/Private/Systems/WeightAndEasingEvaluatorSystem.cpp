// Copyright Epic Games, Inc. All Rights Reserved.

#include "Systems/WeightAndEasingEvaluatorSystem.h"
#include "Async/TaskGraphInterfaces.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntityMutations.h"
#include "EntitySystem/MovieSceneRootInstantiatorSystem.h"
#include "EntitySystem/MovieSceneComponentRegistry.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Systems/MovieScenePropertyInstantiator.h"
#include "Systems/MovieSceneMaterialParameterSystem.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"
#include "Sections/MovieSceneSubSection.h"
#include "Systems/FloatChannelEvaluatorSystem.h"
#include "Systems/DoubleChannelEvaluatorSystem.h"
#include "Templates/Greater.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneSection.h"

#include "EntitySystem/MovieSceneEvalTimeSystem.h"

#include "Algo/Find.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(WeightAndEasingEvaluatorSystem)

DECLARE_CYCLE_STAT(TEXT("Weights: Reset"), MovieSceneEval_ResetWeightsTask, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Weights: Evaluate easing"), MovieSceneEval_EvaluateEasingTask, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Weights: Accumulate manual weights"), MovieSceneEval_AccumulateManualWeights, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Weights: Harvest hierarchical easing"), MovieSceneEval_HarvestEasingTask, STATGROUP_MovieSceneECS);
DECLARE_CYCLE_STAT(TEXT("Weights: Propagate hierarchical easing"), MovieSceneEval_PropagateEasing, STATGROUP_MovieSceneECS);


namespace UE::MovieScene
{

static constexpr uint16 INVALID_EASING_CHANNEL = uint16(-1);

float Factorial(int32 In)
{
	check(In >= 0);

	int32 Result = 1;
	for (; In > 0; --In)
	{
		Result *= In;
	}
	return static_cast<float>(Result);
}

enum class EHierarchicalSequenceChannelFlags
{
	None = 0,
	SharedWithParent = 1,
	BlendTarget = 2,
};
ENUM_CLASS_FLAGS(EHierarchicalSequenceChannelFlags);

bool FHierarchicalEasingChannelBuffer::IsEmpty() const
{
	return Channels.Num() == 0;
}

uint16 FHierarchicalEasingChannelBuffer::AddBaseChannel(const FHierarchicalEasingChannelData& InChannelData)
{
	checkf(BlendTargetStartIndex == INDEX_NONE, TEXT("Cannot add base channels once blend targets have been added"));

	const uint16 ChannelID = static_cast<uint16>(Channels.Num());
	Channels.Emplace(InChannelData);
	return ChannelID;
}

uint16 FHierarchicalEasingChannelBuffer::AddBlendTargetChannel(const FHierarchicalEasingChannelData& InChannelData)
{
	if (BlendTargetStartIndex == INDEX_NONE)
	{
		BlendTargetStartIndex = Channels.Num();
	}

	const uint16 ChannelID = static_cast<uint16>(Channels.Num());
	Channels.Emplace(InChannelData);
	return ChannelID;
}

void FHierarchicalEasingChannelBuffer::Reset()
{
	Channels.Empty();
	BlendTargetStartIndex = INDEX_NONE;
}

void FHierarchicalEasingChannelBuffer::ResetBlendTargets()
{
	if (BlendTargetStartIndex != INDEX_NONE)
	{
		Channels.RemoveAtSwap(BlendTargetStartIndex, Channels.Num() - BlendTargetStartIndex);
		BlendTargetStartIndex = INDEX_NONE;
	}
}

/**
 * Class that accumulates active easing data for a particular root sequence hierarchy
 */
struct FRootSequenceData
{
	explicit FRootSequenceData(FRootInstanceHandle InRootInstanceHandle, const FInstanceRegistry* InInstanceRegistry)
		: InstanceRegistry(InInstanceRegistry)
		, RootHierarchy(nullptr) // defer initializing the hierarchy until we need it
		, RootInstanceHandle(InRootInstanceHandle)
	{}

	/** Retrieve the index of the specified sequence ID if it has already been encountered */
	int32 IndexOf(FMovieSceneSequenceID SequenceID) const
	{
		const int32 Index = Algo::LowerBoundBy(SequenceIDs, SequenceID, &FSequenceIDAndChannel::SequenceID);
		if (SequenceIDs.IsValidIndex(Index) && SequenceIDs[Index].SequenceID == SequenceID)
		{
			return Index;
		}
		return INDEX_NONE;
	}

	/* Retrieve the assigned channel ID for the specified sequence ID. Only valid once ProduceChannels has been called. */
	uint16 GetChannelID(FMovieSceneSequenceID SequenceID) const
	{
		const int32 ChannelIndex = IndexOf(SequenceID);
		if (ChannelIndex != INDEX_NONE)
		{
			return SequenceIDs[ChannelIndex].ChannelID;
		}
		return INVALID_EASING_CHANNEL;
	}

	/** Add the specified sequence ID to our map */
	int32 Add(FMovieSceneSequenceID SequenceID, EHierarchicalSequenceChannelFlags InFlags)
	{
		const int32 Index = Algo::LowerBoundBy(SequenceIDs, SequenceID, &FSequenceIDAndChannel::SequenceID);
		check(!SequenceIDs.IsValidIndex(Index) || SequenceIDs[Index].SequenceID != SequenceID);

		SequenceIDs.Insert(FSequenceIDAndChannel{ SequenceID, INVALID_EASING_CHANNEL, InFlags }, Index);

		return Index;
	}

	/** Add the specified sequence ID to our map if it resides as a child underneath one that has an active weight */
	bool AddImplicitChild(FMovieSceneSequenceID SequenceID)
	{
		const bool bHasChannel = IndexOf(SequenceID) != INDEX_NONE;
		if (bHasChannel)
		{
			return true;
		}

		if (!RootHierarchy)
		{
			RootHierarchy = InstanceRegistry->GetInstance(RootInstanceHandle).GetSharedPlaybackState()->GetHierarchy();
		}

		if (ensure(RootHierarchy))
		{
			const FMovieSceneSequenceHierarchyNode* Node = RootHierarchy->FindNode(SequenceID);

			if (ensure(Node) && Node->ParentID != MovieSceneSequenceID::Invalid && AddImplicitChild(Node->ParentID))
			{
				Add(SequenceID, EHierarchicalSequenceChannelFlags::SharedWithParent);
				return true;
			}
		}
		return false;
	}

	/**
	 * Produce channel IDs in the specified output data, ordered parent-first
	 */
	void ProduceChannels(TSparseArray<uint16>& OutInstanceIDToChannel, FHierarchicalEasingChannelBuffer& OutData)
	{
		const FSequenceInstance& RootInstance = InstanceRegistry->GetInstance(RootInstanceHandle);

		for (int32 Index = 0; Index < SequenceIDs.Num(); ++Index)
		{
			ProduceChannelForIndex(Index, RootInstance, OutInstanceIDToChannel, OutData);
		}
	}

	/**
	 * Produce channels for the specified index within our map, including any of its parents
	 */
	uint16 ProduceChannelForIndex(int32 Index, const FSequenceInstance& RootInstance, TSparseArray<uint16>& OutInstanceIDToChannel, FHierarchicalEasingChannelBuffer& OutData)
	{
		const uint16 ExistingChannel = SequenceIDs[Index].ChannelID;
		if (ExistingChannel != INVALID_EASING_CHANNEL)
		{
			return ExistingChannel;
		}

		FSequenceIDAndChannel ChannelData = SequenceIDs[Index];

		if (ChannelData.SequenceID == MovieSceneSequenceID::Root)
		{
			// Add a new channel with no parent
			FHierarchicalEasingChannelData NewData;

			const uint16 ChannelID = OutData.AddBaseChannel(NewData);
			OutInstanceIDToChannel.Insert(RootInstanceHandle.InstanceID, ChannelID);

			SequenceIDs[Index].ChannelID = ChannelID;
			return ChannelID;
		}

		if (!RootHierarchy)
		{
			RootHierarchy = InstanceRegistry->GetInstance(RootInstanceHandle).GetSharedPlaybackState()->GetHierarchy();
		}

		if (!RootHierarchy)
		{
			return INVALID_EASING_CHANNEL;
		}

		// Set up hierarchy info for non-root sequences
		const FMovieSceneSequenceHierarchyNode* Node    = RootHierarchy->FindNode(ChannelData.SequenceID);
		const FMovieSceneSubSequenceData*       SubData = RootHierarchy->FindSubData(ChannelData.SequenceID);
		if (!Node || !SubData)
		{
			return INVALID_EASING_CHANNEL;
		}

		uint16 ParentChannel = INVALID_EASING_CHANNEL;
		while (Node)
		{
			const int32 ParentIndex = IndexOf(Node->ParentID);
			if (ParentIndex != INDEX_NONE)
			{
				ParentChannel = ProduceChannelForIndex(ParentIndex, RootInstance, OutInstanceIDToChannel, OutData);
				break;
			}

			Node = RootHierarchy->FindNode(Node->ParentID);
		}

		// Don't bother creating channels for sequences that don't have an instance
		FInstanceHandle SubInstanceHandle = RootInstance.FindSubInstance(ChannelData.SequenceID);
		if (!SubInstanceHandle.IsValid())
		{
			return ParentChannel;
		}

		if (EnumHasAnyFlags(ChannelData.Flags, EHierarchicalSequenceChannelFlags::SharedWithParent))
		{
			OutInstanceIDToChannel.Insert(SubInstanceHandle.InstanceID, ParentChannel);
			SequenceIDs[Index].ChannelID = ParentChannel;
			return ParentChannel;
		}
		else
		{
			FHierarchicalEasingChannelData NewData;
			NewData.ParentChannel = ParentChannel;
			NewData.HBias = SubData->HierarchicalBias;

			const uint16 ChannelID = OutData.AddBaseChannel(NewData);
			OutInstanceIDToChannel.Insert(SubInstanceHandle.InstanceID, ChannelID);
			SequenceIDs[Index].ChannelID = ChannelID;
			return ChannelID;
		}
	}

private:

	struct FSequenceIDAndChannel
	{
		FMovieSceneSequenceID SequenceID;
		uint16 ChannelID;
		EHierarchicalSequenceChannelFlags Flags;
	};

	const FInstanceRegistry* InstanceRegistry;
	const FMovieSceneSequenceHierarchy* RootHierarchy;
	TArray<FSequenceIDAndChannel, TInlineAllocator<32>> SequenceIDs;
	FRootInstanceHandle RootInstanceHandle;
};

using FRootSequenceDataMap = TSortedMap<FRootInstanceHandle, FRootSequenceData, TInlineAllocator<8>>;

/**
 * Mutation that adds easing channel components to consumers that have active weights
 */
struct FEasingChannelMutationBase : IMovieSceneConditionalEntityMutation
{
	explicit FEasingChannelMutationBase(UMovieSceneEntitySystemLinker* Linker)
	{
		BuiltInComponents = FBuiltInComponentTypes::Get();
		Factories = &Linker->EntityManager.GetComponents()->Factories;
	}

	/**
	 * Create the new entity type for a group of entities
	 */
	void CreateMutation(FEntityManager* InEntityManager, FComponentMask* InOutEntityComponentTypes) const override
	{
		InOutEntityComponentTypes->Set(BuiltInComponents->HierarchicalEasingChannel);
		InOutEntityComponentTypes->Set(BuiltInComponents->Tags.NeedsLink);
	}

	bool HasStaleComponents() const
	{
		return StaleEntities.Num() != 0;
	}

	void RemoveStaleComponents(UMovieSceneEntitySystemLinker* Linker)
	{
		if (StaleEntities.Num())
		{
			FComponentMask ComponentsToRemove;
			ComponentsToRemove.Set(BuiltInComponents->HierarchicalBlendTarget);
			ComponentsToRemove.Set(BuiltInComponents->HierarchicalEasingChannel);
			ComponentsToRemove.Set(BuiltInComponents->WeightAndEasingResult);

			for (FMovieSceneEntityID Entity : StaleEntities)
			{
				Linker->EntityManager.RemoveComponents(Entity, ComponentsToRemove);
			}
		}
	}

protected:

	FBuiltInComponentTypes* BuiltInComponents;
	FEntityFactories* Factories;
	mutable TArray<FMovieSceneEntityID> StaleEntities;
};

/**
 * Mutation that adds or assigns easing channels on easing providers
 */
struct FProviderEasingChannelMutation : FEasingChannelMutationBase
{
	explicit FProviderEasingChannelMutation(UMovieSceneEntitySystemLinker* Linker, const FRootSequenceDataMap* InSequenceDataMap)
		: FEasingChannelMutationBase(Linker)
		, SequenceDataMap(InSequenceDataMap)
	{}

	/**
	 * Called on matching allocations to mark specific entities that need mutating.
	 * If the entity exists within a weighted sequence (directly or indirectly), a channel will be created (or referenced) and assigned
	 */
	void MarkAllocation(FEntityAllocation* Allocation, TBitArray<>& OutEntitiesToMutate) const override
	{
		const FMovieSceneEntityID* EntityIDs = Allocation->GetRawEntityIDs();

		TComponentReader<FRootInstanceHandle>   RootInstanceHandles   = Allocation->ReadComponents(BuiltInComponents->RootInstanceHandle);
		TComponentReader<FMovieSceneSequenceID> EasingProviders       = Allocation->ReadComponents(BuiltInComponents->HierarchicalEasingProvider);
		TOptionalComponentWriter<uint16>        ExistingEasingChannels = Allocation->TryWriteComponents(BuiltInComponents->HierarchicalEasingChannel, FEntityAllocationWriteContext::NewAllocation());

		const bool bHasExistingEasing = ExistingEasingChannels.IsValid();

		// Loop through each entity and check whether it has easing.
		// If so, either assign the new easing channel or mark it for mutation.
		// If not, mark it for removal
		for (int32 Index = 0; Index < Allocation->Num(); ++Index)
		{
			const FRootSequenceData* ChannelData = SequenceDataMap->Find(RootInstanceHandles[Index]);
			const uint16             ChannelID   = ChannelData ? ChannelData->GetChannelID(EasingProviders[Index]) : INVALID_EASING_CHANNEL;

			if (ChannelID != INVALID_EASING_CHANNEL)
			{
				if (bHasExistingEasing)
				{
					ExistingEasingChannels[Index] = ChannelID;
				}
				else
				{
					// Mark this entity for mutation
					OutEntitiesToMutate.PadToNum(Index + 1, false);
					OutEntitiesToMutate[Index] = true;
				}
			}
			else if (bHasExistingEasing)
			{
				StaleEntities.Add(EntityIDs[Index]);
			}
		}
	}

	/**
	 * Initialize a range of new entities with their easing channels
	 */
	void InitializeEntities(const FEntityRange& EntityRange, const FComponentMask& AllocationType) const override
	{
		TComponentReader<FRootInstanceHandle>   RootInstanceHandles = EntityRange.Allocation->ReadComponents(BuiltInComponents->RootInstanceHandle);
		TComponentReader<FMovieSceneSequenceID> EasingProviders     = EntityRange.Allocation->ReadComponents(BuiltInComponents->HierarchicalEasingProvider);
		TComponentWriter<uint16>                EasingChannels      = EntityRange.Allocation->WriteComponents(BuiltInComponents->HierarchicalEasingChannel, FEntityAllocationWriteContext::NewAllocation());

		for (int32 Index = 0; Index < EntityRange.Num; ++Index)
		{
			const int32 Offset = EntityRange.ComponentStartOffset + Index;

			const FRootSequenceData& ChannelData = SequenceDataMap->FindChecked(RootInstanceHandles[Offset]);
			EasingChannels[Offset] = ChannelData.GetChannelID(EasingProviders[Offset]);
		}
	}

private:

	const FRootSequenceDataMap* SequenceDataMap;
};

/**
 * Mutation that adds easing channel components to consumers that have active weights
 */
struct FConsumerEasingChannelMutation : FEasingChannelMutationBase
{
	explicit FConsumerEasingChannelMutation(UMovieSceneEntitySystemLinker* Linker, TSparseArray<uint16>* InInstanceIDToChannel, TMap<TTuple<int16, FHierarchicalBlendTarget>, uint16>* InCachedHierarchicalBlendTargetChannels)
		: FEasingChannelMutationBase(Linker)
		, InstanceIDToChannel(InInstanceIDToChannel)
		, CachedHierarchicalBlendTargetChannels(InCachedHierarchicalBlendTargetChannels)
	{}

	/**
	 * Called on matching allocations to mark specific entities that need mutating.
	 * If the entity exists within a weighted sequence (directly or indirectly), a channel will be created (or referenced) and assigned
	 */
	void MarkAllocation(FEntityAllocation* Allocation, TBitArray<>& OutEntitiesToMutate) const override
	{
		const FMovieSceneEntityID* EntityIDs = Allocation->GetRawEntityIDs();

		TComponentReader<FInstanceHandle>                  InstanceHandles        = Allocation->ReadComponents(BuiltInComponents->InstanceHandle);
		TOptionalComponentWriter<uint16>                   ExistingEasingChannels = Allocation->TryWriteComponents(BuiltInComponents->HierarchicalEasingChannel, FEntityAllocationWriteContext::NewAllocation());
		TOptionalComponentReader<FHierarchicalBlendTarget> BlendTargets           = Allocation->TryReadComponents(BuiltInComponents->HierarchicalBlendTarget);

		const bool bHasExistingEasing = ExistingEasingChannels.IsValid();

		// Loop through each entity and check whether it has easing.
		// If so, either assign the new easing channel or mark it for mutation.
		// If not, mark it for removal
		if (BlendTargets)
		{
			check(CachedHierarchicalBlendTargetChannels);
			TOptionalComponentReader<int16> HierarchicalBias = Allocation->TryReadComponents(BuiltInComponents->HierarchicalBias);

			for (int32 Index = 0; Index < Allocation->Num(); ++Index)
			{
				TTuple<int16, FHierarchicalBlendTarget> BlendTargetKey = MakeTuple(
					HierarchicalBias ? HierarchicalBias[Index] : 0, BlendTargets[Index]
					);

				if (const uint16* BlendTargetChannel = CachedHierarchicalBlendTargetChannels->Find(BlendTargetKey))
				{
					if (bHasExistingEasing)
					{
						ExistingEasingChannels[Index] = *BlendTargetChannel;
					}
					else
					{
						// Mark this entity for mutation
						OutEntitiesToMutate.PadToNum(Index + 1, false);
						OutEntitiesToMutate[Index] = true;
					}
				}
				else if (bHasExistingEasing)
				{
					StaleEntities.Add(EntityIDs[Index]);
				}
			}
		}
		else
		{
			for (int32 Index = 0; Index < Allocation->Num(); ++Index)
			{
				if (InstanceIDToChannel->IsValidIndex(InstanceHandles[Index].InstanceID))
				{
					if (bHasExistingEasing)
					{
						ExistingEasingChannels[Index] = (*InstanceIDToChannel)[InstanceHandles[Index].InstanceID];
					}
					else
					{
						// Mark this entity for mutation
						OutEntitiesToMutate.PadToNum(Index + 1, false);
						OutEntitiesToMutate[Index] = true;
					}
				}
				else if (bHasExistingEasing)
				{
					StaleEntities.Add(EntityIDs[Index]);
				}
			}
		}
	}

	/**
	 * Initialize a range of new entities with their easing channels
	 */
	void InitializeEntities(const FEntityRange& EntityRange, const FComponentMask& AllocationType) const override
	{
		FEntityAllocationWriteContext NewAllocation = FEntityAllocationWriteContext::NewAllocation();

		TComponentWriter<uint16>                           EasingChannels  = EntityRange.Allocation->WriteComponents(BuiltInComponents->HierarchicalEasingChannel, NewAllocation);
		TComponentReader<FInstanceHandle>                  InstanceHandles = EntityRange.Allocation->ReadComponents(BuiltInComponents->InstanceHandle);
		TOptionalComponentReader<FHierarchicalBlendTarget> BlendTargets    = EntityRange.Allocation->TryReadComponents(BuiltInComponents->HierarchicalBlendTarget);

		if (BlendTargets)
		{
			check(CachedHierarchicalBlendTargetChannels);

			TOptionalComponentReader<int16> HierarchicalBias = EntityRange.Allocation->TryReadComponents(BuiltInComponents->HierarchicalBias);

			for (int32 Index = 0; Index < EntityRange.Num; ++Index)
			{
				const int32 Offset = EntityRange.ComponentStartOffset + Index;

				TTuple<int16, FHierarchicalBlendTarget> BlendTargetKey
					= MakeTuple(HierarchicalBias ? HierarchicalBias[Index] : 0, BlendTargets[Index]);

				EasingChannels[Offset] = CachedHierarchicalBlendTargetChannels->FindChecked(BlendTargetKey);
			}
		}
		else
		{
			for (int32 Index = 0; Index < EntityRange.Num; ++Index)
			{
				const int32 Offset = EntityRange.ComponentStartOffset + Index;
				EasingChannels[Offset] = (*InstanceIDToChannel)[InstanceHandles[Offset].InstanceID];
			}
		}
	}

private:

	TSparseArray<uint16>* InstanceIDToChannel;
	TMap<TTuple<int16, FHierarchicalBlendTarget>, uint16>* CachedHierarchicalBlendTargetChannels;
};

/**
 * Reset WeightAndEasingResult components before accumulation
 */
struct FResetFinalWeightResults
{
	static void ForEachEntity(double& Result)
	{
		Result = 1.f;
	}
};

/**
 * Evaluate section easing into EasingResult components
 */
struct FEvaluateEasings
{
	static void ForEachEntity(FFrameTime EvalTime, const FEasingComponentData& Easing, double& Result)
	{
		const double EasingWeight = Easing.Section->EvaluateEasing(EvalTime);
		Result = FMath::Max(EasingWeight, 0.f);
	}
};

/**
 * Accumulate EasingResult and WeightResult components into WeightAndEasingResult components
 */
struct FAccumulateManualWeights
{
	static void ForEachAllocation(const FEntityAllocation* Allocation, const TReadOneOrMoreOf<double, double>& Results, double* OutAccumulatedResults)
	{
		using NumericType = TDecay<decltype(OutAccumulatedResults)>::Type;
		
		const int32 Num = Allocation->Num();

		const double* WeightResults = Results.Get<0>();
		const double* EasingResults = Results.Get<1>();

		check(WeightResults || EasingResults);

		// Have to do math
		if (WeightResults && EasingResults)
		{
			for (int32 Index = 0; Index < Num; ++Index)
			{
				OutAccumulatedResults[Index] = WeightResults[Index] * EasingResults[Index];
			}
		}
		else
		{
			FMemory::Memcpy(OutAccumulatedResults, WeightResults ? WeightResults : EasingResults, Num * sizeof(NumericType));
		}
	}
};

/**
 * Harvest hierarchical WeightAndEasingResult components from HierarchicalEasingProvider components into a linear buffer for propagation to consumers
 */
struct FHarvestHierarchicalEasings
{
	TArrayView<FHierarchicalEasingChannelData> ComputationData;

	FHarvestHierarchicalEasings(TArray<FHierarchicalEasingChannelData>* InComputationData)
		: ComputationData(*InComputationData)
	{
		check(InComputationData);
	}

	// Before the task runs, initialize the results array
	void PreTask()
	{
		for (FHierarchicalEasingChannelData& Data : ComputationData)
		{
			Data.FinalResult = 1.0;
			Data.UnaccumulatedResult = 1.0;
		}
	}

	// Accumulate all entities that contribute to the channel
	void ForEachEntity(double Result, uint16 EasingChannel) const
	{
		ComputationData[EasingChannel].FinalResult *= Result;
		ComputationData[EasingChannel].UnaccumulatedResult *= Result;
	}

	// Multiply hierarchical weights with sub sequences
	void PostTask()
	{
		// Move forward through the results array, multiplying with parents
		// This is possible because the results array is already sorted by depth
		for (int32 Index = 0; Index < ComputationData.Num(); ++Index)
		{
			FHierarchicalEasingChannelData& This = ComputationData[Index];
			switch (This.BlendMode)
			{
#if UE_ENABLE_MOVIESCENE_BEZIER_BLENDING
			case EHierarchicalBlendMode::BezierSeries:
			{
				checkSlow(This.ParentChannel != INVALID_EASING_CHANNEL);

				const float T         = ComputationData[This.ParentChannel].UnaccumulatedResult;
				const float OneMinusT = 1.f - T;

				// Of the form a(1-t)^b + t^c
				This.FinalResult = This.CoeffA * FMath::Pow(OneMinusT, This.ExpB) * FMath::Pow(T, This.ExpC);

				// Result channel is not used for bezier blends
				break;
			}
#endif
			case EHierarchicalBlendMode::ChildFirstBlendTarget:
			{
				checkSlow(This.ParentChannel != INVALID_EASING_CHANNEL);

				const float T          = ComputationData[This.ParentChannel].UnaccumulatedResult;
				const float RemainingT = This.FinalResult;
				const float Result     = RemainingT*T;

				This.FinalResult = FMath::Clamp(Result, 0.f, 1.f);
				if (This.ResultChannel != INVALID_EASING_CHANNEL)
				{
					// Pass on the remaining T to the next channel
					ComputationData[This.ResultChannel].FinalResult = FMath::Clamp(RemainingT-Result, 0.f, 1.f);
				}
				break;
			}
			case EHierarchicalBlendMode::AccumulateParentToChild:
				if (This.ParentChannel != INVALID_EASING_CHANNEL)
				{
					FHierarchicalEasingChannelData Parent = ComputationData[This.ParentChannel];

					// The parent result has already been multiplied by all its parent weights by this point
					This.FinalResult *= Parent.FinalResult;
				}

				if (This.ResultChannel != INVALID_EASING_CHANNEL)
				{
					// Forward onto the result channel
					ComputationData[This.ResultChannel].UnaccumulatedResult *= This.FinalResult;
				}
				break;
			}
		}
	}
};

/**
 * Propagate hierarchical weight values to WeightAndEasingResult components on consumers
 */
struct FPropagateHierarchicalEasings
{
	TArrayView<const FHierarchicalEasingChannelData> ComputationData;

	FPropagateHierarchicalEasings(TArrayView<const FHierarchicalEasingChannelData> InComputationData)
		: ComputationData(InComputationData)
	{}

	void ForEachAllocation(const FEntityAllocation* Allocation, TRead<uint16> HierarchicalEasingChannels, TWrite<double> WeightAndEasingResults) const
	{
		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			const uint16 HierarchicalEasingChannel = HierarchicalEasingChannels[Index];
			if (ensureAlways(ComputationData.IsValidIndex(HierarchicalEasingChannel)))
			{
				WeightAndEasingResults[Index] *= ComputationData[HierarchicalEasingChannel].FinalResult;
			}
		}
	}
};

} // namespace UE::MovieScene


UMovieSceneHierarchicalEasingInstantiatorSystem::UMovieSceneHierarchicalEasingInstantiatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	SystemCategories = EEntitySystemCategory::Core;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		// Has to come after bound object and root instantiation
		DefineImplicitPrerequisite(UMovieSceneRootInstantiatorSystem::StaticClass(), GetClass());
		DefineComponentConsumer(GetClass(), BuiltInComponents->BoundObject);

		// Has to come before property instantiation so that initial values get created correctly
		DefineImplicitPrerequisite(GetClass(), UMovieScenePropertyInstantiatorSystem::StaticClass());
		DefineImplicitPrerequisite(GetClass(), UMovieSceneInterrogatedPropertyInstantiatorSystem::StaticClass());
	}
}

bool UMovieSceneHierarchicalEasingInstantiatorSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	return CachedInstanceIDToChannel.Num()
		|| InLinker->EntityManager.ContainsComponent(FBuiltInComponentTypes::Get()->HierarchicalEasingProvider)
		|| InLinker->EntityManager.ContainsComponent(FBuiltInComponentTypes::Get()->Tags.BlendHierarchicalBias);
}

void UMovieSceneHierarchicalEasingInstantiatorSystem::OnLink()
{
	EvaluatorSystem = Linker->LinkSystem<UWeightAndEasingEvaluatorSystem>();
	// Keep the evaluator system alive as long as we are alive
	Linker->SystemGraph.AddReference(this, EvaluatorSystem);
}

void UMovieSceneHierarchicalEasingInstantiatorSystem::OnUnlink()
{
	CachedInstanceIDToChannel.Empty();
}

void UMovieSceneHierarchicalEasingInstantiatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* const BuiltInComponents = FBuiltInComponentTypes::Get();
	const FInstanceRegistry* const InstanceRegistry = Linker->GetInstanceRegistry();

	// Update if we have any NeedsLink or NeedsUnlink providers (that contribute to hierachical weights)
	FEntityComponentFilter ChangedProviderFilter;
	ChangedProviderFilter.All({ BuiltInComponents->HierarchicalEasingProvider });
	ChangedProviderFilter.Any({ BuiltInComponents->Tags.NeedsLink, BuiltInComponents->Tags.NeedsUnlink });

	// Or any NeedsLink sub instances (that might need to receieve a parent weight)
	FEntityComponentFilter NewSubInstanceFilter;
	NewSubInstanceFilter.All({ BuiltInComponents->Tags.SubInstance, BuiltInComponents->Tags.NeedsLink });

	// Or any NeedsUnlink sub instances that have an easing channel already assigned
	FEntityComponentFilter ExpiringSubInstanceFilter;
	ExpiringSubInstanceFilter.All({ BuiltInComponents->Tags.SubInstance, BuiltInComponents->HierarchicalEasingChannel, BuiltInComponents->Tags.NeedsUnlink });

	// Check if we have any new or expiring easing providers
	bChannelsHaveBeenInvalidated = Linker->EntityManager.Contains(ChangedProviderFilter) || Linker->EntityManager.Contains(NewSubInstanceFilter) || Linker->EntityManager.Contains(ExpiringSubInstanceFilter);
	if (bChannelsHaveBeenInvalidated)
	{
		FRootSequenceDataMap SequenceDataMap;

		// Called for all easing providers to add them to our accumulation map
		auto VisitEasingProvider = [InstanceRegistry, &SequenceDataMap](FMovieSceneEntityID EntityID, FRootInstanceHandle RootInstance, FMovieSceneSequenceID ProviderID)
		{
			if (InstanceRegistry->IsHandleValid(RootInstance))
			{
				// Make the entry for our root instance
				FRootSequenceData* RootSequenceData = SequenceDataMap.Find(RootInstance);
				if (!RootSequenceData)
				{
					RootSequenceData = &SequenceDataMap.Emplace(RootInstance, FRootSequenceData(RootInstance, InstanceRegistry));
				}

				// Add the sequence ID to the channel map if it doesn't already exist
				if (RootSequenceData->IndexOf(ProviderID) == INDEX_NONE)
				{
					RootSequenceData->Add(ProviderID, EHierarchicalSequenceChannelFlags::None);
				}
			}
		};

		// Called for all sub instances to add them to our accumulation map if they reside within a parent that is blended
		auto VisitSubInstance = [InstanceRegistry, &SequenceDataMap](FMovieSceneEntityID EntityID, FRootInstanceHandle RootInstance, FMovieSceneSequenceID SequenceID)
		{
			if (SequenceID != MovieSceneSequenceID::Root && InstanceRegistry->IsHandleValid(RootInstance))
			{
				if (FRootSequenceData* RootSequenceData = SequenceDataMap.Find(RootInstance))
				{
					RootSequenceData->AddImplicitChild(SequenceID);
				}
			}
		};

		// Step 1 : Iterate all easing providers and add their SequenceIDs to the map
		// 
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(BuiltInComponents->HierarchicalEasingProvider)
		.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerEntity(&Linker->EntityManager, VisitEasingProvider);

		// Step 2 : Iterate all sub instances and see if they exist within one of the previously collected easing providers
		// 
		FEntityTaskBuilder()
		.ReadEntityIDs()
		.Read(BuiltInComponents->RootInstanceHandle)
		.Read(BuiltInComponents->SequenceID)
		.FilterAll({ BuiltInComponents->Tags.SubInstance })
		.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
		.Iterate_PerEntity(&Linker->EntityManager, VisitSubInstance);

		// Step 3: Produce parent-first channel IDs for anything we collected
		//
		{
			FHierarchicalEasingChannelBuffer& Buffer = EvaluatorSystem->GetComputationBuffer();
			Buffer.Reset();
			CachedInstanceIDToChannel.Empty();

			for (TPair<FRootInstanceHandle, FRootSequenceData>& Pair : SequenceDataMap)
			{
				Pair.Value.ProduceChannels(CachedInstanceIDToChannel, Buffer);
			}
		}

		// Step 4: Assign channel IDs to easing providers, potentially adding or removing invalid ones
		//
		{
			FEntityComponentFilter ProviderFilter;
			ProviderFilter.All({ BuiltInComponents->RootInstanceHandle, BuiltInComponents->HierarchicalEasingProvider });
			ProviderFilter.None({ BuiltInComponents->Tags.NeedsUnlink });

			FProviderEasingChannelMutation ProviderMutation(Linker, &SequenceDataMap);
			Linker->EntityManager.MutateConditional(ProviderFilter, ProviderMutation, EMutuallyInclusiveComponentType::All);

			ProviderMutation.RemoveStaleComponents(Linker);
		}

		// Step 5: Assign easing channels for anything else that is left and remove stale ones
		//
		{
			FEntityComponentFilter AssignEasingChannelsFilter;
			AssignEasingChannelsFilter.All({ BuiltInComponents->InstanceHandle});
			AssignEasingChannelsFilter.None({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->HierarchicalEasingProvider, BuiltInComponents->HierarchicalBlendTarget, BuiltInComponents->Tags.ImportedEntity });

			FConsumerEasingChannelMutation ConsumerMutation(Linker, &CachedInstanceIDToChannel, nullptr);
			Linker->EntityManager.MutateConditional(AssignEasingChannelsFilter, ConsumerMutation, EMutuallyInclusiveComponentType::All);

			ConsumerMutation.RemoveStaleComponents(Linker);
		}
	}
	else
	{
		// Just add easing channels to NeedsLink entities
		FEntityComponentFilter AssignEasingChannelsFilter;
		AssignEasingChannelsFilter.All({ BuiltInComponents->InstanceHandle, BuiltInComponents->Tags.NeedsLink  });
		AssignEasingChannelsFilter.None({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->HierarchicalEasingProvider, BuiltInComponents->HierarchicalBlendTarget, BuiltInComponents->Tags.ImportedEntity });

		FConsumerEasingChannelMutation ConsumerMutation(Linker, &CachedInstanceIDToChannel, nullptr);
		Linker->EntityManager.MutateConditional(AssignEasingChannelsFilter, ConsumerMutation, EMutuallyInclusiveComponentType::All);

		ConsumerMutation.RemoveStaleComponents(Linker);
	}


	FinalizeBlendTargets();
}

void UMovieSceneHierarchicalEasingInstantiatorSystem::FinalizeBlendTargets()
{
	using namespace UE::MovieScene;

	const FBuiltInComponentTypes* const BuiltInComponents = FBuiltInComponentTypes::Get();

	FEntityComponentFilter NewBlendTargetFilter;
	NewBlendTargetFilter.All({ BuiltInComponents->HierarchicalBlendTarget, BuiltInComponents->Tags.NeedsLink });

	if (bChannelsHaveBeenInvalidated || Linker->EntityManager.Contains(NewBlendTargetFilter))
	{
		FHierarchicalEasingChannelBuffer& Buffer = EvaluatorSystem->GetComputationBuffer();

		const int32 NumStartingChannels = Buffer.Channels.Num();

		// Map from HBias to whether it is a blend target or not
		struct FHBiasChannelData
		{
			uint16 AccumulatorChannelID = UE::MovieScene::INVALID_EASING_CHANNEL;
		};
		TSortedMap<int16, FHBiasChannelData, TInlineAllocator<16>, TGreater<>> HBiasToChannelData;
		TSet<FHierarchicalBlendTarget, DefaultKeyFuncs<FHierarchicalBlendTarget>, TInlineSetAllocator<16>> BlendTargetChannelData;

		// If we have invalidated our channel list we need to recompute everything.
		//    Otherwise we will only visit NeedsLink entities and create channels for anything that's missing
		if (bChannelsHaveBeenInvalidated)
		{
			Buffer.ResetBlendTargets();
			CachedHierarchicalBlendTargetChannels.Empty();
		}

		// Step 1: Gather all the hierarchical blend targets
		//
		{
			auto Task = FEntityTaskBuilder()
			.Read(BuiltInComponents->HierarchicalBlendTarget)
			.FilterNone({ BuiltInComponents->Tags.NeedsUnlink });
			if (!bChannelsHaveBeenInvalidated)
			{
				Task.FilterAll({ BuiltInComponents->Tags.NeedsLink });
			}
			Task.Iterate_PerEntity(&Linker->EntityManager,
				[&BlendTargetChannelData](const FHierarchicalBlendTarget& BlendTarget)
				{
					BlendTargetChannelData.Add(BlendTarget);
				}
			);

			for (const FHierarchicalBlendTarget& BlendTarget : BlendTargetChannelData)
			{
				// Ensure that there are hbias channels for each hbias in the chain
				for (int16 HBias : BlendTarget.AsArray())
				{
					HBiasToChannelData.FindOrAdd(HBias);
				}
			}
		}

		// Step 2: Create channels for each blend target sorted by bias
		//
		{
			for (TPair<int16, FHBiasChannelData>& Pair : HBiasToChannelData)
			{
				Pair.Value.AccumulatorChannelID = Buffer.AddBlendTargetChannel(FHierarchicalEasingChannelData());
			}

			// Point any pre-existing easing channels to their HBias outputs
			for (int32 ChannelIndex = 0; ChannelIndex < NumStartingChannels; ++ChannelIndex)
			{
				FHierarchicalEasingChannelData& Channel = Buffer.Channels[ChannelIndex];
				const FHBiasChannelData* ChannelData = HBiasToChannelData.Find(Channel.HBias);
				if (ChannelData)
				{
					Channel.ResultChannel = ChannelData->AccumulatorChannelID;
				}
			}
		}

		// Step 3: Create chains of channels for each HBias designated as a blend target
		{
			for (const FHierarchicalBlendTarget& BlendTarget : BlendTargetChannelData)
			{
				// This is a blend target - create channels for its entire parent chain
				uint16 ParentChannel = INVALID_EASING_CHANNEL;

				const int32 N = BlendTarget.Num();
				int32 I = N;

				for (int16 SourceHBias : BlendTarget.AsArray())
				{
					if (CachedHierarchicalBlendTargetChannels.Contains(MakeTuple(SourceHBias, BlendTarget)))
					{
						// This should only occur when bChannelsHaveBeenInvalidated is false
						continue;
					}

					FHierarchicalEasingChannelData BlendTargetChannel;
					// Get our value from the accumulator channel for the source bias which includes all the accumulated weights for that HBias level.
					// Normally this should just be a single weight
					BlendTargetChannel.ParentChannel = HBiasToChannelData.FindChecked(SourceHBias).AccumulatorChannelID;
					BlendTargetChannel.BlendMode = EHierarchicalBlendMode::ChildFirstBlendTarget;

#if UE_ENABLE_MOVIESCENE_BEZIER_BLENDING
					BlendTargetChannel.CoeffA = Factorial(N) / (Factorial(I) * Factorial(N-I));
					BlendTargetChannel.ExpB   = static_cast<float>(N-I);
					BlendTargetChannel.ExpC   = static_cast<float>(I);
					BlendTargetChannel.BlendMode = EHierarchicalBlendMode::BezierSeries;
#else
					BlendTargetChannel.BlendMode = EHierarchicalBlendMode::ChildFirstBlendTarget;
#endif
					const uint16 BlendTargetChannelID = Buffer.AddBlendTargetChannel(BlendTargetChannel);

					if (N != I)
					{
						// If this isn't the first one, connect the last one to this one so we can
						// combine weights hierarchically up the chain
						Buffer.Channels[Buffer.Channels.Num()-2].ResultChannel = BlendTargetChannelID;
					}

					CachedHierarchicalBlendTargetChannels.Add(MakeTuple(SourceHBias, BlendTarget), BlendTargetChannelID);

					--I;
				}
			}
		}
	}

	// Assign easing channels
	{
		FEntityComponentFilter AssignEasingChannelsFilter;
		AssignEasingChannelsFilter.All({ BuiltInComponents->InstanceHandle, BuiltInComponents->HierarchicalBlendTarget });
		if (bChannelsHaveBeenInvalidated == false)
		{
			// If channels have not been invalidated, only visit new blend target entities to assign their channels
			AssignEasingChannelsFilter.All({ BuiltInComponents->Tags.NeedsLink });
		}
		AssignEasingChannelsFilter.None({ BuiltInComponents->Tags.NeedsUnlink, BuiltInComponents->HierarchicalEasingProvider, BuiltInComponents->Tags.ImportedEntity });

		FConsumerEasingChannelMutation ConsumerMutation(Linker, &CachedInstanceIDToChannel, &CachedHierarchicalBlendTargetChannels);
		Linker->EntityManager.MutateConditional(AssignEasingChannelsFilter, ConsumerMutation, EMutuallyInclusiveComponentType::All);

		ensure(bChannelsHaveBeenInvalidated || !ConsumerMutation.HasStaleComponents());
		ConsumerMutation.RemoveStaleComponents(Linker);
	}


#if UE_MOVIESCENE_EXPENSIVE_CONSISTENCY_CHECKS
	FHierarchicalEasingChannelBuffer& Buffer = EvaluatorSystem->GetComputationBuffer();

	// Check that all hierarchical easing components map to a valid channel
	FEntityTaskBuilder()
	.Read(BuiltInComponents->HierarchicalEasingChannel)
	.FilterNone({ BuiltInComponents->Tags.NeedsUnlink })
	.Iterate_PerAllocation(&Linker->EntityManager, [Buffer](const FEntityAllocation* Allocation, TRead<uint16> Channels) {

		for (int32 Index = 0; Index < Allocation->Num(); ++Index)
		{
			const uint16 Channel = Channels[Index];
			ensureAlways(Buffer.Channels.IsValidIndex(Channel));
		}
	});
#endif
}

UMovieSceneHierarchicalEasingFinalizationSystem::UMovieSceneHierarchicalEasingFinalizationSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
}

UWeightAndEasingEvaluatorSystem::UWeightAndEasingEvaluatorSystem(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	using namespace UE::MovieScene;

	SystemCategories = UE::MovieScene::EEntitySystemCategory::ChannelEvaluators;
	Phase = ESystemPhase::Scheduling;

	if (HasAnyFlags(RF_ClassDefaultObject))
	{
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		DefineComponentConsumer(GetClass(), BuiltInComponents->WeightResult);

		DefineImplicitPrerequisite(UMovieSceneEvalTimeSystem::StaticClass(), GetClass());
		DefineComponentProducer(GetClass(), BuiltInComponents->EasingResult);
		DefineComponentProducer(GetClass(), BuiltInComponents->WeightAndEasingResult);
	}
}

bool UWeightAndEasingEvaluatorSystem::IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	return InLinker->EntityManager.ContainsAnyComponent({ Components->WeightAndEasingResult });
}

void UWeightAndEasingEvaluatorSystem::OnLink()
{
	PreAllocatedComputationData.Reset();
}

void UWeightAndEasingEvaluatorSystem::OnUnlink()
{
	PreAllocatedComputationData.Reset();
}

UE::MovieScene::FHierarchicalEasingChannelBuffer& UWeightAndEasingEvaluatorSystem::GetComputationBuffer()
{
	return PreAllocatedComputationData;
}

void UWeightAndEasingEvaluatorSystem::OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	// Reset all weights back to one. This can happen immediately.
	//
	FTaskID ResetWeights = FEntityTaskBuilder()
	.Write(Components->WeightAndEasingResult)
	.SetStat(GET_STATID(MovieSceneEval_ResetWeightsTask))
	.Fork_PerEntity<FResetFinalWeightResults>(&Linker->EntityManager, TaskScheduler);

	// Evaluate easing - this can be scheduled as soon as eval times are populated
	//
	FTaskID EvaluateEasing = FEntityTaskBuilder()
	.Read(Components->EvalTime)
	.Read(Components->Easing)
	.Write(Components->EasingResult)
	.SetStat(GET_STATID(MovieSceneEval_EvaluateEasingTask))
	.Fork_PerEntity<FEvaluateEasings>(&Linker->EntityManager, TaskScheduler);

	// Accumulate weights - must be done after ResetWeights and EvaluateEasing
	//
	FTaskID AccumulateWeights = FEntityTaskBuilder()
	.ReadOneOrMoreOf(Components->WeightResult, Components->EasingResult)
	.Write(Components->WeightAndEasingResult)
	.SetStat(GET_STATID(MovieSceneEval_AccumulateManualWeights))
	.Schedule_PerAllocation<FAccumulateManualWeights>(&Linker->EntityManager, TaskScheduler);

	TaskScheduler->AddPrerequisite(ResetWeights, AccumulateWeights);
	TaskScheduler->AddPrerequisite(EvaluateEasing, AccumulateWeights);

	// If we have hierarchical easing, we initialize all the weights to their hierarchical defaults
	if (!PreAllocatedComputationData.IsEmpty())
	{
		// Step 2: Harvest any hierarchical results from providers
		//
		FTaskID HarvestTask = FEntityTaskBuilder()
		.Read(Components->WeightAndEasingResult)
		.Read(Components->HierarchicalEasingChannel)
		.FilterAll({ Components->HierarchicalEasingProvider })  // Only harvest results from entities that are providing results
		.SetParams(FTaskParams(GET_STATID(MovieSceneEval_HarvestEasingTask)).ForcePrePostTask())
		.Schedule_PerEntity<FHarvestHierarchicalEasings>(&Linker->EntityManager, TaskScheduler, &PreAllocatedComputationData.Channels);

		// Step 3: Apply hierarchical easing results to all entities inside affected sub-sequences.
		//
		FTaskID PropagateTask = FEntityTaskBuilder()
		.Read(Components->HierarchicalEasingChannel)
		.Write(Components->WeightAndEasingResult)
		.FilterNone({ Components->HierarchicalEasingProvider }) // Do not propagate hierarchical weights onto providers!
		.SetStat(GET_STATID(MovieSceneEval_PropagateEasing))
		.Fork_PerAllocation<FPropagateHierarchicalEasings>(&Linker->EntityManager, TaskScheduler, PreAllocatedComputationData.Channels);

		TaskScheduler->AddPrerequisite(HarvestTask, PropagateTask);
	}
}

void UWeightAndEasingEvaluatorSystem::OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	// No hierarchical weighting, just reset everything to 1.0
	FGraphEventRef ResetWeights = FEntityTaskBuilder()
	.Write(Components->WeightAndEasingResult)
	.SetStat(GET_STATID(MovieSceneEval_EvaluateEasingTask))
	.Dispatch_PerEntity<FResetFinalWeightResults>(&Linker->EntityManager, InPrerequisites, &Subsequents);

	FSystemTaskPrerequisites ResetWeightsDependencies {InPrerequisites};
	ResetWeightsDependencies.AddComponentTask(Components->WeightAndEasingResult, ResetWeights);

	// Step 1: Evaluate section easing and manual weights in parallel
	//
	FGraphEventRef EvaluateEasing = FEntityTaskBuilder()
	.Read(Components->EvalTime)
	.Read(Components->Easing)
	.Write(Components->EasingResult)
	.SetStat(GET_STATID(MovieSceneEval_EvaluateEasingTask))
	.Dispatch_PerEntity<FEvaluateEasings>(&Linker->EntityManager, ResetWeightsDependencies, &Subsequents);

	ResetWeightsDependencies.AddComponentTask(Components->EasingResult, EvaluateEasing);

	FGraphEventRef AccumulateManualWeights = FEntityTaskBuilder()
	.ReadOneOrMoreOf(Components->WeightResult, Components->EasingResult)
	.Write(Components->WeightAndEasingResult)
	.SetStat(GET_STATID(MovieSceneEval_EvaluateEasingTask))
	.Dispatch_PerAllocation<FAccumulateManualWeights>(&Linker->EntityManager, ResetWeightsDependencies, &Subsequents);

	// If we have hierarchical easing, we initialize all the weights to their hierarchical defaults
	if (!PreAllocatedComputationData.IsEmpty())
	{
		FSystemTaskPrerequisites HarvestPrereqs {InPrerequisites};
		HarvestPrereqs.AddComponentTask(Components->WeightAndEasingResult, EvaluateEasing);
		HarvestPrereqs.AddComponentTask(Components->WeightAndEasingResult, AccumulateManualWeights);

		// Step 2: Harvest any hierarchical results from providers
		//
		FGraphEventRef HarvestTask = FEntityTaskBuilder()
		.Read(Components->WeightAndEasingResult)
		.Read(Components->HierarchicalEasingChannel)
		.FilterAll({ Components->HierarchicalEasingProvider })  // Only harvest results from entities that are providing results
		.SetStat(GET_STATID(MovieSceneEval_HarvestEasingTask))
		.Dispatch_PerEntity<FHarvestHierarchicalEasings>(&Linker->EntityManager, HarvestPrereqs, nullptr, &PreAllocatedComputationData.Channels);

		FSystemTaskPrerequisites PropagatePrereqs {InPrerequisites};
		PropagatePrereqs.AddRootTask(HarvestTask);

		// Step 3: Apply hierarchical easing results to all entities inside affected sub-sequences.
		//
		FEntityTaskBuilder()
		.Read(Components->HierarchicalEasingChannel)
		.Write(Components->WeightAndEasingResult)
		.FilterNone({ Components->HierarchicalEasingProvider }) // Do not propagate hierarchical weights onto providers!
		.SetStat(GET_STATID(MovieSceneEval_EvaluateEasingTask))
		.Dispatch_PerAllocation<FPropagateHierarchicalEasings>(&Linker->EntityManager, PropagatePrereqs, &Subsequents,
			PreAllocatedComputationData.Channels);
	}
}
