// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreTypes.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "Conditions/MovieScenePlatformCondition.h"
#include "Engine/Engine.h"
#include "Misc/AutomationTest.h"
#include "Sections/MovieSceneFadeSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Sections/MovieSceneTimeWarpSection.h"
#include "Tests/MovieSceneTestDataBuilders.h"
#include "Tracks/MovieSceneFadeTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Tracks/MovieSceneTimeWarpTrack.h"
#include "UObject/Package.h"
#include "Variants/MovieScenePlayRateCurve.h"

#if WITH_DEV_AUTOMATION_TESTS

#define LOCTEXT_NAMESPACE "MovieSceneCompiledDataManagerTests"

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieSceneCompiledDataManagerGCReferenceTest,
		"System.Engine.Sequencer.Compilation.GCReferenceTests", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneCompiledDataManagerGCReferenceTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	TStrongObjectPtr<UMovieSceneCompiledDataManager> CompiledData(
		NewObject<UMovieSceneCompiledDataManager>(GetTransientPackage(), "GCReferenceTest")
	);

	UMovieSceneTimeWarpSection* TimeWarpSection = nullptr;
	UMovieSceneSubSection*      SubSection      = nullptr;
	UMovieSceneFadeSection*     FadeSection     = nullptr;

	FSequenceBuilder RootSequenceBuilder;
	{
		RootSequenceBuilder
		.AddRootTrack<UMovieSceneTimeWarpTrack>()
			.AddSection(0, 96000)
			.Assign(TimeWarpSection)
			.Pop()
		.Pop()
		.AddRootTrack<UMovieSceneFadeTrack>()
			.AddSection(0, 96000)
			.Assign(FadeSection)
			.Pop()
		.Pop()
		.AddRootTrack<UMovieSceneSubTrack>()
			.AddSection(0, 2400)
			.Assign(SubSection)
			.Pop()
		.Pop();
	}

	// Add a condition to ensure that they do not hold a strong reference in the compiled data
	FadeSection->ConditionContainer.Condition = NewObject<UMovieScenePlatformCondition>(FadeSection);

	// Build an empty sub sequence - we only need this so we can create a hierarchy and shove
	//     its timewarp into the compiled data manager for testing
	FSequenceBuilder SubSequenceBuilder;

	TWeakObjectPtr<UMovieSceneSequence> WeakRootSequence = RootSequenceBuilder.Sequence;
	TWeakObjectPtr<UMovieSceneSequence> WeakSubSequence  = SubSequenceBuilder.Sequence;

	TWeakObjectPtr<UMovieScenePlayRateCurve> WeakRootTimeWarp;
	TWeakObjectPtr<UMovieScenePlayRateCurve> WeakSubTimeWarp;

	// Add root time warp
	{
		UMovieScenePlayRateCurve* PlayRate = NewObject<UMovieScenePlayRateCurve>(SubSequenceBuilder.Sequence);

		WeakRootTimeWarp = PlayRate;

		TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = PlayRate->PlayRate.GetData();
		ChannelData.AddKey(0, FMovieSceneDoubleValue(1.0));
		ChannelData.AddKey(24000, FMovieSceneDoubleValue(0.5));
		ChannelData.AddKey(48000, FMovieSceneDoubleValue(1.0));

		TimeWarpSection->TimeWarp.Set(PlayRate);
	}

	// Setup the subsequence and time warp
	{
		SubSection->SetSequence(SubSequenceBuilder.Sequence);

		UMovieScenePlayRateCurve* SubPlayRate = NewObject<UMovieScenePlayRateCurve>(SubSequenceBuilder.Sequence);

		WeakSubTimeWarp = SubPlayRate;

		TMovieSceneChannelData<FMovieSceneDoubleValue> ChannelData = SubPlayRate->PlayRate.GetData();
		ChannelData.AddKey(0, FMovieSceneDoubleValue(0.25));

		SubSection->Parameters.TimeScale.Set(SubPlayRate);
	}


	// Compile the sequence, run a GC (which should clean up all the sequences), and verify that the compiled data manager has removed everything

	if (WeakRootSequence.Get()    == nullptr
		|| WeakSubSequence.Get()  == nullptr
		|| WeakRootTimeWarp.Get() == nullptr
		|| WeakSubTimeWarp.Get()  == nullptr)
	{
		AddError(TEXT("Test failed to initialize weak pointers correctly."));
		return false;
	}

	CompiledData->Compile(RootSequenceBuilder.Sequence);

	FMovieSceneCompiledDataID RootDataID = CompiledData->FindDataID(RootSequenceBuilder.Sequence);
	FMovieSceneCompiledDataID SubDataID  = CompiledData->FindDataID(SubSequenceBuilder.Sequence);

	if (!RootDataID.IsValid() || !SubDataID.IsValid())
	{
		AddError(TEXT("Sequence was not compiled successfully."));
		return false;
	}

	if (CompiledData->FindHierarchy(RootDataID) == nullptr)
	{
		AddError(TEXT("Sequence did not compile a hierarchy when it should have."));
		return false;
	}

	CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);

	if (WeakRootSequence.Get()    != nullptr
		|| WeakSubSequence.Get()  != nullptr
		|| WeakRootTimeWarp.Get() != nullptr
		|| WeakSubTimeWarp.Get()  != nullptr)
	{
		if (UObject* Object = WeakRootSequence.Get())
		{
			GEngine->Exec(nullptr, *FString::Printf(TEXT("obj refs name=\"%s\""), *Object->GetPathName()));
		}
		if (UObject* Object = WeakSubSequence.Get())
		{
			GEngine->Exec(nullptr, *FString::Printf(TEXT("obj refs name=\"%s\""), *Object->GetPathName()));
		}
		if (UObject* Object = WeakRootTimeWarp.Get())
		{
			GEngine->Exec(nullptr, *FString::Printf(TEXT("obj refs name=\"%s\""), *Object->GetPathName()));
		}
		if (UObject* Object = WeakSubTimeWarp.Get())
		{
			GEngine->Exec(nullptr, *FString::Printf(TEXT("obj refs name=\"%s\""), *Object->GetPathName()));
		}

		AddError(TEXT("Objects were unexpectedly still alive after GC please see logs for references."));
		return false;
	}

	if (CompiledData->FindHierarchy(RootDataID) != nullptr)
	{
		AddError(TEXT("Compiled data was not correctly cleaned up."));
		return false;
	}

	return true;
}

#undef LOCTEXT_NAMESPACE

#endif // WITH_DEV_AUTOMATION_TESTS

