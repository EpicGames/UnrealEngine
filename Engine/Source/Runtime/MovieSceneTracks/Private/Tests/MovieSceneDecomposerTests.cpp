// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/MovieSceneDecomposerTests.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogatedPropertyInstantiator.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "IMovieScenePlayer.h"
#include "Misc/AutomationTest.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Tests/MovieSceneTestDataBuilders.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieSceneCinematicShotTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDecomposerTests)

#define LOCTEXT_NAMESPACE "MovieSceneDecomposerTests"

namespace UE::MovieScene::Test
{

template<typename PropertyTraits>
TRecompositionResult<typename PropertyTraits::StorageType>
RecomposeBlendOperational(
		UObject* InObject,
		TArray<UMovieSceneSection*> InPropertySections,
		const UE::MovieScene::TPropertyComponents<PropertyTraits>& InPropertyComponents,
		const typename PropertyTraits::StorageType& InCurrentValue,
		FFrameTime InCurrentTime = FFrameTime(0))
{
	using namespace UE::MovieScene;

	// Create the interrogator.
	FSystemInterrogator Interrogator;
	Interrogator.TrackImportedEntities(true);
	
	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &Interrogator.GetLinker()->EntityManager);

	// All given sections should belong to the same track.
	check(InPropertySections.Num() > 0);
	UMovieScenePropertyTrack* PropertyTrack = InPropertySections[0]->GetTypedOuter<UMovieScenePropertyTrack>();
	check(PropertyTrack);

	// Run an interrogation on the track at the specified time.
	FInterrogationKey InterrogationKey(FInterrogationKey::Default());
	FInterrogationChannel InterrogationChannel = Interrogator.AllocateChannel(InObject, PropertyTrack->GetPropertyBinding());
	InterrogationKey.Channel = InterrogationChannel;
	Interrogator.ImportTrack(PropertyTrack, InterrogationChannel);

	Interrogator.AddInterrogation(InCurrentTime);
	Interrogator.Update();

	// Gather the entity IDs for the sections we're interested in.
	TArray<FMovieSceneEntityID> EntityIDs;
	for (UMovieSceneSection* PropertySection : InPropertySections)
	{
		const FMovieSceneEntityID EntityID = Interrogator.FindEntityFromOwner(InterrogationKey, PropertySection, 0);
		if (ensure(EntityID.IsValid()))
		{
			EntityIDs.Add(EntityID);
		}
	}
	if (EntityIDs.IsEmpty())
	{
		return TRecompositionResult<typename PropertyTraits::StorageType>(0, 1);
	}

	// Run the recomposition on those sections' entities.
	UMovieSceneInterrogatedPropertyInstantiatorSystem* System = Interrogator.GetLinker()->FindSystem<UMovieSceneInterrogatedPropertyInstantiatorSystem>();

	FDecompositionQuery Query;
	Query.Entities = EntityIDs;
	Query.bConvertFromSourceEntityIDs = false;
	Query.Object = InObject;
	TRecompositionResult<typename PropertyTraits::StorageType> RecomposeResult = System->RecomposeBlendOperational(
			InPropertyComponents, Query, InCurrentValue);

	return RecomposeResult;
}

template<typename PropertyTraits>
TRecompositionResult<typename PropertyTraits::StorageType>
RecomposeBlendOperational(
	UObject* InObject,
	UMovieSceneSection* InPropertySection,
	const UE::MovieScene::TPropertyComponents<PropertyTraits>& InPropertyComponents,
	const typename PropertyTraits::StorageType& InCurrentValue,
	FFrameTime InCurrentTime = FFrameTime(0))
{
	return RecomposeBlendOperational(InObject, TArray<UMovieSceneSection*>({ InPropertySection }), InPropertyComponents, InCurrentValue, InCurrentTime);
}

} // namespace UE::MovieScene::Test

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieSceneDecomposerAbsoluteTest, 
		"System.Engine.Sequencer.Decomposer.Absolute", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneDecomposerAbsoluteTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	UMovieSceneSection* TestObjectFloatSection;
	UMovieSceneDecomposerTestObject* TestObject = NewObject<UMovieSceneDecomposerTestObject>();

	// Absolute section setting the value at 100. Find what it would take to make it 120. It's simple, it should be 120.
	FSequenceBuilder()
		.AddObjectBinding(TestObject)
		.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieSceneDecomposerTestObject, FloatProperty))
			.AddSection(0, 5000)
				.Assign(TestObjectFloatSection)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 100.f)
			.Pop()
		.Pop();

	FMovieSceneTracksComponentTypes* ComponentTypes = FMovieSceneTracksComponentTypes::Get();
	TRecompositionResult<double> Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection, ComponentTypes->Float, 120.f);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 120.0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMovieSceneDecomposerAbsoluteWithEasingTest,
	"System.Engine.Sequencer.Decomposer.AbsoluteWithEasing",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
	bool FMovieSceneDecomposerAbsoluteWithEasingTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	UMovieSceneSection* TestObjectFloatSection;
	UMovieSceneDecomposerTestObject* TestObject = NewObject<UMovieSceneDecomposerTestObject>();
	FMovieSceneTracksComponentTypes* ComponentTypes = FMovieSceneTracksComponentTypes::Get();

	// We need to set the initial value cache manually for now, because of limitations of the interrogator.
	TestObject->FloatProperty = 20.f;
	TSharedPtr<FInitialValueCache> InitialValueCache = FInitialValueCache::GetGlobalInitialValues();
	InitialValueCache->GetStorage<FFloatPropertyTraits>(ComponentTypes->Float.InitialValue)->AddInitialValue(TestObject, 20.0, offsetof(UMovieSceneDecomposerTestObject, FloatProperty));

	// Absolute section setting the value at 100. But it has linear ease-in and we consider the time at which the easing is at 50%.
	// This means it brings in 50% of the default value, which we set to 20. The result is (20/2 + 50/2) = 35.
	FSequenceBuilder()
		.AddObjectBinding(TestObject)
		.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieSceneDecomposerTestObject, FloatProperty))
			.AddSection(0, 5000)
				.Assign(TestObjectFloatSection)
				.SetEaseIn(1000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 100.f)
			.Pop()
		.Pop();

	// Find what it would take to make the result 120. It should be 220, so that (20/2 + 220/2) = 120.
	TRecompositionResult<double> Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection, ComponentTypes->Float, 120.f, 500);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 220.0);

	// Run the same but at time 0, when the weight is 0%. This should hit the special case where we just key the value.
	Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection, ComponentTypes->Float, 120.f, 0);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 120.0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieSceneDecomposerMultipleAbsolutesTest,
		"System.Engine.Sequencer.Decomposer.MultipleAbsolutes",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneDecomposerMultipleAbsolutesTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	UMovieSceneSection* TestObjectFloatSection1;
	UMovieSceneSection* TestObjectFloatSection2;
	UMovieSceneDecomposerTestObject* TestObject = NewObject<UMovieSceneDecomposerTestObject>();

	// One absolute section sets the value to 50, and the other to 150. When they're both at full weight, it gives
	// a result of 100. We add some ease in to the second section in order to test how weight also factors into it.
	FSequenceBuilder()
		.AddObjectBinding(TestObject)
		.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieSceneDecomposerTestObject, FloatProperty))
			.AddSection(0, 5000)
				.Assign(TestObjectFloatSection1)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 50.f)
			.Pop()
			.AddSection(0, 5000)
				.Assign(TestObjectFloatSection2)
				.SetEaseIn(1000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 150.f)
			.Pop()
		.Pop();

	// To get 120, we need the second section to be at 190, so that (50 + 190)/2 = 120.
	FMovieSceneTracksComponentTypes* ComponentTypes = FMovieSceneTracksComponentTypes::Get();
	TRecompositionResult<double> Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection2, ComponentTypes->Float, 120.f, 2000);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 190.0);

	// Now try again at a time when the ease-in on the second section is active and at 50%. To get to 120, we need the
	// first section to be at 105, so that (105 + 150/2)/1.5 = 120.
	Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection1, ComponentTypes->Float, 120.f, 500);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 105.0);

	// If we recompose the second section, it needs to be at 260, so that (50 + 260/2)/1.5 = 120.
	Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection2, ComponentTypes->Float, 120.f, 500);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 260.0);

	// And now we recompose both at the same time. When the ease-in is at 50%, the result is 83.3333 so getting to
	// 120 means we are missing +36.6666, to share among both sections.
	// This sharing is done based on the weights of the sections at the given time, which is 100% and 50% respectively.
	// So the first section will get two thirds of this, and the second will get one third. Of course, that one third
	// gets in practice bumped up to two thirds since that second section has 50% weight, which means we need to double
	// the value to get the desired effect. Each section gets +36.666 to reach the desired value.
	// Bottom line, we get (86.666 + 186.666/2)/1.5 = 120 (give or take).
	Result = RecomposeBlendOperational(TestObject, { TestObjectFloatSection1, TestObjectFloatSection2 }, ComponentTypes->Float, 120.f, 500);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 2);
	UTEST_EQUAL_TOLERANCE("Value to key", Result.Values[0], 86.66666667, 0.0001);
	UTEST_EQUAL_TOLERANCE("Value to key", Result.Values[1], 186.66666667, 0.0001);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieSceneDecomposerAdditiveTest,
		"System.Engine.Sequencer.Decomposer.Additive", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneDecomposerAdditiveTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	UMovieSceneSection* TestObjectFloatSection1;
	UMovieSceneSection* TestObjectFloatSection2;
	UMovieSceneDecomposerTestObject* TestObject = NewObject<UMovieSceneDecomposerTestObject>();

	// Absolute section setting the value to 100, with an additive on top worth +10. Figure out what it would take for the additive
	// to give us a final value of 120. It should make the additive +20.
	FSequenceBuilder()
		.AddObjectBinding(TestObject)
		.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieSceneDecomposerTestObject, FloatProperty))
			.AddSection(0, 5000)
				.Assign(TestObjectFloatSection1)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 100.f)
			.Pop()
			.AddSection(0, 5000, 1)
				.Assign(TestObjectFloatSection2)
				.SetBlendType(EMovieSceneBlendType::Additive)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 10.f)
			.Pop()
		.Pop();

	FMovieSceneTracksComponentTypes* ComponentTypes = FMovieSceneTracksComponentTypes::Get();
	TRecompositionResult<double> Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection2, ComponentTypes->Float, 120.f);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 20.0);

	// Now let's decompose both sections. It should put the whole difference on the additive section, leaving the
	// absolute section as-is.
	Result = RecomposeBlendOperational(TestObject, { TestObjectFloatSection1, TestObjectFloatSection2 }, ComponentTypes->Float, 120.f);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 2);
	UTEST_EQUAL("Value to key", Result.Values[0], 100.0);
	UTEST_EQUAL("Value to key", Result.Values[1], 20.0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieSceneDecomposerAdditiveWithEasingTest,
		"System.Engine.Sequencer.Decomposer.AdditiveWithEasing", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneDecomposerAdditiveWithEasingTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	UMovieSceneSection* TestObjectFloatSection1;
	UMovieSceneSection* TestObjectFloatSection2;
	UMovieSceneDecomposerTestObject* TestObject = NewObject<UMovieSceneDecomposerTestObject>();

	// Absolute section setting the value to 100, with an additive on top worth +10. There is ease-in on the additive,
	// and we'll be considering the time at which the weight is 50% and the additive only contributes +5.
	FSequenceBuilder()
		.AddObjectBinding(TestObject)
		.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieSceneDecomposerTestObject, FloatProperty))
			.AddSection(0, 5000)
				.Assign(TestObjectFloatSection1)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 100.f)
			.Pop()
			.AddSection(0, 5000, 1)
				.Assign(TestObjectFloatSection2)
				.SetBlendType(EMovieSceneBlendType::Additive)
				.SetEaseIn(2000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 10.f)
			.Pop()
		.Pop();

	// Figure out what it would take for the additive to give us a final value of 120. It should make the additive +40,
	// since it only has a weight of 50% at the evaluation time.
	FMovieSceneTracksComponentTypes* ComponentTypes = FMovieSceneTracksComponentTypes::Get();
	TRecompositionResult<double> Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection2, ComponentTypes->Float, 120.f, 1000);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 40.0);

	// Run the decomposition for both sections at the same time. The absolute section shouldn't change, as we put all
	// the difference on the additive in this sort of case.
	Result = RecomposeBlendOperational(TestObject, { TestObjectFloatSection1, TestObjectFloatSection2 }, ComponentTypes->Float, 120.f, 1000);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 2);
	UTEST_EQUAL("Value to key", Result.Values[0], 100.0);
	UTEST_EQUAL("Value to key", Result.Values[1], 40.0);

	// Run the same but at time 0, where the weight is 0%. This should hit the special case where we just key the
	// difference in value between the absolute and the desired value.
	Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection2, ComponentTypes->Float, 120.f, 0);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 20.0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMovieSceneDecomposerOverrideWithEasingTest,
	"System.Engine.Sequencer.Decomposer.OverrideWithEasing",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneDecomposerOverrideWithEasingTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	UMovieSceneSection* TestObjectFloatSection1;
	UMovieSceneSection* TestObjectFloatSection2;
	UMovieSceneSection* TestObjectFloatSection3;
	UMovieSceneSection* TestObjectFloatSection4;

	UMovieSceneDecomposerTestObject* TestObject = NewObject<UMovieSceneDecomposerTestObject>();

	// Absolute section setting the value to 100, with an additive on top worth +100.
	// There is ease-in on the additive.  There is then an Override with an ease-in with a value of 0.
	// On top of that is another additive with a value of 25.
	// With weight of 0.5 the total value is (100 + (0.5) *100) *0.5  +(0.0) *0.5  + 25 = 100
	// With weight of 1.0 the total value is (100 + 100) *0.0 + 0.0 + 25 = 25
	FSequenceBuilder()
		.AddObjectBinding(TestObject)
		.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieSceneDecomposerTestObject, FloatProperty))
		.AddSection(0, 5000)
		.Assign(TestObjectFloatSection1)
		.AddKey<FMovieSceneFloatChannel, float>(0, 0, 100.f)
		.Pop()
		.AddSection(0, 5000, 1)
		.Assign(TestObjectFloatSection2)
		.SetBlendType(EMovieSceneBlendType::Additive)
		.SetEaseIn(2000)
		.AddKey<FMovieSceneFloatChannel, float>(0, 0, 100.f)
		.Pop()
		.AddSection(0, 5000, 1)
		.Assign(TestObjectFloatSection3)
		.SetBlendType(EMovieSceneBlendType::Override)
		.SetEaseIn(2000)
		.AddKey<FMovieSceneFloatChannel, float>(0, 0, 0.0f)
		.Pop()
		.AddSection(0, 5000, 1)
		.Assign(TestObjectFloatSection4)
		.SetBlendType(EMovieSceneBlendType::Additive)
		.AddKey<FMovieSceneFloatChannel, float>(0, 0, 25.0f)
		.Pop()
		.Pop();

	// Figure out what it would take for the final additive to give us a final value of 125. It should make the additive 50,
	// since it only has a weight of 50% at the evaluation time.
	FMovieSceneTracksComponentTypes* ComponentTypes = FMovieSceneTracksComponentTypes::Get();
	TRecompositionResult<double> Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection4, ComponentTypes->Float, 125.f, 1000);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 50.0);

	// Run at time 2000, where the weight is 100%. With a total of 100.0,  the Override should be 75, which is 100 - 25(additive) value.
	// The absolute and previous additive values don't matter, they have been overriden.
	Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection3, ComponentTypes->Float, 100.0f, 2000);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 75.0);

	return true;
}


IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieSceneDecomposerAdditiveWithMultipleEasingTest,
		"System.Engine.Sequencer.Decomposer.AdditiveWithMultipleEasing", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneDecomposerAdditiveWithMultipleEasingTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	UMovieSceneSection* TestObjectFloatSection;
	UMovieSceneDecomposerTestObject* TestObject = NewObject<UMovieSceneDecomposerTestObject>();

	// Absolute section setting the value to 100, with an additive on top worth +10. Both sections have ease-in and
	// ease-out in order to test various combinations and edge cases.
	// Absolute section: ease-in [0, 1000[, ease-out [4500, 5000[
	// Additive section: ease-in [0, 500[,  ease-out [4000, 5000[
	FSequenceBuilder()
		.AddObjectBinding(TestObject)
		.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieSceneDecomposerTestObject, FloatProperty))
			.AddSection(0, 5000)
				.SetEaseIn(1000).SetEaseOut(500)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 100.f)
			.Pop()
			.AddSection(0, 5000, 1)
				.Assign(TestObjectFloatSection)
				.SetBlendType(EMovieSceneBlendType::Additive)
				.SetEaseIn(500).SetEaseOut(1000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 10.f)
			.Pop()
		.Pop();

	// At time 250, absolute is 25% (25) and additive is 50% (+5), for a total of 30. To get to 50 we need another 20.
	// The additive being at 50%, this means we need 40, so additive should be +50.
	FMovieSceneTracksComponentTypes* ComponentTypes = FMovieSceneTracksComponentTypes::Get();
	TRecompositionResult<double> Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection, ComponentTypes->Float, 50.f, 250);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 50.0);

	// At time 750, absolute is 75% (75) and additive is 100% (+10), for a total of 85. To get to 100 we need another
	// 15. So the additive should be +25.
	Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection, ComponentTypes->Float, 100.f, 750);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 25.0);

	// At time 0, both sections have 0% weight. We should hit the fallback code which just keys the desired value.
	// It's not a super awesome value in this case but no value is really awesome anyway.
	Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection, ComponentTypes->Float, 120.f, 0);

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 120.0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieSceneDecomposerAdditiveFromBaseTest,
		"System.Engine.Sequencer.Decomposer.AdditiveFromBase",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneDecomposerAdditiveFromBaseTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	UMovieSceneSection* TestObjectFloatSection;
	UMovieSceneDecomposerTestObject* TestObject = NewObject<UMovieSceneDecomposerTestObject>();

	// Absolute section setting a value of 100, with an additive that goes from +0 to +5 over time using an
	// additive-from-base blend with a base value of 50. In order to reach a final value of 120, we need to
	// set the additive to 70, so that (70-50) = +20.
	FSequenceBuilder()
		.AddObjectBinding(TestObject)
		.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieSceneDecomposerTestObject, FloatProperty))
			.AddSection(0, 5000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 100.f)
			.Pop()
			.AddSection(0, 5000, 1)
				.Assign(TestObjectFloatSection)
				.SetBlendType(EMovieSceneBlendType::AdditiveFromBase)
				.AddKeys<FMovieSceneFloatChannel, float>(0, { 0, 3000 }, { 50.f, 55.f })
			.Pop()
		.Pop();

	FMovieSceneTracksComponentTypes* ComponentTypes = FMovieSceneTracksComponentTypes::Get();
	TRecompositionResult<double> Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection, ComponentTypes->Float, 120.f, FFrameTime(4000));

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 70.0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieSceneDecomposerAdditiveFromBaseWithEasingTest,
		"System.Engine.Sequencer.Decomposer.AdditiveFromBaseWithEasing",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneDecomposerAdditiveFromBaseWithEasingTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	UMovieSceneSection* TestObjectFloatSection;
	UMovieSceneDecomposerTestObject* TestObject = NewObject<UMovieSceneDecomposerTestObject>();
	FMovieSceneTracksComponentTypes* ComponentTypes = FMovieSceneTracksComponentTypes::Get();

	// We need to set the initial value cache manually for now, because of limitations of the interrogator.
	TestObject->FloatProperty = 20.f;
	TSharedPtr<FInitialValueCache> InitialValueCache = FInitialValueCache::GetGlobalInitialValues();
	InitialValueCache->GetStorage<FFloatPropertyTraits>(ComponentTypes->Float.InitialValue)->AddInitialValue(TestObject, 20.0, offsetof(UMovieSceneDecomposerTestObject, FloatProperty));

	// Absolute section setting a value of 100, with an additive that goes from +0 to +10 over time using an
	// additive-from-base blend with a base value of 50.
	FSequenceBuilder()
		.AddObjectBinding(TestObject)
		.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieSceneDecomposerTestObject, FloatProperty))
			.AddSection(0, 5000)
				.SetEaseOut(1000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 100.f)
			.Pop()
			.AddSection(0, 5000, 1)
				.Assign(TestObjectFloatSection)
				.SetBlendType(EMovieSceneBlendType::AdditiveFromBase)
				.SetEaseIn(2000)
				.AddKeys<FMovieSceneFloatChannel, float>(0, { 0, 1000 }, { 50.f, 60.f })
			.Pop()
		.Pop();

	// By the time the additive is at +10, its ease-in keeps it at 50%. If we recompose at that time for a target value
	// of 120, we need the additive to be at 90, so that (90-50) = +40 x 50% = +20.
	TRecompositionResult<double> Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection, ComponentTypes->Float, 120.f, FFrameTime(1000));

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 90.0);

	// Try again, but at a later time, when the additive is at 100% weight, but the absolute is at 50% with its
	// ease-out. Since the initial value is 20, the full absolute value is (100 x 0.5 + 20 x 0.5) = 60. The additive
	// needs to add +60 to reach 120, so we should get 110.
	Result = RecomposeBlendOperational(TestObject, TestObjectFloatSection, ComponentTypes->Float, 120.f, FFrameTime(4500));

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 1);
	UTEST_EQUAL("Value to key", Result.Values[0], 110.0);

	return true;
}

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
		FMovieSceneDecomposerComplexTest,
		"System.Engine.Sequencer.Decomposer.Complex",
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieSceneDecomposerComplexTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	UMovieSceneSection *TestObjectFloatSectionAbsolute1, *TestObjectFloatSectionAbsolute2;
	UMovieSceneSection *TestObjectFloatSectionAdditive1, *TestObjectFloatSectionAdditive2;
	UMovieSceneSection *TestObjectFloatSectionAdditiveFromBase1, *TestObjectFloatSectionAdditiveFromBase2;
	UMovieSceneDecomposerTestObject* TestObject = NewObject<UMovieSceneDecomposerTestObject>();
	FMovieSceneTracksComponentTypes* ComponentTypes = FMovieSceneTracksComponentTypes::Get();

	// We need to set the initial value cache manually for now, because of limitations of the interrogator.
	TestObject->FloatProperty = 20.f;
	TSharedPtr<FInitialValueCache> InitialValueCache = FInitialValueCache::GetGlobalInitialValues();
	InitialValueCache->GetStorage<FFloatPropertyTraits>(ComponentTypes->Float.InitialValue)->AddInitialValue(TestObject, 20.0, offsetof(UMovieSceneDecomposerTestObject, FloatProperty));

	// Ok let's pull all the stops here. It's like a bad joke: three absolute sections, three additive sections, and
	// three additive-from-base sections walk into a bar... anyway, we're going to decompose two of each, leaving the
	// third as non-decomposed. We also add ease-in on all of them, so we can also test with weights.
	FSequenceBuilder()
		.AddObjectBinding(TestObject)
		.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieSceneDecomposerTestObject, FloatProperty))
			// Absolute sections are at 50, 60, 70
			.AddSection(0, 5000)
				.SetEaseIn(1000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 50.f)
			.Pop()
			.AddSection(0, 5000, 1)
				.Assign(TestObjectFloatSectionAbsolute1)
				.SetEaseIn(1000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 60.f)
			.Pop()
			.AddSection(0, 5000, 2)
				.Assign(TestObjectFloatSectionAbsolute2)
				.SetEaseIn(1000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 70.f)
			.Pop()
			// Additive sections are at +10, +30, +40
			.AddSection(0, 5000, 2)
				.SetBlendType(EMovieSceneBlendType::Additive)
				.SetEaseIn(1000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 10.f)
			.Pop()
			.AddSection(0, 5000, 3)
				.Assign(TestObjectFloatSectionAdditive1)
				.SetBlendType(EMovieSceneBlendType::Additive)
				.SetEaseIn(1000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 30.f)
			.Pop()
			.AddSection(0, 5000, 4)
				.Assign(TestObjectFloatSectionAdditive2)
				.SetBlendType(EMovieSceneBlendType::Additive)
				.SetEaseIn(1000)
				.AddKey<FMovieSceneFloatChannel, float>(0, 0, 40.f)
			.Pop()
			// Additive-from-base sections are at +5, +10, +20
			.AddSection(0, 5000, 5)
				.SetBlendType(EMovieSceneBlendType::AdditiveFromBase)
				.SetEaseIn(1000)
				.AddKeys<FMovieSceneFloatChannel, float>(0, { 0, 500 }, { 50.f, 55.f })
			.Pop()
			.AddSection(0, 5000, 6)
				.Assign(TestObjectFloatSectionAdditiveFromBase1)
				.SetBlendType(EMovieSceneBlendType::AdditiveFromBase)
				.SetEaseIn(1000)
				.AddKeys<FMovieSceneFloatChannel, float>(0, { 0, 500 }, { 50.f, 60.f })
			.Pop()
			.AddSection(0, 5000, 7)
				.Assign(TestObjectFloatSectionAdditiveFromBase2)
				.SetBlendType(EMovieSceneBlendType::AdditiveFromBase)
				.SetEaseIn(1000)
				.AddKeys<FMovieSceneFloatChannel, float>(0, { 0, 500 }, { 50.f, 70.f })
			.Pop()
		.Pop();

	// Decompose all the values at time 2000, when all weights are 100%. At that time, we have:
	//
	// (50 + 60 + 70)/3 + (10 + 30 + 40) + ((55 - 50) + (60 - 50) + (70 - 50)) = 175
	//
	// If we want to reach 275, we will leave all the absolutes alone and distribute the difference across the
	// additives. The absolutes give out (50 + 60 + 70)/3 = 60, so we need +215. We have 4 decomposed additives, and
	// two that we don't decompose. These two contribute +15, so there's +200 left to redistribute between our additives.
	// These 4 decomposed additives originally contirbuted +100, which makes it easy to compute their contribution
	// factors: it's their value in percentage! So Additive1 will get 30% of +200, Additive2 will get 40% of +200, and
	// so on.
	TRecompositionResult<double> Result = RecomposeBlendOperational(
			TestObject, 
			{ TestObjectFloatSectionAbsolute1, TestObjectFloatSectionAbsolute2,
			  TestObjectFloatSectionAdditive1, TestObjectFloatSectionAdditive2,
			  TestObjectFloatSectionAdditiveFromBase1, TestObjectFloatSectionAdditiveFromBase2 },
			ComponentTypes->Float, 
			275.f,
			FFrameTime(1000));

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 6);
	UTEST_EQUAL("Value to key (absolute 1)", Result.Values[0], 60.0);
	UTEST_EQUAL("Value to key (absolute 2)", Result.Values[1], 70.0);
	UTEST_EQUAL("Value to key (additive 1)", Result.Values[2], 60.0);
	UTEST_EQUAL("Value to key (additive 2)", Result.Values[3], 80.0);
	UTEST_EQUAL("Value to key (additive from base 1)", Result.Values[4], 70.0);
	UTEST_EQUAL("Value to key (additive from base 2)", Result.Values[5], 90.0);

	// Now let's try again at a time when all the weights are at 50%. We have:
	//
	// (50/2 + 60/2 + 70/2)/1.5 + (10/2 + 30/2 + 40/2) + ((55 - 50)/2 + (60 - 50)/2 + (70 - 50)/2) = 117.5
	//
	// If we want to get 275, the same happens as previously, only the differences applied to the additives get doubled
	// to account for the half-weights. The absolutes give out 60, and the two non-decomposed additives give +7.5. So
	// we're missing (275 - 60 - 7.5) = 207.5 which will be proportionally spread out between the decomposed additives
	// (since, once again, we don't change the decomposed absolutes). Additive1 gets 30% of 207.5, Additive2 gets 40% of
	// 207.5, and so on. Of course, these values get doubled to account for these additives being at half-weight.
	Result = RecomposeBlendOperational(
			TestObject, 
			{ TestObjectFloatSectionAbsolute1, TestObjectFloatSectionAbsolute2,
			  TestObjectFloatSectionAdditive1, TestObjectFloatSectionAdditive2,
			  TestObjectFloatSectionAdditiveFromBase1, TestObjectFloatSectionAdditiveFromBase2 },
			ComponentTypes->Float, 
			275.f,
			FFrameTime(500));

	UTEST_EQUAL("Recomposed values", Result.Values.Num(), 6);
	UTEST_EQUAL("Value to key (absolute 1)", Result.Values[0], 60.0);
	UTEST_EQUAL("Value to key (absolute 2)", Result.Values[1], 70.0);
	UTEST_EQUAL("Value to key (additive 1)", Result.Values[2], 124.5);
	UTEST_EQUAL("Value to key (additive 2)", Result.Values[3], 166.0);
	UTEST_EQUAL("Value to key (additive from base 1)", Result.Values[4], 91.5);
	UTEST_EQUAL("Value to key (additive from base 2)", Result.Values[5], 133.0);
	
	return true;
}

#undef LOCTEXT_NAMESPACE

