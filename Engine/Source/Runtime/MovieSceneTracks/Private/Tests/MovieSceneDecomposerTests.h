// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"

#include "MovieSceneDecomposerTests.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneDecomposerTestObject : public UObject
{
	GENERATED_BODY()

public:
	UPROPERTY()
	float FloatProperty = 0.f;
};

