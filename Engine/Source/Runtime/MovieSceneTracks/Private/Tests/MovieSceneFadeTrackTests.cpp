// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/MovieSceneFadeTrackTests.h"

#include "Camera/PlayerCameraManager.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneFadeTrackTests)

float UMovieSceneFadeTrackTestLibrary::GetManualFadeAmount(APlayerCameraManager* PlayerCameraManager)
{
	return PlayerCameraManager ? PlayerCameraManager->FadeAmount : 0.f;
}

