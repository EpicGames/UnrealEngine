// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/MovieScenePartialEvaluationTests.h"
#include "Tests/MovieSceneTestDataBuilders.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "IMovieScenePlayer.h"
#include "Evaluation/MovieSceneEvaluationTemplateInstance.h"
#include "Compilation/MovieSceneCompiledDataManager.h"
#include "MovieSceneSequence.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "Sections/MovieSceneFloatSection.h"
#include "Sections/MovieSceneSubSection.h"
#include "Misc/AutomationTest.h"
#include "UObject/StrongObjectPtr.h"
#include "UObject/Package.h"
#include "Engine/Engine.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePartialEvaluationTests)

#if WITH_DEV_AUTOMATION_TESTS


IMPLEMENT_SIMPLE_AUTOMATION_TEST(FMovieScenePartialEvaluationTest, 
		"System.Engine.Sequencer.PartialEvaluation", 
		EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)
bool FMovieScenePartialEvaluationTest::RunTest(const FString& Parameters)
{
	using namespace UE::MovieScene;
	using namespace UE::MovieScene::Test;

	TStrongObjectPtr<UMovieScenePartialEvaluationTestObject> TestObject(NewObject<UMovieScenePartialEvaluationTestObject>());

	auto MakeSubSequence = [&TestObject]
	{
		return FSequenceBuilder()
			.AddObjectBinding(TestObject.Get())
			.AddPropertyTrack<UMovieSceneFloatTrack>(GET_MEMBER_NAME_CHECKED(UMovieScenePartialEvaluationTestObject, FloatProperty))
				.AddSection(0, 5000)
					.AddKey<FMovieSceneFloatChannel, float>(0, 0, 100.f)
				.Pop()
			.Pop()
			.Sequence;
	};

	// Sub-sequences are intentionally not strong ptrs so they can be GC'd
	TWeakObjectPtr<UMovieSceneSequence> SubSequenceA = MakeSubSequence();
	TWeakObjectPtr<UMovieSceneSequence> SubSequenceB = MakeSubSequence();

	UMovieSceneSubTrack* SubTrack = nullptr;

	UMovieSceneSubSection* SubSectionA = nullptr;
	UMovieSceneSubSection* SubSectionB = nullptr;

	TStrongObjectPtr<UMovieSceneSequence> RootSequence(FSequenceBuilder()
		.AddRootTrack<UMovieSceneSubTrack>()
			.Assign(SubTrack)
			.AddSection(0, 5000, 0)
				.Do<UMovieSceneSubSection>([SubSequenceA](UMovieSceneSubSection* SubSection){ SubSection->SetSequence(SubSequenceA.Get()); })
				.Assign(SubSectionA)
			.Pop()
			.AddSection(0, 5000, 1)
				.Do<UMovieSceneSubSection>([SubSequenceB](UMovieSceneSubSection* SubSection){ SubSection->SetSequence(SubSequenceB.Get()); })
				.Assign(SubSectionB)
			.Pop()
		.Pop()
	.Sequence.Get());

	UMovieSceneCompiledDataManager* CompiledDataManager = UMovieSceneCompiledDataManager::GetPrecompiledData();
	TStrongObjectPtr<UMovieSceneEntitySystemLinker> Linker(NewObject<UMovieSceneEntitySystemLinker>(GetTransientPackage()));
	TSharedPtr<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	TGuardValue<FEntityManager*> DebugVizGuard(GEntityManagerForDebuggingVisualizers, &Linker->EntityManager);

	struct FLocalPlayer : IMovieScenePlayer
	{
		FMovieSceneRootEvaluationTemplateInstance Template;
		virtual FMovieSceneRootEvaluationTemplateInstance& GetEvaluationTemplate() override { return Template; }
		virtual UMovieSceneEntitySystemLinker* ConstructEntitySystemLinker() override { return TestLinker; }
		virtual void UpdateCameraCut(UObject* CameraObject, const EMovieSceneCameraCutParams& CameraCutParams) override {}
		virtual void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) override {}
		virtual void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const override {}
		virtual EMovieScenePlayerStatus::Type GetPlaybackStatus() const override { return EMovieScenePlayerStatus::Playing; }
		virtual void SetPlaybackStatus(EMovieScenePlayerStatus::Type InPlaybackStatus) override {}

		UMovieSceneEntitySystemLinker* TestLinker;
	};

	FLocalPlayer Player;
	Player.TestLinker = Linker.Get();

	FMovieSceneCompiledDataID DataID = CompiledDataManager->Compile(RootSequence.Get());
	Player.Template.Initialize(*RootSequence, Player, CompiledDataManager);

	bool bIsActive = true;

	ERunnerFlushState FlushStates[] = {
		ERunnerFlushState::Start,
		ERunnerFlushState::Import,
		ERunnerFlushState::Spawn,
		ERunnerFlushState::Instantiation,
		ERunnerFlushState::Evaluation,
		ERunnerFlushState::Finalization,
		ERunnerFlushState::EventTriggers,
		ERunnerFlushState::PostEvaluation,
		ERunnerFlushState::End,
	};

	Runner->QueueUpdate(FMovieSceneContext(FMovieSceneEvaluationRange(FFrameTime(0), RootSequence->GetMovieScene()->GetTickResolution()), EMovieScenePlayerStatus::Playing), Player.Template.GetRootInstanceHandle());

	FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	auto RunTest = [this, &Runner, &Linker, FlushStates, InstanceRegistry](TArrayView<UMovieSceneSubSection* const> SubSections)
	{
		for (int32 Index = 0; Index < UE_ARRAY_COUNT(FlushStates); ++Index)
		{
			ERunnerFlushState ThisFlushState = FlushStates[Index];

			FString BeforeString = FString::Printf(TEXT("VOLATILITY: Instance Handle should be valid before Flush State %d"), (int32)ThisFlushState);
			FString AfterString = FString::Printf(TEXT("VOLATILITY: Instance Handle should be valid after Flush State %d"), (int32)ThisFlushState);

			// Check that all instance handles are still valid
			FEntityTaskBuilder()
			.Read(FBuiltInComponentTypes::Get()->InstanceHandle)
			.Iterate_PerEntity(&Linker->EntityManager, [this, InstanceRegistry, &BeforeString](FInstanceHandle InstanceHandle){
				this->TestTrue(*BeforeString, InstanceRegistry->IsHandleValid(InstanceHandle));
			});
			
			for (UMovieSceneSubSection* SubSection : SubSections)
			{
				SubSection->SetIsActive(!SubSection->IsActive());
			}
			Runner->Flush(0.0, ThisFlushState);

			FEntityTaskBuilder()
			.Read(FBuiltInComponentTypes::Get()->InstanceHandle)
			.Iterate_PerEntity(&Linker->EntityManager, [this, InstanceRegistry, &AfterString](FInstanceHandle InstanceHandle){
				this->TestTrue(*AfterString, InstanceRegistry->IsHandleValid(InstanceHandle));
			});
		}
	};

	SubSectionA->SetIsActive(true);
	RunTest(MakeArrayView(&SubSectionA, 1));

	SubSectionA->SetIsActive(false);
	RunTest(MakeArrayView(&SubSectionA, 1));

	SubSectionA->SetIsActive(true);
	SubSectionB->SetIsActive(false);
	RunTest(MakeArrayView({ SubSectionA, SubSectionB }));

	SubSectionA->SetIsActive(false);
	SubSectionB->SetIsActive(false);
	RunTest(MakeArrayView({ SubSectionA, SubSectionB }));

	// Run the test with a garbage collection in between each
	for (int32 Index = 0; Index < UE_ARRAY_COUNT(FlushStates); ++Index)
	{
		ERunnerFlushState ThisFlushState = FlushStates[Index];

		FString BeforeString = FString::Printf(TEXT("GC: Instance Handle should be valid before Flush State %d"), (int32)ThisFlushState);
		FString AfterString = FString::Printf(TEXT("GC: Instance Handle should be valid after Flush State %d"), (int32)ThisFlushState);

		// Check that all instance handles are still valid
		FEntityTaskBuilder()
		.Read(FBuiltInComponentTypes::Get()->InstanceHandle)
		.Iterate_PerEntity(&Linker->EntityManager, [this, InstanceRegistry, &BeforeString](FInstanceHandle InstanceHandle){
			this->TestTrue(*BeforeString, InstanceRegistry->IsHandleValid(InstanceHandle));
		});

		Runner->Flush(0.0, ThisFlushState);

		// Remove the sub section and force a garbage collection. This should collect the sub sequence and destroy the instance immediately
		SubTrack->RemoveSection(*SubSectionA);
		SubTrack->MarkAsChanged();
		SubTrack->GetEvaluationField(); // Force update of the evaluation field to remove any references (really this should reference sections by weak ptr)
		SubSectionA = nullptr;

		// Collect garbage! This should have the effect of destroying the compiled data
		CollectGarbage(GARBAGE_COLLECTION_KEEPFLAGS, true);

		if (SubSequenceA.Get() != nullptr)
		{
			this->AddError(TEXT("Sub Sequence should have been garbage collected but was not!"));
			GEngine->Exec(nullptr, *FString::Printf(TEXT("obj refs name=\"%s\""), *SubSequenceA->GetPathName()));
		}

		SubSequenceA = MakeSubSequence();

		// Create a new sub section
		SubSectionA = CastChecked<UMovieSceneSubSection>(SubTrack->CreateNewSection());
		SubSectionA->SetRange(TRange<FFrameNumber>(0, 5000));
		SubSectionA->SetRowIndex(0);
		SubSectionA->SetSequence(SubSequenceA.Get());

		SubTrack->AddSection(*SubSectionA);

		FEntityTaskBuilder()
		.Read(FBuiltInComponentTypes::Get()->InstanceHandle)
		.Iterate_PerEntity(&Linker->EntityManager, [this, InstanceRegistry, &AfterString](FInstanceHandle InstanceHandle){
			this->TestTrue(*AfterString, InstanceRegistry->IsHandleValid(InstanceHandle));
		});
	}



	return true;
}

#endif // WITH_DEV_AUTOMATION_TESTS
