// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tests/MovieSceneTestDataBuilders.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTestDataBuilders)

void UMovieSceneTestSequence::Initialize()
{
	MovieScene = NewObject<UMovieScene>(this);
}

FGuid UMovieSceneTestSequence::AddObjectBinding(TObjectPtr<UObject> InObject)
{
	FGuid NewBindingGuid = MovieScene->AddPossessable(InObject->GetName(), InObject->GetClass());

	BoundObjects.Add(InObject);
	BindingGuids.Add(NewBindingGuid);

	return NewBindingGuid;
}

void UMovieSceneTestSequence::LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const
{
	int32 Index = BindingGuids.Find(ObjectId);
	if (ensure(Index != INDEX_NONE))
	{
		OutObjects.Add(BoundObjects[Index]);
	}
}

