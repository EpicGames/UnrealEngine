// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackInstances/MovieSceneCVarTrackInstance.h"
#include "Sections/MovieSceneCVarSection.h"
#include "Sections/MovieSceneConsoleVariableTrackInterface.h"
#include "Tracks/MovieSceneCVarTrack.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/TrackInstance/MovieSceneTrackInstanceSystem.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSource.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateExtension.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCVarTrackInstance)

namespace UE
{
namespace MovieScene
{

struct FPreAnimatedCVarTraits : FPreAnimatedStateTraits
{
	using KeyType     = FString;
	using StorageType = FString;

	void RestorePreAnimatedValue(const FString& InKey, const FString& CachedValue, const FRestoreStateParams& Params)
	{
		static IConsoleManager& ConsoleManager = IConsoleManager::Get();
		IConsoleVariable* ConsoleVariable = ConsoleManager.FindConsoleVariable(*InKey);
		if (ConsoleVariable)
		{
			ConsoleVariable->SetWithCurrentPriority(*CachedValue);
		}
	}

	static FString CachePreAnimatedValue(const FString& InKey)
	{
		static IConsoleManager& ConsoleManager = IConsoleManager::Get();

		IConsoleVariable* ConsoleVariable = ConsoleManager.FindConsoleVariable(*InKey);
		return ConsoleVariable ? ConsoleVariable->GetString() : FString();
	}
};
struct FPreAnimatedCVarStorage : TPreAnimatedStateStorage<FPreAnimatedCVarTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedCVarStorage> StorageID;
};
TAutoRegisterPreAnimatedStorageID<FPreAnimatedCVarStorage> FPreAnimatedCVarStorage::StorageID;

} // namespace MovieScene
} // namespace UE

void UMovieSceneCVarTrackInstance::OnBeginUpdateInputs()
{
	CVarsNeedingUpdate.Reset();
}

void UMovieSceneCVarTrackInstance::OnAnimate()
{
}

void UMovieSceneCVarTrackInstance::OnInputAdded(const FMovieSceneTrackInstanceInput& InInput)
{
	using namespace UE::MovieScene;

	UMovieSceneEntitySystemLinker* Linker  = GetLinker();
	UMovieSceneCVarSection*        Section = CastChecked<UMovieSceneCVarSection>(InInput.Section);

	FScopedPreAnimatedCaptureSource CaptureSource(Linker, InInput);

	// Capture pre-animated state for all the cvars
	if (Linker->PreAnimatedState.IsCapturingGlobalState() || CaptureSource.WantsRestoreState())
	{
		static const FPreAnimatedStorageGroupHandle UngroupedHandle;

		TSharedPtr<FPreAnimatedCVarStorage> CVarStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedCVarStorage>();

		// Gather all CVars
		TArray<TPair<FString, FString>> CommandsAndValues = Section->ConsoleVariables.ValuesByCVar.Array();
		for (FMovieSceneConsoleVariableCollection& Collection : Section->ConsoleVariableCollections)
		{
			if (Collection.Interface)
			{
				Collection.Interface->GetConsoleVariablesForTrack(Collection.bOnlyIncludeChecked, CommandsAndValues);
			}
		}

		for (const TPair<FString, FString>& Pair : CommandsAndValues)
		{
			CVarStorage->CachePreAnimatedValue(UngroupedHandle, Pair.Key, FPreAnimatedCVarTraits::CachePreAnimatedValue, EPreAnimatedCaptureSourceTracking::AlwaysCache);
		}
	}

	for (const TPair<FString, FString>& Pair : Section->ConsoleVariables.ValuesByCVar)
	{
		CVarsNeedingUpdate.Add(Pair.Key);
	}
}

void UMovieSceneCVarTrackInstance::OnInputRemoved(const FMovieSceneTrackInstanceInput& InInput)
{
	using namespace UE::MovieScene;

	UMovieSceneCVarSection* Section = CastChecked<UMovieSceneCVarSection>(InInput.Section);
	
	for (const TPair<FString, FString>& Pair : Section->ConsoleVariables.ValuesByCVar)
	{
		CVarsNeedingUpdate.Add(Pair.Key);
	}
}

void UMovieSceneCVarTrackInstance::OnEndUpdateInputs()
{
	using namespace UE::MovieScene;

	const FInstanceRegistry* InstanceRegistry = GetLinker()->GetInstanceRegistry();

	struct FCVarInfo
	{
		FString Value;
		int32 HierarchicalBias;
	};
	TMap<FString, FCVarInfo> CVarInfo;

	for (const FMovieSceneTrackInstanceInput& Input : GetInputs())
	{
		const int32 HierarchicalBias = InstanceRegistry->GetInstance(Input.InstanceHandle).GetContext().GetHierarchicalBias();

		UMovieSceneCVarSection* Section = CastChecked<UMovieSceneCVarSection>(Input.Section);

		// Gather all CVars
		TArray<TPair<FString, FString>> CommandsAndValues = Section->ConsoleVariables.ValuesByCVar.Array();
		for (FMovieSceneConsoleVariableCollection& Collection : Section->ConsoleVariableCollections)
		{
			if (Collection.Interface)
			{
				Collection.Interface->GetConsoleVariablesForTrack(Collection.bOnlyIncludeChecked, CommandsAndValues);
			}
		}

		for (const TPair<FString, FString>& Pair : CommandsAndValues)
		{
			// If CVarsNeedingUpdate is empty that implies that a section was changed so we need to reapply everything
			if (CVarsNeedingUpdate.Num() == 0 || CVarsNeedingUpdate.Contains(Pair.Key))
			{
				FCVarInfo* ExistingInfo = CVarInfo.Find(Pair.Key);
				if (ExistingInfo)
				{
					// If this input has greater hbias, it should override the existing one
					if (HierarchicalBias > ExistingInfo->HierarchicalBias)
					{
						ExistingInfo->Value = Pair.Value;
					}
					// Currently we do not support blending with cvars, so this is a warning
					else if (HierarchicalBias == ExistingInfo->HierarchicalBias)
					{
						UE_LOGF(LogMovieScene, Warning, "Multiple sections animating the same CVar '%ls', this is not supported.", *Pair.Key);
					}
				}
				else
				{
					CVarInfo.Add(Pair.Key, FCVarInfo{ Pair.Value, HierarchicalBias });
				}
			}
		}
	}

	IConsoleManager& ConsoleManager = IConsoleManager::Get();
	for (const TPair<FString, FCVarInfo>& Pair : CVarInfo)
	{
		IConsoleVariable* ConsoleVariable = ConsoleManager.FindConsoleVariable(*Pair.Key);
		if (ConsoleVariable)
		{
			ConsoleVariable->SetWithCurrentPriority(*Pair.Value.Value);
		}
		else
		{
			UE_LOGF(LogMovieScene, Warning, "Attempting to animate undefined CVar '%ls' with value '%ls'.", *Pair.Key, *Pair.Value.Value);
		}
	}
}

void UMovieSceneCVarTrackInstance::OnDestroyed()
{
}

