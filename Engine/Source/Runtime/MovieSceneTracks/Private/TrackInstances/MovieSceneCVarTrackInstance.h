// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/TrackInstance/MovieSceneTrackInstance.h"
#include "UObject/ObjectMacros.h"
#include "Sections/MovieSceneCVarSection.h"
#include "MovieSceneCVarTrackInstance.generated.h"

UCLASS()
class UMovieSceneCVarTrackInstance : public UMovieSceneTrackInstance
{
	GENERATED_BODY()

private:
	virtual void OnBeginUpdateInputs() override;
	virtual void OnAnimate() override;
	virtual void OnInputAdded(const FMovieSceneTrackInstanceInput& InInput) override;
	virtual void OnInputRemoved(const FMovieSceneTrackInstanceInput& InInput) override;
	virtual void OnEndUpdateInputs() override;
	virtual void OnDestroyed() override;

	TSet<FString> CVarsNeedingUpdate;
};
