// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackInstances/MovieSceneCameraCutEditorHandler.h"
#include "Engine/EngineTypes.h"

#if WITH_EDITOR

#include "Evaluation/CameraCutPlaybackCapability.h"
#include "IMovieScenePlayer.h"
#include "LevelEditorViewport.h"
#include "MovieSceneTracksSettings.h"
#include "Systems/MovieSceneMotionVectorSimulationSystem.h"
#include "TrackInstances/MovieSceneCameraCutTrackInstance.h"
#include "TrackInstances/MovieSceneCameraCutViewportPreviewer.h"

namespace UE::MovieScene
{

bool FPreAnimatedCameraCutEditorTraits::ShouldHandleViewportCameraCuts(UWorld* ViewportWorld)
{
	const bool bSupportsSimulateWorlds = GetDefault<UMovieSceneTracksSettings>()->GetPreviewCameraCutsInSimulate();

	return ViewportWorld && 
		// We can handle editor worlds, and game worlds that don't have an active player controller/pawn,
		// such as PIE/SIE where the user has "ejected" out of the player controller, as long as the user
		// hasn't opt out of that in the settings.
		(ViewportWorld->WorldType == EWorldType::Editor || 
			ViewportWorld->WorldType == EWorldType::EditorPreview ||
			(
				bSupportsSimulateWorlds &&
				ViewportWorld->WorldType == EWorldType::PIE && 
				GEditor && GEditor->bIsSimulatingInEditor));
}

FPreAnimatedCameraCutEditorState FPreAnimatedCameraCutEditorTraits::CachePreAnimatedValue(
		FLevelEditorViewportClient* InKey)
{
	FPreAnimatedCameraCutEditorState CachedValue;
	CachedValue.ViewportLocation = InKey->GetViewLocation();
	CachedValue.ViewportRotation = InKey->GetViewRotation();
	return CachedValue;
}

void FPreAnimatedCameraCutEditorTraits::RestorePreAnimatedValue(
		FLevelEditorViewportClient* InKey, 
		const FPreAnimatedCameraCutEditorState& CachedValue, 
		const FRestoreStateParams& Params)
{
	if (!GEditor || !InKey)
	{
		return;
	}

	// Check that our pointer is still valid by searching it in active viewports,
	// in case the user removed that viewport since we cached the pre-animated state.
	if (GEditor->GetLevelViewportClients().Find(InKey) == INDEX_NONE)
	{
		return;
	}

	// Check that we have an editor viewport.
	if (!ShouldHandleViewportCameraCuts(InKey->GetWorld()))
	{
		return;
	}

	// If the viewport wasn't locked to cinematics anyway, don't mess it up.
	// However, we don't call `IsLockedToCinematics` because we want to also consider the case
	// of a camera actor that is PendingKill after having been unspawned by sequencer in the
	// current update.
	if (InKey->GetCinematicActorLock().LockedActor.IsExplicitlyNull())
	{
		return;
	}
	
	// Restore pre-animated viewport position if needed. Then disable cinematic lock either way.
	FInstanceRegistry* InstanceRegistry = Params.Linker->GetInstanceRegistry();
	const FSequenceInstance& TerminalInstance = InstanceRegistry->GetInstance(Params.TerminalInstanceHandle);
	FCameraCutPlaybackCapabilityCompatibilityWrapper Wrapper(TerminalInstance);
	if (Wrapper.ShouldRestoreEditorViewports())
	{
		InKey->SetViewLocation(CachedValue.ViewportLocation);
		InKey->SetViewRotation(CachedValue.ViewportRotation);
		InKey->ViewFOV = InKey->FOVAngle;
	}

	InKey->SetCinematicActorLock(nullptr);
	InKey->bLockedCameraView = false;
	InKey->UpdateViewForLockedActor();
	InKey->Invalidate();
}

TAutoRegisterPreAnimatedStorageID<FPreAnimatedCameraCutEditorStorage> FPreAnimatedCameraCutEditorStorage::StorageID;

void FCameraCutEditorHandler::CachePreAnimatedValue(
		UMovieSceneEntitySystemLinker* Linker,
		const FSequenceInstance& SequenceInstance)
{
	if (!GEditor)
	{
		return;
	}

	UObject* PlaybackContext = SequenceInstance.GetSharedPlaybackState()->GetPlaybackContext();
	UWorld* ContextWorld = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	// Only handle editor world/viewports.
	if (!FPreAnimatedCameraCutEditorTraits::ShouldHandleViewportCameraCuts(ContextWorld))
	{
		return;
	}
	
	TSharedPtr<FPreAnimatedCameraCutEditorStorage> PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedCameraCutEditorStorage>();

	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		// Only handle the viewports that are tied to our playback context.
		if (!LevelVC || LevelVC->GetWorld() != ContextWorld)
		{
			continue;
		}

		PreAnimatedStorage->CachePreAnimatedValue(
				LevelVC,
				[](FLevelEditorViewportClient* InKey) { return FPreAnimatedCameraCutEditorTraits::CachePreAnimatedValue(InKey); },
				EPreAnimatedCaptureSourceTracking::AlwaysCache);
	}
}

void FCameraCutEditorHandler::ForcePreAnimatedValueOperation(
		UMovieSceneEntitySystemLinker* Linker,
		const FSequenceInstance& SequenceInstance,
		EForcedCameraCutPreAnimatedStorageOperation Operation)
{
	if (!GEditor)
	{
		return;
	}

	TGuardValue<FEntityManager*> EntityManagerForDebugging(GEntityManagerForDebuggingVisualizers, &Linker->EntityManager);

	UObject* PlaybackContext = SequenceInstance.GetSharedPlaybackState()->GetPlaybackContext();
	UWorld* ContextWorld = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	// Only handle editor world/viewports.
	if (!FPreAnimatedCameraCutEditorTraits::ShouldHandleViewportCameraCuts(ContextWorld))
	{
		return;
	}

	TSharedPtr<FPreAnimatedCameraCutEditorStorage> PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedCameraCutEditorStorage>();

	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		// Only handle the viewports that are tied to our playback context.
		if (!LevelVC || LevelVC->GetWorld() != ContextWorld)
		{
			continue;
		}

		FPreAnimatedStorageIndex StorageIndex = PreAnimatedStorage->FindStorageIndex(LevelVC);
		if (StorageIndex.IsValid())
		{
			switch (Operation)
			{
				case EForcedCameraCutPreAnimatedStorageOperation::Cache:
					{
						PreAnimatedStorage->DiscardPreAnimatedStateStorage(StorageIndex, EPreAnimatedStorageRequirement::Transient);
						PreAnimatedStorage->DiscardPreAnimatedStateStorage(StorageIndex, EPreAnimatedStorageRequirement::Persistent);
						PreAnimatedStorage->CachePreAnimatedValue(
								LevelVC,
								[](FLevelEditorViewportClient* InKey) { return FPreAnimatedCameraCutEditorTraits::CachePreAnimatedValue(InKey); },
								EPreAnimatedCaptureSourceTracking::AlwaysCache);
					}
					break;
				case EForcedCameraCutPreAnimatedStorageOperation::Restore:
					{
						FRestoreStateParams Params;
						Params.Linker = Linker;
						Params.TerminalInstanceHandle = SequenceInstance.GetRootInstanceHandle();
						PreAnimatedStorage->RestorePreAnimatedStateStorage(StorageIndex, EPreAnimatedStorageRequirement::Transient, EPreAnimatedStorageRequirement::Persistent, Params);
					}
					break;
				case EForcedCameraCutPreAnimatedStorageOperation::Discard:
					{
						Linker->PreAnimatedState.DiscardStateForStorage(FPreAnimatedCameraCutEditorStorage::StorageID, StorageIndex);
					}
					break;
			}
		}
	}
}

FCameraCutEditorHandler::FCameraCutEditorHandler(
		UMovieSceneEntitySystemLinker* InLinker,
		const FSequenceInstance& InSequenceInstance,
		FCameraCutViewportPreviewer& InViewportPreviewer)
	: Linker(InLinker)
	, SequenceInstance(InSequenceInstance)
	, ViewportPreviewer(InViewportPreviewer)
{
}

void FCameraCutEditorHandler::SetCameraCut(
		UObject* CameraObject, 
		const FMovieSceneCameraCutParams& CameraCutParams)
{
	if (!GEditor)
	{
		return;
	}

	UObject* PlaybackContext = SequenceInstance.GetSharedPlaybackState()->GetPlaybackContext();
	UWorld* ContextWorld = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	// Only handle editor world/viewports.
	if (!FPreAnimatedCameraCutEditorTraits::ShouldHandleViewportCameraCuts(ContextWorld))
	{
		return;
	}

	FCameraCutPlaybackCapabilityCompatibilityWrapper Wrapper(SequenceInstance);

	// If we don't want to update camera cuts, let's remember it and release the viewports
	// in case they were still locked to cinematics.
	const bool bPreviewCameraCuts = Wrapper.ShouldUpdateCameraCut();

	// Make sure our viewport modifiers are correctly registered/unregistered.
	ViewportPreviewer.ToggleViewportPreviewModifiers(bPreviewCameraCuts);

	// Gather the viewports we want to affect.
	TArray<FLevelEditorViewportClient*> CinematicVCs;

	AActor* CameraActor = Cast<AActor>(CameraObject);
	AActor* UnlockIfCameraActor = Cast<AActor>(CameraCutParams.UnlockIfCameraObject);
	UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromRuntimeObject(CameraObject);

	for (FLevelEditorViewportClient* LevelVC : GEditor->GetLevelViewportClients())
	{
		// Only handle the viewports that are tied to our playback context.
		if (!LevelVC || LevelVC->GetWorld() != ContextWorld)
		{
			continue;
		}

		if (!bPreviewCameraCuts || !LevelVC->AllowsCinematicControl())
		{
			ReleaseCameraCutForViewport(*LevelVC);
			continue;
		}

		if (CameraActor == nullptr && 
				UnlockIfCameraActor != nullptr && 
				!LevelVC->IsLockedToActor(UnlockIfCameraActor))
		{
			continue;
		}

		SetCameraCutForViewport(*LevelVC, CameraActor, CameraComponent, CameraCutParams);
	}

	// Trigger any callback.
	FOnCameraCutUpdatedParams CameraCutUpdatedParams;
	CameraCutUpdatedParams.ViewTarget = CameraActor;
	CameraCutUpdatedParams.ViewTargetCamera = CameraComponent;
	CameraCutUpdatedParams.bIsJumpCut = true;
	Wrapper.OnCameraCutUpdated(CameraCutUpdatedParams);
}

void FCameraCutEditorHandler::SetCameraCutForViewport(
		FLevelEditorViewportClient& ViewportClient, 
		AActor* CameraActor, 
		UCameraComponent* CameraComponent,
		const FMovieSceneCameraCutParams& CameraCutParams)
{
	FVector ViewLocation = ViewportClient.GetViewLocation();
	FRotator ViewRotation = ViewportClient.GetViewRotation();
	float ViewFOV = ViewportClient.ViewFOV;
	bool bCameraHasBeenCut = CameraCutParams.bJumpCut;

	TSharedPtr<FPreAnimatedCameraCutEditorStorage> PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage(
			FPreAnimatedCameraCutEditorStorage::StorageID);

	// If that viewport wasn't locked to cinematics, see if we need to re-cache pre-animated state.
	// This is necessary if the user released control with the camera button on the camera cut track.
	// (see ReleaseCameraCutForViewport)
	//
	// Note that storage index can be invalid here if we never cached any pre-animated state, which
	// is possible if the option to restore viewports on unlock is off.
	if (!ViewportClient.IsLockedToCinematic())
	{
		FPreAnimatedStorageIndex StorageIndex = PreAnimatedStorage->FindStorageIndex(&ViewportClient);
		if (!StorageIndex.IsValid())
		{
			PreAnimatedStorage->CachePreAnimatedValue(
					&ViewportClient,
					[](FLevelEditorViewportClient* InKey) { return FPreAnimatedCameraCutEditorTraits::CachePreAnimatedValue(InKey); },
					EPreAnimatedCaptureSourceTracking::AlwaysCache);
		}
	}

	if (CameraActor)
	{
		// When possible, let's get values from the camera components instead of the actor itself.
		ViewLocation = CameraComponent ? CameraComponent->GetComponentLocation() : CameraActor->GetActorLocation();
		ViewRotation = CameraComponent ? CameraComponent->GetComponentRotation() : CameraActor->GetActorRotation();
		ViewFOV = CameraComponent ? CameraComponent->FieldOfView : ViewportClient.FOVAngle;
		bCameraHasBeenCut = bCameraHasBeenCut || !ViewportClient.IsLockedToActor(CameraActor);
	}
	else
	{
		// If CameraActor is null, we are releasing camera control. This can happen here instead of
		// inside pre-animated state if we are *blending* out of the cinematic. In this case, let's
		// restore the pre-animated viewport location ourselves and enable blending.
		FPreAnimatedStorageIndex StorageIndex = PreAnimatedStorage->FindStorageIndex(&ViewportClient);
		if (StorageIndex.IsValid())
		{
			FPreAnimatedCameraCutEditorState CachedValue = PreAnimatedStorage->GetCachedValue(StorageIndex);
			ViewLocation = CachedValue.ViewportLocation;
			ViewRotation = CachedValue.ViewportRotation;
			ViewFOV = ViewportClient.FOVAngle;
		}
	}

	// Set viewport properties.
	ViewportClient.SetViewLocation(ViewLocation);
	ViewportClient.SetViewRotation(ViewRotation);
	ViewportClient.ViewFOV = ViewFOV;

	if (bCameraHasBeenCut)
	{
		ViewportClient.SetIsCameraCut();

		if (UMovieSceneMotionVectorSimulationSystem* MotionVectorSim = Linker->FindSystem<UMovieSceneMotionVectorSimulationSystem>())
		{
			MotionVectorSim->SimulateAllTransforms();
		}
	}

	// Set the actor lock.
	ViewportClient.SetCinematicActorLock(CameraActor);
	ViewportClient.SetActorLock(nullptr);
	ViewportClient.bLockedCameraView = (CameraActor != nullptr);
	ViewportClient.RemoveCameraRoll();

	// Deal with extra camera properties.
	if (CameraComponent)
	{
		if (bCameraHasBeenCut)
		{
			// tell the camera we cut
			CameraComponent->NotifyCameraCut();
		}

		// enforce aspect ratio.
		if (CameraComponent->AspectRatio == 0)
		{
			ViewportClient.AspectRatio = 1.7f;
		}
		else
		{
			ViewportClient.AspectRatio = CameraComponent->AspectRatio;
		}

		// enforce viewport type.
		if (CameraComponent->ProjectionMode == ECameraProjectionMode::Type::Perspective)
		{
			if (ViewportClient.GetViewportType() != LVT_Perspective)
			{
				ViewportClient.SetViewportType(LVT_Perspective);
			}
		}

		// If there are selected actors, invalidate the viewports hit proxies, otherwise they won't be selectable afterwards
		if (ViewportClient.Viewport && GEditor && GEditor->GetSelectedActorCount() > 0)
		{
			ViewportClient.Viewport->InvalidateHitProxy();
		}
	}

	// Setup blending preview.
	const bool bIsBlending = (
			(CameraCutParams.bCanBlend) &&
			(CameraCutParams.BlendTime > 0.f) &&
			(CameraCutParams.PreviewBlendFactor < 1.f - SMALL_NUMBER));
	if (bIsBlending)
	{
		AActor* PreviousCameraActor = Cast<AActor>(CameraCutParams.PreviousCameraObject);
		UCameraComponent* PreviousCameraComponent = MovieSceneHelpers::CameraComponentFromRuntimeObject(CameraCutParams.PreviousCameraObject);

		FCameraCutViewportPreviewerTarget FromPreviewTarget;
		if (PreviousCameraActor)
		{
			FromPreviewTarget.CameraActor = PreviousCameraActor;
			FromPreviewTarget.CameraComponent = PreviousCameraComponent;
		}
		else
		{
			// We have no "from" camera, so let's blend from the original viewport position.
			FromPreviewTarget.PreAnimatedStorage = PreAnimatedStorage;
		}

		FCameraCutViewportPreviewerTarget ToPreviewTarget;
		if (CameraActor)
		{
			ToPreviewTarget.CameraActor = CameraActor;
			ToPreviewTarget.CameraComponent = CameraComponent;
		}
		else
		{
			// We have no "to" camera, so let's blend back to the original viewport position.
			ToPreviewTarget.PreAnimatedStorage = PreAnimatedStorage;
		}

		ViewportPreviewer.SetupBlend(FromPreviewTarget, ToPreviewTarget, CameraCutParams.PreviewBlendFactor);
	}
	else
	{
		ViewportPreviewer.TeardownBlend();
	}

	// Update ControllingActorViewInfo, so it is in sync with the updated viewport
	ViewportClient.UpdateViewForLockedActor();
	ViewportClient.Invalidate();
}

void FCameraCutEditorHandler::ReleaseCameraCutForViewport(
		FLevelEditorViewportClient& ViewportClient)
{
	// If the viewport was already released, we have nothing to do.
	if (!ViewportClient.IsLockedToCinematic())
	{
		return;
	}
	
	// Restore the viewport to its pre-animated state.
	//
	// Note that PreAnimatedStorage can be null here if we never cached any pre-animated state, which
	// is possible if the option to restore viewports on unlock is off.
	FPreAnimatedStorageIndex StorageIndex;
	TSharedPtr<FPreAnimatedCameraCutEditorStorage> PreAnimatedStorage = Linker->PreAnimatedState.FindStorage(FPreAnimatedCameraCutEditorStorage::StorageID);
	if (PreAnimatedStorage.IsValid())
	{
		StorageIndex = PreAnimatedStorage->FindStorageIndex(&ViewportClient);
	}
	if (StorageIndex.IsValid())
	{
		FPreAnimatedCameraCutEditorState CachedValue = PreAnimatedStorage->GetCachedValue(StorageIndex);
		ViewportClient.SetViewLocation(CachedValue.ViewportLocation);
		ViewportClient.SetViewRotation(CachedValue.ViewportRotation);
		ViewportClient.ViewFOV = ViewportClient.FOVAngle;
	}

	// Actually release control.
	ViewportClient.SetCinematicActorLock(nullptr);
	ViewportClient.bLockedCameraView = false;
	ViewportClient.UpdateViewForLockedActor();
	ViewportClient.Invalidate();
}

}  // namespace UE::MovieScene
 
#endif  // WITH_EDITOR

