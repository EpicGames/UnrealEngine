// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

class AActor;
class FLevelEditorViewportClient;
class UCameraComponent;
class UMovieSceneEntitySystemLinker;
struct FMovieSceneCameraCutParams;

namespace UE::MovieScene
{

class FCameraCutViewportPreviewer;
struct FCameraCutPlaybackCapability;
struct FPreAnimatedCameraCutEditorStorage;
struct FSequenceInstance;

/** Pre-animated editor viewpoint */
struct FPreAnimatedCameraCutEditorState
{
	FVector ViewportLocation;
	FRotator ViewportRotation;
};

/** Pre-animated traits for in-editor camera cuts */
struct FPreAnimatedCameraCutEditorTraits : FPreAnimatedStateTraits
{
	// Key type is the viewport to restore location/rotation/FOV on.
	using KeyType = FLevelEditorViewportClient*;
	using StorageType = FPreAnimatedCameraCutEditorState;

	static bool ShouldHandleViewportCameraCuts(UWorld* ViewportWorld);
	static StorageType CachePreAnimatedValue(KeyType InKey);

	void RestorePreAnimatedValue(KeyType InKey, const StorageType& CachedValue, const FRestoreStateParams& Params);
};

// Make it possible to declare our storage below with a forward-declared FLevelEditorViewportClient.
template<>
struct THasAddReferencedObjectForComponent<FLevelEditorViewportClient*>
{
	static constexpr bool Value = false;
};

/** Pre-animated state storage for in-editor camera cuts */
struct FPreAnimatedCameraCutEditorStorage : TPreAnimatedStateStorage<FPreAnimatedCameraCutEditorTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedCameraCutEditorStorage> StorageID;
};

/** Type of forced operation on camera cut editor pre-animated state storage */
enum class EForcedCameraCutPreAnimatedStorageOperation
{
	Cache,
	Restore,
	Discard
};

/**
 * Utility class for executing camera cuts in the editor viewports.
 */
struct FCameraCutEditorHandler
{
	FCameraCutEditorHandler(
			UMovieSceneEntitySystemLinker* InLinker,
			const FSequenceInstance& InSequenceInstance,
			FCameraCutViewportPreviewer& InViewportPreviewer);

	/** Sets the given camera cut in all editor viewport with cinematic control enabled */
	void SetCameraCut(
			UObject* CameraObject, 
			const FMovieSceneCameraCutParams& CameraCutParams);

	/** Cache any pre-animated values required for handling camera cuts in editor. */
	static void CachePreAnimatedValue(
			UMovieSceneEntitySystemLinker* Linker,
			const FSequenceInstance& SequenceInstance);
	/** Force cache/discard/restore pre-animated values */
	static void ForcePreAnimatedValueOperation(
			UMovieSceneEntitySystemLinker* Linker,
			const FSequenceInstance& SequenceInstance,
			EForcedCameraCutPreAnimatedStorageOperation Operation);

private:

	void SetCameraCutForViewport(
			FLevelEditorViewportClient& ViewportClient, 
			AActor* CameraActor, 
			UCameraComponent* CameraComponent,
			const FMovieSceneCameraCutParams& CameraCutParams);
	void ReleaseCameraCutForViewport(
			FLevelEditorViewportClient& ViewportClient);

private:
	UMovieSceneEntitySystemLinker* Linker;
	const FSequenceInstance& SequenceInstance;
	FCameraCutViewportPreviewer& ViewportPreviewer;
};

}  // namespace UE::MovieScene

#endif  // WITH_EDITOR

