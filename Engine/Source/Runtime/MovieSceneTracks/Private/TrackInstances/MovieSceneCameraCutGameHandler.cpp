// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackInstances/MovieSceneCameraCutGameHandler.h"

#include "Camera/CameraComponent.h"
#include "Camera/PlayerCameraManager.h"
#include "Engine/Engine.h"
#include "Engine/EngineTypes.h"
#include "Engine/GameInstance.h"
#include "Engine/LocalPlayer.h"
#include "Engine/World.h"
#include "Evaluation/CameraCutPlaybackCapability.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneCommonHelpers.h"
#include "Systems/MovieSceneMotionVectorSimulationSystem.h"
#include "TrackInstances/MovieSceneCameraCutTrackInstance.h"

#if WITH_EDITOR
#include "Editor.h"
#endif

namespace UE::MovieScene
{
static bool bCameraCutSetViewTargetThroughPlayerController = true;
FAutoConsoleVariableRef CVarCameraCutSetViewTargetThroughPlayerController(
	TEXT("Sequencer.CameraCutSetViewTargetThroughPlayerController"),
	bCameraCutSetViewTargetThroughPlayerController,
	TEXT("When true (default), camera cut tracks will call SetViewTarget on the affected player controller instead of that controller's camera manager.\n")
);

APlayerController* GetPlaybackController(const UObject* PlaybackContext)
{
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
	if (World == nullptr || World->GetGameInstance() == nullptr)
	{
		return nullptr;
	}
	
	if (const AActor* PlaybackContextActor = Cast<AActor>(PlaybackContext))
	{
		if (APlayerController* OwnerController = Cast<APlayerController>(PlaybackContextActor->GetOwner()))
		{
			return OwnerController;
		}
	}

	return World->GetGameInstance()->GetFirstLocalPlayerController();
}

bool FPreAnimatedCameraCutTraits::ShouldHandleWorldCameraCuts(UWorld* World)
{
	return World &&
		// We can handle any ongoing game worlds. We just don't handle worlds where there is
		// no active player controller/pawn, such as PIE/SIE where the user has "ejected" out
		// of the player controller.
		World->GetGameInstance() != nullptr &&
		World->WorldType != EWorldType::Editor &&
		World->WorldType != EWorldType::EditorPreview
#if WITH_EDITOR
		&&
		(!GEditor || !GEditor->bIsSimulatingInEditor)
#endif
		;
}

FPreAnimatedCameraCutState FPreAnimatedCameraCutTraits::CachePreAnimatedValue(
		UObject* PlaybackContext,
		uint8 InKey)
{
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;
	if (ShouldHandleWorldCameraCuts(World))
	{
		CA_SUPPRESS(6011);
		const APlayerController* PC = GetPlaybackController(PlaybackContext);

		// Save previous view target.
		APlayerCameraManager* CameraManager = (PC != nullptr) ? PC->PlayerCameraManager.Get() : nullptr;
		AActor* ViewTarget = CameraManager ? CameraManager->GetViewTarget() : nullptr;

		// Save previous aspect ratio axis constraint.
		ULocalPlayer* LocalPlayer = (PC != nullptr) ? PC->GetLocalPlayer() : nullptr;
		TOptional<EAspectRatioAxisConstraint> AspectRatioAxisConstraint;
		if (LocalPlayer)
		{
			AspectRatioAxisConstraint = LocalPlayer->AspectRatioAxisConstraint;
		}

		return StorageType{ World, LocalPlayer, ViewTarget, AspectRatioAxisConstraint };
	}
	return StorageType();
}

void FPreAnimatedCameraCutTraits::RestorePreAnimatedValue(
		uint8 InKey, 
		const FPreAnimatedCameraCutState& CachedValue, 
		const FRestoreStateParams& Params)
{
	UWorld* World = Cast<UWorld>(CachedValue.LastWorld.ResolveObjectPtr());
	if (!ShouldHandleWorldCameraCuts(World))
	{
		return;
	}


	APlayerController* PC = nullptr;
	if (ULocalPlayer* PreviousLocalPlayer = Cast<ULocalPlayer>(CachedValue.LastLocalPlayer.ResolveObjectPtr()))
	{
		if (APlayerController* OwnerController = Cast<APlayerController>(PreviousLocalPlayer->GetPlayerController(World)))
		{
			PC = OwnerController;
		}
	}

	// Restore previous view target.
	// If the previous view target is not valid anymore, we still set it on the camera manger. This will by
	// default fall back to using the player controller as the view target.
	if (PC)
	{
		AActor* PreviousViewTarget = Cast<AActor>(CachedValue.LastViewTarget.ResolveObjectPtr());

		if (bCameraCutSetViewTargetThroughPlayerController)
		{
			PC->SetViewTarget(PreviousViewTarget);
		}

		if (APlayerCameraManager* CameraManager = PC->PlayerCameraManager.Get())
		{
			if (!bCameraCutSetViewTargetThroughPlayerController)
			{
				CameraManager->SetViewTarget(PreviousViewTarget);
			}

			// TODO james.fleming ideally we would cache this before, just in case it had been set true (which is not usual, but could be possible) 
			CameraManager->bClientSimulatingViewTarget = false;
		}
	}

	// Restore previous aspect ratio axis constraint. Use the cached local player if there's no local player
	// to be found, which can happen if pre-animated state is restored during level loads and such.
	ULocalPlayer* LocalPlayer = (PC != nullptr) ? 
		PC->GetLocalPlayer() : 
		Cast<ULocalPlayer>(CachedValue.LastLocalPlayer.ResolveObjectPtr());
	if (LocalPlayer && CachedValue.LastAspectRatioAxisConstraint.IsSet())
	{
		LocalPlayer->AspectRatioAxisConstraint = CachedValue.LastAspectRatioAxisConstraint.GetValue();
	}
}

TAutoRegisterPreAnimatedStorageID<FPreAnimatedCameraCutStorage> FPreAnimatedCameraCutStorage::StorageID;

/** Utility function for converting sequence blend curves to camera manager blend types */
static TTuple<EViewTargetBlendFunction, float> BuiltInEasingTypeToBlendFunction(EMovieSceneBuiltInEasing EasingType)
{
	using Return = TTuple<EViewTargetBlendFunction, float>;
	switch (EasingType)
	{
		case EMovieSceneBuiltInEasing::Linear:
			return Return(EViewTargetBlendFunction::VTBlend_Linear, 1.f);

		case EMovieSceneBuiltInEasing::QuadIn:
			return Return(EViewTargetBlendFunction::VTBlend_EaseIn, 2);
		case EMovieSceneBuiltInEasing::QuadOut:
			return Return(EViewTargetBlendFunction::VTBlend_EaseOut, 2);
		case EMovieSceneBuiltInEasing::QuadInOut:
			return Return(EViewTargetBlendFunction::VTBlend_EaseInOut, 2);

		case EMovieSceneBuiltInEasing::Cubic:
		case EMovieSceneBuiltInEasing::HermiteCubicInOut:
			return Return(EViewTargetBlendFunction::VTBlend_Cubic, 3);

		case EMovieSceneBuiltInEasing::CubicIn:
			return Return(EViewTargetBlendFunction::VTBlend_EaseIn, 3);
		case EMovieSceneBuiltInEasing::CubicOut:
			return Return(EViewTargetBlendFunction::VTBlend_EaseOut, 3);
		case EMovieSceneBuiltInEasing::CubicInOut:
			return Return(EViewTargetBlendFunction::VTBlend_EaseInOut, 3);

		case EMovieSceneBuiltInEasing::QuartIn:
			return Return(EViewTargetBlendFunction::VTBlend_EaseIn, 4);
		case EMovieSceneBuiltInEasing::QuartOut:
			return Return(EViewTargetBlendFunction::VTBlend_EaseOut, 4);
		case EMovieSceneBuiltInEasing::QuartInOut:
			return Return(EViewTargetBlendFunction::VTBlend_EaseInOut, 4);

		case EMovieSceneBuiltInEasing::QuintIn:
			return Return(EViewTargetBlendFunction::VTBlend_EaseIn, 5);
		case EMovieSceneBuiltInEasing::QuintOut:
			return Return(EViewTargetBlendFunction::VTBlend_EaseOut, 5);
		case EMovieSceneBuiltInEasing::QuintInOut:
			return Return(EViewTargetBlendFunction::VTBlend_EaseInOut, 5);

		// UNSUPPORTED
		case EMovieSceneBuiltInEasing::SinIn:
		case EMovieSceneBuiltInEasing::SinOut:
		case EMovieSceneBuiltInEasing::SinInOut:
		case EMovieSceneBuiltInEasing::CircIn:
		case EMovieSceneBuiltInEasing::CircOut:
		case EMovieSceneBuiltInEasing::CircInOut:
		case EMovieSceneBuiltInEasing::ExpoIn:
		case EMovieSceneBuiltInEasing::ExpoOut:
		case EMovieSceneBuiltInEasing::ExpoInOut:
		case EMovieSceneBuiltInEasing::Custom:
			break;
	}
	return Return(EViewTargetBlendFunction::VTBlend_Linear, 1.f);
}

void FCameraCutGameHandler::ForcePreAnimatedValueRestore(
			UMovieSceneEntitySystemLinker* Linker,
			const FSequenceInstance& SequenceInstance)
{
	TSharedPtr<FPreAnimatedCameraCutStorage> PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedCameraCutStorage>();

	FRestoreStateParams Params;
	Params.Linker = Linker;
	Params.TerminalInstanceHandle = SequenceInstance.GetRootInstanceHandle();

	FPreAnimatedStorageIndex StorageIndex = PreAnimatedStorage->FindStorageIndex(0);
	if (StorageIndex.IsValid())
	{		
		PreAnimatedStorage->RestorePreAnimatedStateStorage(
				(uint8)0,  // See comment below
				EPreAnimatedStorageRequirement::Transient, 
				EPreAnimatedStorageRequirement::Persistent,
				Params);
	}
}

void FCameraCutGameHandler::CachePreAnimatedValue(
		UMovieSceneEntitySystemLinker* Linker,
		const FSequenceInstance& SequenceInstance)
{
	TSharedPtr<FPreAnimatedCameraCutStorage> PreAnimatedStorage = Linker->PreAnimatedState.GetOrCreateStorage<FPreAnimatedCameraCutStorage>();

	UObject* PlaybackContext = SequenceInstance.GetSharedPlaybackState()->GetPlaybackContext();
	PreAnimatedStorage->CachePreAnimatedValue(
			(uint8)0,  // Later this can be an index for split-screen player
			[PlaybackContext](uint8 InKey) { return FPreAnimatedCameraCutTraits::CachePreAnimatedValue(PlaybackContext, InKey); },
			EPreAnimatedCaptureSourceTracking::AlwaysCache);
}

FCameraCutGameHandler::FCameraCutGameHandler(
		UMovieSceneEntitySystemLinker* InLinker,
		const FSequenceInstance& InSequenceInstance)
	: Linker(InLinker)
	, SequenceInstance(InSequenceInstance)
{
}

void FCameraCutGameHandler::SetCameraCut(
		UObject* CameraObject,
		const FMovieSceneCameraCutParams& CameraCutParams)
{
	FCameraCutPlaybackCapabilityCompatibilityWrapper Wrapper(SequenceInstance);

	// If we don't want to update camera cuts, let's bail out.
	if (!Wrapper.ShouldUpdateCameraCut())
	{
		return;
	}

	UObject* PlaybackContext = SequenceInstance.GetSharedPlaybackState()->GetPlaybackContext();
	UWorld* World = PlaybackContext ? PlaybackContext->GetWorld() : nullptr;

	// Also bail out if we don't have a world running any sort of game.
	if (!FPreAnimatedCameraCutTraits::ShouldHandleWorldCameraCuts(World))
	{
		return;
	}

	CA_SUPPRESS(6011);
	APlayerController* PC = GetPlaybackController(PlaybackContext);
	APlayerCameraManager* CameraManager = (PC != nullptr) ? PC->PlayerCameraManager.Get() : nullptr;

	// If the player controller is missing, there is no camera manager for us to manage the view target
	// so, again, we bail out.
	if (CameraManager == nullptr)
	{
		return;
	}

	// Let's get the current view target directly from the player camera manager first. This is because
	// we don't want to go through GetViewTarget, which checks if the current view target is valid, and
	// re-assigns it to the player controller if it's not. We don't want this to happen, especially since
	// a spawnable camera might have just been unspawned, causing SetCameraCut to be called, and we
	// need to handle this properly.
	AActor* ViewTarget = CameraManager->PendingViewTarget.Target;
	if (!ViewTarget)
	{
		ViewTarget = CameraManager->ViewTarget.Target;
	}

	// If UnlockIfCameraActor is valid, release lock only if currently locked to the specified object.
	AActor* UnlockIfCameraActor = Cast<AActor>(CameraCutParams.UnlockIfCameraObject);
	if (CameraObject == nullptr && UnlockIfCameraActor != nullptr && UnlockIfCameraActor != ViewTarget)
	{
		return;
	}

	// See if we need to override the aspect ratio axis constraint.
	TOptional<EAspectRatioAxisConstraint> OverrideAspectRatioAxisConstraint;
	if (Wrapper.CameraCutCapability)
	{
		OverrideAspectRatioAxisConstraint = Wrapper.CameraCutCapability->GetAspectRatioAxisConstraintOverride();
	}

	// CameraObject is null if we need to release control, which can happen here (instead of via pre-animated 
	// state restore) if we are *blending* back to gameplay, and not cutting back to it at the end of a camera 
	// cut section. Let's get the pre-animated value and blend back towards it.
	if (CameraObject == nullptr)
	{
		TSharedPtr<FPreAnimatedCameraCutStorage> PreAnimatedStorage = Linker->PreAnimatedState.FindStorage(FPreAnimatedCameraCutStorage::StorageID);
		FPreAnimatedStorageIndex StorageIndex = PreAnimatedStorage ? PreAnimatedStorage->FindStorageIndex(0) : FPreAnimatedStorageIndex();
		if (ensureMsgf(StorageIndex.IsValid(), TEXT("Blending camera back to gameplay but can't find pre-animated camera info!")))
		{
			FPreAnimatedCameraCutState CachedValue = PreAnimatedStorage->GetCachedValue(StorageIndex);
			CameraObject = CachedValue.LastViewTarget.ResolveObjectPtr();
			OverrideAspectRatioAxisConstraint = CachedValue.LastAspectRatioAxisConstraint;
		}
	}

	// If we find a camera component inside the provided object, let's make sure we are going to set
	// its owner as the next view target, and not some component (including the camera component itself).
	AActor* CameraActor = Cast<AActor>(CameraObject);
	UCameraComponent* CameraComponent = MovieSceneHelpers::CameraComponentFromRuntimeObject(CameraObject);
	if (CameraComponent && CameraComponent->GetOwner() != CameraObject)
	{
		CameraObject = CameraComponent->GetOwner();
	}

	// If the view target isn't really changing, we don't have much to do.
	if (CameraObject == ViewTarget)
	{
		if (CameraCutParams.bJumpCut)
		{
			if (CameraManager)
			{
				CameraManager->SetGameCameraCutThisFrame();
			}
		
			if (CameraComponent)
			{
				CameraComponent->NotifyCameraCut();
			}
		
			if (UMovieSceneMotionVectorSimulationSystem* MotionVectorSim = Linker->FindSystem<UMovieSceneMotionVectorSimulationSystem>())
			{
				MotionVectorSim->SimulateAllTransforms();
			}
		}

		return;
	}

	// Time to set the camera cut! How we do it depends on whether we need to do some blending, or a straight cut.
	bool bDoSetViewTarget = true;
	FViewTargetTransitionParams TransitionParams;
	const bool bDoBlend = (
			CameraCutParams.BlendType.IsSet() &&
			CameraCutParams.BlendTime > 0.f &&
			CameraCutParams.PreviewBlendFactor < 1.f   // If the blend factor is already 100%, we should cut to that camera
													   // instead of blending to it. This can happen when cutting back to
													   // the middle of a camera cut section. Even if that section has an
													   // ease-in, we should simply cut to it.
			);  
	if (bDoBlend)
	{
		// The playrate of sequences is defined as delta time * rate, so we need to match approach.
		const float PlayRateFactor = Wrapper.CameraCutCapability ? Wrapper.CameraCutCapability->GetCameraBlendPlayRate() : 1.f;
		const float BlendTime = CameraCutParams.BlendTime * (1 / FMath::Max(UE_SMALL_NUMBER, abs(PlayRateFactor)));

		UE_LOGF(LogMovieScene, Verbose, "Blending into new camera cut: '%ls' -> '%ls' (blend time: %f)",
			(ViewTarget ? *ViewTarget->GetName() : TEXT("None")),
			(CameraActor ? *CameraActor->GetName() : TEXT("None")),
			BlendTime);

		// Convert known easing functions to their corresponding view target blend parameters.
		TTuple<EViewTargetBlendFunction, float> BlendFunctionAndExp = BuiltInEasingTypeToBlendFunction(CameraCutParams.BlendType.GetValue());
		
		// The playrate of sequences is defined as delta time * rate, so we need to match approach.
		TransitionParams.BlendTime = BlendTime;
		TransitionParams.bLockOutgoing = CameraCutParams.bLockPreviousCamera;
		TransitionParams.BlendFunction = BlendFunctionAndExp.Get<0>();
		TransitionParams.BlendExp = BlendFunctionAndExp.Get<1>();

		// Calling SetViewTarget on a camera that we are currently transitioning to will 
		// result in that transition being aborted, and the view target being set immediately.
		// We want to avoid that, so let's leave the transition running if it's the case.
		const AActor* PendingViewTarget = CameraManager->PendingViewTarget.Target;
		if (CameraActor != nullptr && PendingViewTarget == CameraActor)
		{
			UE_LOGF(LogMovieScene, Verbose, "Camera transition aborted, we are already blending towards the intended camera");
			bDoSetViewTarget = false;
		}
	}
	else
	{
		UE_LOGF(LogMovieScene, Verbose, "Starting new camera cut: '%ls'",
			(CameraActor ? *CameraActor->GetName() : TEXT("None")));
	}
	if (bDoSetViewTarget)
	{
		if (bCameraCutSetViewTargetThroughPlayerController)
		{
			if (ensureMsgf(PC, TEXT("Can't set view target when there is no player controller!")))
			{
				PC->SetViewTarget(CameraActor, TransitionParams);
			}
		}
		else
		{
			if (ensureMsgf(CameraManager, TEXT("Can't set view target when there is no camera manager!")))
			{
				CameraManager->SetViewTarget(CameraActor, TransitionParams);
			}
		}
	}

	// Override the aspect ratio constraint if this sequence requires it.
	ULocalPlayer* LocalPlayer = (PC != nullptr) ? PC->GetLocalPlayer() : nullptr;
	if (LocalPlayer != nullptr && OverrideAspectRatioAxisConstraint.IsSet())
	{
		LocalPlayer->AspectRatioAxisConstraint = OverrideAspectRatioAxisConstraint.GetValue();
	}

	// We want to notify of cuts on hard cuts and time jumps, but not on blend cuts.
	const bool bIsStraightCut = !CameraCutParams.BlendType.IsSet() || CameraCutParams.bJumpCut;

	if (CameraComponent && bIsStraightCut)
	{
		CameraComponent->NotifyCameraCut();
	}

	if (CameraManager)
	{
		CameraManager->bClientSimulatingViewTarget = (CameraActor != nullptr);

		if (bIsStraightCut)
		{
			CameraManager->SetGameCameraCutThisFrame();
		}
	}

	if (bIsStraightCut)
	{
		if (UMovieSceneMotionVectorSimulationSystem* MotionVectorSim = Linker->FindSystem<UMovieSceneMotionVectorSimulationSystem>())
		{
			MotionVectorSim->SimulateAllTransforms();
		}

		FOnCameraCutUpdatedParams CameraCutUpdatedParams;
		CameraCutUpdatedParams.ViewTarget = CameraActor;
		CameraCutUpdatedParams.ViewTargetCamera = CameraComponent;
		CameraCutUpdatedParams.bIsJumpCut = true;
		Wrapper.OnCameraCutUpdated(CameraCutUpdatedParams);
	}
}

}  // namespace UE::MovieScene
