// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.inl"

class UMovieSceneEntitySystemLinker;
struct FMovieSceneCameraCutParams;

namespace UE::MovieScene
{

struct FCameraCutPlaybackCapability;
struct FPreAnimatedCameraCutStorage;
struct FSequenceInstance;

/** Pre-animated view target info */
struct FPreAnimatedCameraCutState
{
	FObjectKey LastWorld;
	FObjectKey LastLocalPlayer;
	FObjectKey LastViewTarget;
	TOptional<EAspectRatioAxisConstraint> LastAspectRatioAxisConstraint;
};

/** Pre-animated traits for in-game camera cuts */
struct FPreAnimatedCameraCutTraits : FPreAnimatedStateTraits
{
	// Key type is an integer, to be maybe later used as the splitscreen index.
	using KeyType = uint8;
	using StorageType = FPreAnimatedCameraCutState;

	static bool ShouldHandleWorldCameraCuts(UWorld* World);
	static StorageType CachePreAnimatedValue(UObject* PlaybackContext, uint8 InKey);

	void RestorePreAnimatedValue(uint8 InKey, const StorageType& CachedValue, const FRestoreStateParams& Params);
};

/** Pre-aniamted state storage for in-game camera cuts */
struct FPreAnimatedCameraCutStorage : TPreAnimatedStateStorage<FPreAnimatedCameraCutTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedCameraCutStorage> StorageID;
};

/**
 * Utility class for executing camera cuts in game worlds (including PIE).
 */
struct FCameraCutGameHandler
{
	FCameraCutGameHandler(
			UMovieSceneEntitySystemLinker* InLinker,
			const FSequenceInstance& InSequenceInstance);

	/** Sets the given camera cut in all qualifying game worlds. */
	void SetCameraCut(
			UObject* CameraObject, 
			const FMovieSceneCameraCutParams& CameraCutParams);

	/** Cache any pre-animated values required for handling camera cuts in game. */
	static void CachePreAnimatedValue(
			UMovieSceneEntitySystemLinker* Linker,
			const FSequenceInstance& SequenceInstance);
	static void ForcePreAnimatedValueRestore(
			UMovieSceneEntitySystemLinker* Linker,
			const FSequenceInstance& SequenceInstance);

private:
	UMovieSceneEntitySystemLinker* Linker;
	const FSequenceInstance& SequenceInstance;
};

	APlayerController* GetPlaybackController(const UObject* PlaybackContext);
}  // namespace UE::MovieScene
