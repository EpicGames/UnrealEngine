// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackInstances/MovieSceneCameraCutTrackInstance.h"

#include "Camera/CameraComponent.h"
#include "ContentStreaming.h"
#include "Engine/Engine.h"
#include "Engine/World.h"
#include "EngineGlobals.h"
#include "IMovieScenePlaybackClient.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneInstanceRegistry.h"
#include "EntitySystem/MovieSceneSharedPlaybackState.h"
#include "EntitySystem/TrackInstance/MovieSceneTrackInstanceSystem.h"
#include "Evaluation/CameraCutPlaybackCapability.h"
#include "Evaluation/MovieSceneEvaluation.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedCaptureSource.h"
#include "GameFramework/Actor.h"
#include "GameFramework/PlayerController.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "IMovieScenePlayer.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTimeHelpers.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "TrackInstances/MovieSceneCameraCutEditorHandler.h"
#include "TrackInstances/MovieSceneCameraCutGameHandler.h"
#include "TrackInstances/MovieSceneCameraCutViewportPreviewer.h"
#include "Tracks/IMovieSceneTransformOrigin.h"
#include "Tracks/MovieSceneCameraCutTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraCutTrackInstance)

DECLARE_CYCLE_STAT(TEXT("Camera Cut Track Token Execute"), MovieSceneEval_CameraCutTrack_TokenExecute, STATGROUP_MovieSceneEval);

namespace UE
{
namespace MovieScene
{

FCameraCutPlaybackCapabilityCompatibilityWrapper::FCameraCutPlaybackCapabilityCompatibilityWrapper(const FSequenceInstance& SequenceInstance)
{
	TSharedRef<FSharedPlaybackState> SharedPlaybackState = SequenceInstance.GetSharedPlaybackState();
	CameraCutCapability = SharedPlaybackState->FindCapability<FCameraCutPlaybackCapability>();
	Player = FPlayerIndexPlaybackCapability::GetPlayer(SharedPlaybackState);
}

bool FCameraCutPlaybackCapabilityCompatibilityWrapper::ShouldUpdateCameraCut()
{
	if (CameraCutCapability)
	{
		return CameraCutCapability->ShouldUpdateCameraCut();
	}
	else if (Player)
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		return Player->CanUpdateCameraCut();
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
	return true;
}

#if WITH_EDITOR
bool FCameraCutPlaybackCapabilityCompatibilityWrapper::ShouldRestoreEditorViewports()
{
	if (CameraCutCapability)
	{
		return CameraCutCapability->ShouldRestoreEditorViewports();
	}
	return true;
}
#endif

void FCameraCutPlaybackCapabilityCompatibilityWrapper::OnCameraCutUpdated(const FOnCameraCutUpdatedParams& Params)
{
	if (CameraCutCapability)
	{
		CameraCutCapability->OnCameraCutUpdated(Params);
	}
	else if (Player)
	{
		// Not quite correct?
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		Player->UpdateCameraCut(Params.ViewTarget, nullptr, Params.bIsJumpCut);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
	}
}

/** Information about a camera cut's easing (in or out) */
struct FBlendedCameraCutEasingInfo
{
	float RootBlendTime = -1.f;
	TOptional<EMovieSceneBuiltInEasing> BlendType;

	FBlendedCameraCutEasingInfo() {}
	FBlendedCameraCutEasingInfo(float InRootBlendTime, const TScriptInterface<IMovieSceneEasingFunction>& EasingFunction)
	{
		RootBlendTime = InRootBlendTime;

		// If it's a built-in easing function, get the curve type. We'll try to convert it to what the
		// player controller knows later, in the movie scene player.
		const UObject* EaseInScript = EasingFunction.GetObject();
		if (const UMovieSceneBuiltInEasingFunction* BuiltInEaseIn = Cast<UMovieSceneBuiltInEasingFunction>(EaseInScript))
		{
			BlendType = BuiltInEaseIn->Type;
		}
	}
};

/** Camera cut info struct. */
struct FBlendedCameraCut
{
	FMovieSceneTrackInstanceInput Input;

	FMovieSceneObjectBindingID CameraBindingID;
	FMovieSceneSequenceID OperandSequenceID;

	FFrameNumber LocalStartTime;
	FFrameNumber LocalEaseInEndTime;
	FFrameTime LocalContextTime;
	FFrameNumber LocalEaseOutStartTime;
	FFrameNumber LocalEndTime;
	EPlayDirection LocalDirection = EPlayDirection::Forwards;

	FBlendedCameraCutEasingInfo EaseIn;
	FBlendedCameraCutEasingInfo EaseOut;
	bool bLockPreviousCamera = false;

	FMovieSceneObjectBindingID PreviousCameraBindingID;
	FMovieSceneSequenceID PreviousOperandSequenceID;

	float PreviewBlendFactor = -1.f;
	bool bCanBlend = false;
	bool bSavePreAnimatedValue = false;

	FBlendedCameraCut()
	{}
	FBlendedCameraCut(const FMovieSceneTrackInstanceInput& InInput, FMovieSceneObjectBindingID InCameraBindingID, FMovieSceneSequenceID InOperandSequenceID) 
		: Input(InInput)
		, CameraBindingID(InCameraBindingID)
		, OperandSequenceID(InOperandSequenceID)
	{}
};

/** Pre-roll camera cut info struct. */
struct FPreRollCameraCut
{
	FInstanceHandle InstanceHandle;
	FMovieSceneObjectBindingID CameraBindingID;
	FTransform CutTransform;
	bool bHasCutTransform;
};

/** Utility class for executing camera cuts */
struct FCameraCutAnimator
{
	using FCameraCutCache = UMovieSceneCameraCutTrackInstance::FCameraCutCache;

public:

	static UObject* FindBoundObject(FMovieSceneObjectBindingID BindingID, FMovieSceneSequenceIDRef SequenceID, TSharedRef<const FSharedPlaybackState> SharedPlaybackState)
	{
		TArrayView<TWeakObjectPtr<>> Objects = BindingID.ResolveBoundObjects(SequenceID, SharedPlaybackState);
		if (Objects.Num() > 0)
		{
			return Objects[0].Get();
		}
		return nullptr;
	}

	static bool MatchesCameraCutCache(UObject* CameraActor, const FBlendedCameraCut& Params, const FCameraCutCache& CameraCutCache)
	{
		return CameraActor == CameraCutCache.LastLockedCamera.Get() &&
			Params.Input.IsSameInput(CameraCutCache.LastInput);
	}

	static void UpdateCameraCutCache(UObject* CameraActor, const FBlendedCameraCut& Params, FCameraCutCache& OutCameraCutCache)
	{
		OutCameraCutCache.LastLockedCamera = CameraActor;
		OutCameraCutCache.LastInput = Params.Input;
	}
	
public:

	FCameraCutCache& CameraCutCache;
#if WITH_EDITOR
	FCameraCutViewportPreviewer* ViewportPreviewer = nullptr;
#endif

	FCameraCutAnimator(FCameraCutCache& InCameraCutCache)
		: CameraCutCache(InCameraCutCache)
	{
	}

#if WITH_EDITOR
	void SetViewportPreviewer(FCameraCutViewportPreviewer* InViewportPreviewer)
	{
		ViewportPreviewer = InViewportPreviewer;
	}
#endif

public:

	void AnimatePreRoll(const FPreRollCameraCut& Params, const FSequenceInstance& SequenceInstance)
	{
		if (Params.bHasCutTransform)
		{
			FVector Location = Params.CutTransform.GetLocation();
			
			TSharedRef<FSharedPlaybackState> SharedPlaybackState = SequenceInstance.GetSharedPlaybackState(); 

			const IMovieScenePlaybackClient*  Client       = SharedPlaybackState->FindCapability<IMovieScenePlaybackClient>();
			const UObject*                    InstanceData = Client ? Client->GetInstanceData() : nullptr;
			const IMovieSceneTransformOrigin* RawInterface = Cast<const IMovieSceneTransformOrigin>(InstanceData);
			
			const bool bHasInterface = RawInterface || (InstanceData && InstanceData->GetClass()->ImplementsInterface(UMovieSceneTransformOrigin::StaticClass()));
			if (bHasInterface)
			{
				// Retrieve the current origin
				FTransform TransformOrigin = RawInterface ? RawInterface->GetTransformOrigin() : IMovieSceneTransformOrigin::Execute_BP_GetTransformOrigin(InstanceData);
				Location = TransformOrigin.TransformPosition(Location);
			}
				
			IStreamingManager::Get().AddViewLocation(Location);
		}
		else
		{
			UObject* CameraObject = FindBoundObject(Params.CameraBindingID, SequenceInstance.GetSequenceID(), SequenceInstance.GetSharedPlaybackState());

			if (AActor* Actor = Cast<AActor>(CameraObject))
			{
				FVector Location = Actor->GetActorLocation();
				IStreamingManager::Get().AddViewLocation(Location);
			}
		}
	}

	bool AnimateBlendedCameraCut(
			const FBlendedCameraCut& Params, 
			UMovieSceneEntitySystemLinker* Linker,
			const FSequenceInstance& SequenceInstance)
	{
		const FMovieSceneContext& Context = SequenceInstance.GetContext();

		UObject* CameraActor = FindBoundObject(Params.CameraBindingID, Params.OperandSequenceID, SequenceInstance.GetSharedPlaybackState());
		if (Params.CameraBindingID.IsValid() && CameraActor == nullptr)
		{
			// We have an unresolved or incorrect binding.
			return false;
		}

		// Save pre-animated state only now, because we don't want to start tracking this camera cut
		// unless we know it actually resolves to a camera actor (see above) and will actually do something.
		if (Params.bSavePreAnimatedValue)
		{
			FScopedPreAnimatedCaptureSource CaptureSource(Linker, Params.Input);
			if (Linker->PreAnimatedState.IsCapturingGlobalState() || CaptureSource.WantsRestoreState())
			{
				FCameraCutGameHandler::CachePreAnimatedValue(Linker, SequenceInstance);
#if WITH_EDITOR
				FCameraCutEditorHandler::CachePreAnimatedValue(Linker, SequenceInstance);
#endif
			}
		}

		// We do a straight cut if we have jumped in time and we either (1) always cut on loops or (2) it's not a loop.
		UMovieSceneSequence* RootSequence = SequenceInstance.GetSharedPlaybackState()->GetRootSequence();
		const bool bDoLoopCuts = RootSequence ? EnumHasAnyFlags(RootSequence->GetFlags(), EMovieSceneSequenceFlags::LoopCuts) : false;
		const bool bIsJumpCut = Context.HasJumped() && (bDoLoopCuts || !Context.HasLooped());

		FMovieSceneCameraCutParams CameraCutParams;
		CameraCutParams.bJumpCut = bIsJumpCut;
		CameraCutParams.BlendTime = Params.EaseIn.RootBlendTime;
		CameraCutParams.BlendType = Params.EaseIn.BlendType;
		CameraCutParams.bLockPreviousCamera = Params.bLockPreviousCamera;

		UObject* PreviousCameraActor = FindBoundObject(Params.PreviousCameraBindingID, Params.PreviousOperandSequenceID, SequenceInstance.GetSharedPlaybackState());
		CameraCutParams.PreviousCameraObject = PreviousCameraActor;
		CameraCutParams.PreviewBlendFactor = Params.PreviewBlendFactor;
		CameraCutParams.bCanBlend = Params.bCanBlend;

		const bool bMatchesCache = MatchesCameraCutCache(CameraActor, Params, CameraCutCache);
		if (!bMatchesCache)
		{
			CameraCutParams.UnlockIfCameraObject = CameraCutCache.LastLockedCamera.Get();
			SetCameraCut(CameraActor, CameraCutParams, Linker, SequenceInstance);
			UpdateCameraCutCache(CameraActor, Params, CameraCutCache);
			return true;
		}
		else if (CameraActor || CameraCutParams.BlendTime > 0.f)
		{
			SetCameraCut(CameraActor, CameraCutParams, Linker, SequenceInstance);
			return true;
		}

		return false;
	}

	void SetCameraCut(
			UObject* CameraObject, 
			const FMovieSceneCameraCutParams& CameraCutParams, 
			UMovieSceneEntitySystemLinker* Linker,
			const FSequenceInstance& SequenceInstance)
	{
		FCameraCutGameHandler GameHandler(Linker, SequenceInstance);
		GameHandler.SetCameraCut(CameraObject, CameraCutParams);

#if WITH_EDITOR
		FCameraCutEditorHandler EditorHandler(Linker, SequenceInstance, *ViewportPreviewer);
		EditorHandler.SetCameraCut(CameraObject, CameraCutParams);
#endif
	}
};

}  // namespace MovieScene
}  // namespace UE

#if WITH_EDITOR

void UMovieSceneCameraCutTrackInstance::ToggleCameraCutLock(UMovieSceneEntitySystemLinker* Linker, bool bEnableCameraCuts, bool bRestoreViewports)
{
	using namespace UE::MovieScene;

	auto ForceEditorPreAnimatedStorageOperation = [](UMovieSceneCameraCutTrackInstance* This, EForcedCameraCutPreAnimatedStorageOperation Operation)
	{
		using namespace UE::MovieScene;

		UMovieSceneEntitySystemLinker* Linker = This->GetLinker();
		const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
		for (const FCameraCutInputInfo& InputInfo : This->SortedInputInfos)
		{
			FScopedPreAnimatedCaptureSource CaptureSource(Linker, InputInfo.Input);
			const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InputInfo.Input.InstanceHandle);
			FCameraCutEditorHandler::ForcePreAnimatedValueOperation(Linker, SequenceInstance, Operation);
		}
	};
	auto ForceGamePreAnimatedStorageRestore = [](UMovieSceneCameraCutTrackInstance* This)
	{
		using namespace UE::MovieScene;

		UMovieSceneEntitySystemLinker* Linker = This->GetLinker();
		const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();
		for (const FCameraCutInputInfo& InputInfo : This->SortedInputInfos)
		{
			FScopedPreAnimatedCaptureSource CaptureSource(Linker, InputInfo.Input);
			const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(InputInfo.Input.InstanceHandle);
			FCameraCutGameHandler::ForcePreAnimatedValueRestore(Linker, SequenceInstance);
		}
	};

	// Find the camera cut track instance and forcibly manage its pre-animated state for the editor 
	// viewports depending on what we want to do with them.
	UMovieSceneTrackInstanceInstantiator* TrackInstanceSystem = Linker->FindSystem<UMovieSceneTrackInstanceInstantiator>();
	if (TrackInstanceSystem)
	{
		UMovieSceneCameraCutTrackInstance* CameraCutTrackInstance = nullptr;
		for (auto It = TrackInstanceSystem->GetTrackInstances().CreateConstIterator(); It; ++It)
		{
			CameraCutTrackInstance = Cast<UMovieSceneCameraCutTrackInstance>(It->TrackInstance.Get());
			if (CameraCutTrackInstance != nullptr)
			{
				break;
			}
		}

		if (CameraCutTrackInstance)
		{
			if (bEnableCameraCuts)
			{
				// We locked the viewport to the cinematic... re-cache the viewport's position as the
				// new pre-animated state if we intend to restore it later. Otherwise, don't cache it,
				// so that we don't have any state to restore if we scrub/play out of the camera cut
				// section.
				if (bRestoreViewports)
				{
					ForceEditorPreAnimatedStorageOperation(CameraCutTrackInstance, EForcedCameraCutPreAnimatedStorageOperation::Cache);
				}
			}
			else
			{
				// We unlocked the viewport... if we want to restore the original viewport position, 
				// let's restore this position, which we saved as pre-animated value. If we want to stay
				// in the same position as the cinematic camera, we can simply not do anything. But we in
				// fact discard the pre-animated state altogether, because we don't want it to also come
				// back when we scrub/play out of the camera cut section.
				if (bRestoreViewports)
				{
					ForceEditorPreAnimatedStorageOperation(CameraCutTrackInstance, EForcedCameraCutPreAnimatedStorageOperation::Restore);
				}
				else
				{
					ForceEditorPreAnimatedStorageOperation(CameraCutTrackInstance, EForcedCameraCutPreAnimatedStorageOperation::Discard);
				}

				// If we have a PIE session active, we need to tell the game handler to restore its
				// pre-animated state.
				ForceGamePreAnimatedStorageRestore(CameraCutTrackInstance);
			}
		}
	}
}

#endif // WITH_EDITOR

void UMovieSceneCameraCutTrackInstance::OnInitialize()
{
#if WITH_EDITOR
	ViewportPreviewer = MakeUnique<UE::MovieScene::FCameraCutViewportPreviewer>();
#endif
}

void UMovieSceneCameraCutTrackInstance::OnAnimate()
{
	using namespace UE::MovieScene;

	// Gather active camera cuts, and triage pre-rolls from actual cuts.
	TArray<FPreRollCameraCut> CameraCutPreRolls;
	TArray<FBlendedCameraCut> CameraCutParams;

	UMovieSceneEntitySystemLinker* Linker = GetLinker();
	const FInstanceRegistry* InstanceRegistry = Linker->GetInstanceRegistry();

	for (const FCameraCutInputInfo& InputInfo : SortedInputInfos)
	{
		const FMovieSceneTrackInstanceInput& Input = InputInfo.Input;
		const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(Input.InstanceHandle);
		const FSequenceInstance& RootSequenceInstance = InstanceRegistry->GetInstance(SequenceInstance.GetRootInstanceHandle());
		const FMovieSceneContext& Context = SequenceInstance.GetContext();

		const UMovieSceneCameraCutSection* Section = Cast<const UMovieSceneCameraCutSection>(Input.Section);
		const FMovieSceneObjectBindingID CameraBindingID = Section->GetCameraBindingID();

		FTransform CutTransform = Section->InitialCameraCutTransform;
		const bool bHasCutTransform = Section->bHasInitialCameraCutTransform;

		const bool bIsBackwards = Context.GetDirection() == EPlayDirection::Backwards;

		const int32 PreRollFrames = Section->GetPreRollFrames();
		bool bIsSectionPreRoll = false;
		if (PreRollFrames > 0 && Section->HasStartFrame())
		{
			const FFrameNumber SectionStartTime = Section->GetTrueRange().GetLowerBoundValue();
			const TRange<FFrameNumber> PreRollRange(SectionStartTime - PreRollFrames, SectionStartTime);
			bIsSectionPreRoll = !bIsBackwards && PreRollRange.Contains(Context.GetTime().FloorToFrame());
		}

		if (Context.IsPreRoll() || bIsSectionPreRoll)
		{
			FPreRollCameraCut PreRollCameraCut { Input.InstanceHandle, CameraBindingID, CutTransform, bHasCutTransform };
			CameraCutPreRolls.Add(PreRollCameraCut);
		}
		else
		{
			const UMovieSceneCameraCutTrack* Track = Section->GetTypedOuter<UMovieSceneCameraCutTrack>();
			const FMovieSceneInverseSequenceTransform SequenceToRootTransform = Context.GetSequenceToRootSequenceTransform();

			FBlendedCameraCut Params(Input, CameraBindingID, SequenceInstance.GetSequenceID());
			Params.LocalDirection = Context.GetDirection();
			Params.bCanBlend = Track->bCanBlend;
			Params.bSavePreAnimatedValue = InputInfo.bSavePreAnimatedValue;

			// Get start/current/end time.
			Params.LocalContextTime = Context.GetTime();
			const TRange<FFrameNumber> SectionRange(Section->GetTrueRange());
			Params.LocalStartTime = SectionRange.HasLowerBound() ? SectionRange.GetLowerBoundValue() : FFrameNumber(-MAX_int32);
			Params.LocalEndTime = SectionRange.HasUpperBound() ? SectionRange.GetUpperBoundValue() : FFrameNumber(MAX_int32);

			// Get ease-in/out info.
			Params.LocalEaseInEndTime = Params.LocalStartTime;
			Params.LocalEaseOutStartTime = Params.LocalEndTime;
			if (Section->HasStartFrame() && Section->Easing.GetEaseInDuration() > 0)
			{
				Params.LocalEaseInEndTime = Params.LocalStartTime + Section->Easing.GetEaseInDuration();

				TOptional<FFrameTime> RootStart = SequenceToRootTransform.TryTransformTime(Params.LocalStartTime);
				TOptional<FFrameTime> RootEnd   = SequenceToRootTransform.TryTransformTime(Params.LocalEaseInEndTime);

				if (RootStart && RootEnd)
				{
					const float RootEaseInTime = RootSequenceInstance.GetContext().GetFrameRate().AsSeconds(RootEnd.GetValue() - RootStart.GetValue());
					Params.EaseIn = FBlendedCameraCutEasingInfo(RootEaseInTime, Section->Easing.EaseIn);
				}
			}
			if (Section->HasEndFrame() && Section->Easing.GetEaseOutDuration() > 0)
			{
				Params.LocalEaseOutStartTime = Params.LocalEndTime - Section->Easing.GetEaseOutDuration();

				TOptional<FFrameTime> RootStart = SequenceToRootTransform.TryTransformTime(Params.LocalEaseOutStartTime);
				TOptional<FFrameTime> RootEnd   = SequenceToRootTransform.TryTransformTime(Params.LocalEndTime);

				if (RootStart && RootEnd)
				{
					const float RootEaseOutTime = RootSequenceInstance.GetContext().GetFrameRate().AsSeconds(RootEnd.GetValue() - RootStart.GetValue());
					Params.EaseOut = FBlendedCameraCutEasingInfo(RootEaseOutTime, Section->Easing.EaseOut);
				}
			}

			// If we are evaluating backwards, swap all our start/end times.
			if (bIsBackwards)
			{
				Swap(Params.LocalStartTime, Params.LocalEndTime);
				Swap(Params.EaseIn, Params.EaseOut);
				Swap(Params.LocalEaseInEndTime, Params.LocalEaseOutStartTime);
			}

			// Remember locking option.
			Params.bLockPreviousCamera = Section->bLockPreviousCamera;

			// Get preview blending.
			const float Weight = Section->EvaluateEasing(Context.GetTime());
			Params.PreviewBlendFactor = Weight;

			CameraCutParams.Add(Params);
		}
	}

	FCameraCutAnimator Animator(CameraCutCache);
#if WITH_EDITOR
	Animator.SetViewportPreviewer(ViewportPreviewer.Get());
#endif

	// For now we only support one pre-roll.
	if (CameraCutPreRolls.Num() > 0)
	{
		FPreRollCameraCut& CameraCutPreRoll = CameraCutPreRolls.Last();
		const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(CameraCutPreRoll.InstanceHandle);
		Animator.AnimatePreRoll(CameraCutPreRoll, SequenceInstance);
	}

	// For now we only support 2 active camera cuts at most (with blending between them).
	// Remember that our param structs are sorted by hierarchical bias and global start time, i.e. the front
	// of the array has "lower" camera cuts (ones in sub-most sequences) that started most recently, while the
	// rear of the array has the "higher" camera cuts (ones in the top-most sequences) that started earlier.
	FBlendedCameraCut FinalCameraCut;

	if (CameraCutParams.Num() >= 2)
	{
		// Blending 2 camera cuts: keep track of what the next and previous shots are supposed to be.
		//
		// We know that CameraCutParams[0] has more priority than CameraCutParams[1], whether it's 
		// because it started more recently, or because it's in a lower sub-sequence (higher hierarchical bias).
		// We want to blend away from this priority cut only if we are currently in its blend-out. We'll
		// determine that thanks to the various times we saved on the info struct.
		const FBlendedCameraCut& PriorityCameraCut = CameraCutParams[0];
		const bool bIsBackwards = PriorityCameraCut.LocalDirection == EPlayDirection::Backwards;

		if (
				(!bIsBackwards && PriorityCameraCut.LocalContextTime < PriorityCameraCut.LocalEaseInEndTime) ||
				(bIsBackwards && PriorityCameraCut.LocalContextTime > PriorityCameraCut.LocalEaseInEndTime))
		{
			// Blending in from the other cut.
			const FBlendedCameraCut& PrevCameraCut = CameraCutParams[1];
			const FBlendedCameraCut& NextCameraCut = CameraCutParams[0];

			FinalCameraCut = NextCameraCut;
			FinalCameraCut.PreviousCameraBindingID = PrevCameraCut.CameraBindingID;
			FinalCameraCut.PreviousOperandSequenceID = PrevCameraCut.OperandSequenceID;
		}
		else if (
				(!bIsBackwards && PriorityCameraCut.LocalContextTime > PriorityCameraCut.LocalEaseOutStartTime) ||
				(bIsBackwards && PriorityCameraCut.LocalContextTime < PriorityCameraCut.LocalEaseOutStartTime))
		{
			// Blending out to the other cut.
			const FBlendedCameraCut& PrevCameraCut = CameraCutParams[0];
			const FBlendedCameraCut& NextCameraCut = CameraCutParams[1];

			FinalCameraCut = NextCameraCut;
			FinalCameraCut.PreviousCameraBindingID = PrevCameraCut.CameraBindingID;
			FinalCameraCut.PreviousOperandSequenceID = PrevCameraCut.OperandSequenceID;

			// We use the priority cut's ease-out info, instead of the other cut's ease-in info. This is
			// because it's more reliable:
			// - If we're just blending from one cut to the next, the editor should have made the ease-in
			//   and ease-out durations and types the same, based on the overlap between the 2 sections.
			//   So we can use either one and be fine.
			// - However we could be in the case that we're blending out from the last cut of a child sequence
			//   and back up to a cut in a parent sequence. That parent cut may have been active for a long
			//   time (overriden temporarily by the child sequence's cut), and would therefore have no
			//   ease-in information. The child cut would however have ease-out information, which we use.
			FinalCameraCut.EaseIn = PrevCameraCut.EaseOut;
			FinalCameraCut.EaseOut = FBlendedCameraCutEasingInfo();
			FinalCameraCut.PreviewBlendFactor = (1.0f - PrevCameraCut.PreviewBlendFactor);
			FinalCameraCut.bLockPreviousCamera = PrevCameraCut.bLockPreviousCamera;

			// We need to force this flag because we could be blending out from a track that does support
			// blending to a track that does not (e.g. a parent sequence track).
			FinalCameraCut.bCanBlend = true;
		}
		else
		{
			// Fully active.
			FinalCameraCut = PriorityCameraCut;
		}

		FinalCameraCut.bSavePreAnimatedValue = (CameraCutParams[0].bSavePreAnimatedValue || CameraCutParams[1].bSavePreAnimatedValue);
	}
	else if (CameraCutParams.Num() == 1)
	{
		// Only one camera cut active.
		FinalCameraCut = CameraCutParams[0];

		// It may be blending out back to gameplay however.
		const bool bIsBackwards = FinalCameraCut.LocalDirection == EPlayDirection::Backwards;
		if (
				(!bIsBackwards && FinalCameraCut.LocalContextTime > FinalCameraCut.LocalEaseOutStartTime) ||
				(bIsBackwards && FinalCameraCut.LocalContextTime < FinalCameraCut.LocalEaseOutStartTime))
		{
			FinalCameraCut.PreviewBlendFactor = 1.0f - FinalCameraCut.PreviewBlendFactor;
			FinalCameraCut.EaseIn = FinalCameraCut.EaseOut;
			FinalCameraCut.EaseOut = FBlendedCameraCutEasingInfo();
			FinalCameraCut.PreviousCameraBindingID = FinalCameraCut.CameraBindingID;
			FinalCameraCut.PreviousOperandSequenceID = FinalCameraCut.OperandSequenceID;
			FinalCameraCut.CameraBindingID = FMovieSceneObjectBindingID();
			FinalCameraCut.OperandSequenceID = FMovieSceneSequenceID();
		}
	}

	if (CameraCutParams.Num() > 0)
	{
		const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(FinalCameraCut.Input.InstanceHandle);
		Animator.AnimateBlendedCameraCut(FinalCameraCut, Linker, SequenceInstance);
	}

	// Mark all existing inputs as not needing to have their pre-animated state saved anymore.
	for (FCameraCutInputInfo& InputInfo : SortedInputInfos)
	{
		InputInfo.bSavePreAnimatedValue = false;
	}
}

void UMovieSceneCameraCutTrackInstance::OnEndUpdateInputs()
{
	using namespace UE::MovieScene;

	const FInstanceRegistry* InstanceRegistry = GetLinker()->GetInstanceRegistry();

	// Rebuild our sorted input infos.
	TMap<FMovieSceneTrackInstanceInput, float> GlobalStartTimePerSection;
	for (const FCameraCutInputInfo& InputInfo : SortedInputInfos)
	{
		GlobalStartTimePerSection.Add(InputInfo.Input, InputInfo.GlobalStartTime);
	}

	SortedInputInfos.Reset();

	for (const FMovieSceneTrackInstanceInput& Input : GetInputs())
	{
		FCameraCutInputInfo InputInfo;
		InputInfo.Input = Input;

		if (const float* GlobalStartTime = GlobalStartTimePerSection.Find(Input))
		{
			InputInfo.GlobalStartTime = *GlobalStartTime;
		}
		else
		{
			const FSequenceInstance& SequenceInstance = InstanceRegistry->GetInstance(Input.InstanceHandle);
			TSharedRef<const FSharedPlaybackState> SharedPlaybackState = SequenceInstance.GetSharedPlaybackState();

			if (UObject* PlaybackContext = SharedPlaybackState->GetPlaybackContext())
			{
				if (UWorld* World = PlaybackContext->GetWorld())
				{
					const float WorldTime = World->GetTimeSeconds();
					InputInfo.GlobalStartTime = WorldTime;
				}
			}

			// This is a new input, we may need to save its pre-animated state.
			InputInfo.bSavePreAnimatedValue = true;
		}

		SortedInputInfos.Add(InputInfo);
	}

	// Sort all active camera cuts by hierarchical bias first, and when they started in absolute game time second.
	// Later (higher starting time) cuts are sorted first, so we can prioritize the latest camera cut that started.
	Algo::Sort(SortedInputInfos,
			[InstanceRegistry](const FCameraCutInputInfo& A, const FCameraCutInputInfo& B) -> bool
			{
				const FSequenceInstance& SeqA = InstanceRegistry->GetInstance(A.Input.InstanceHandle);
				const FSequenceInstance& SeqB = InstanceRegistry->GetInstance(B.Input.InstanceHandle);
				const int32 HierarchicalBiasA = SeqA.GetContext().GetHierarchicalBias();
				const int32 HierarchicalBiasB = SeqB.GetContext().GetHierarchicalBias();
				if (HierarchicalBiasA > HierarchicalBiasB)
				{
					return true;
				}
				else if (HierarchicalBiasA < HierarchicalBiasB)
				{
					return false;
				}
				else
				{
					return A.GlobalStartTime > B.GlobalStartTime;
				}
			});
}

void UMovieSceneCameraCutTrackInstance::OnDestroyed()
{
#if WITH_EDITOR
	// Make sure we don't have any viewport modifiers registered anymore.
	ViewportPreviewer->ToggleViewportPreviewModifiers(false);
	ViewportPreviewer.Reset();
#endif
}

