// Copyright Epic Games, Inc. All Rights Reserved.

#include "TrackInstances/MovieSceneCameraCutViewportPreviewer.h"

#if WITH_EDITOR

#include "Engine/PostProcessUtils.h"
#include "Evaluation/CameraCutPlaybackCapability.h"
#include "LevelEditorViewport.h"
#include "MovieSceneCommonHelpers.h"
#include "TrackInstances/MovieSceneCameraCutEditorHandler.h"

namespace UE::MovieScene
{

void FCameraCutViewportPreviewerTarget::Get(FLevelEditorViewportClient* InClient, FVector& OutLocation, FRotator& OutRotation, float& OutFOV, FPostProcessSettings& OutPP, float& OutPPWeight) const
{
	if (CameraComponent)
	{
		FMinimalViewInfo CameraView;
		CameraComponent->GetCameraView(0.f, CameraView);

		OutLocation = CameraView.Location;
		OutRotation = CameraView.Rotation;
		OutFOV = CameraView.FOV;
		OutPP = CameraView.PostProcessSettings;
		OutPPWeight = CameraView.PostProcessBlendWeight;
		return;
	}

	if (CameraActor)
	{
		OutLocation = CameraActor->GetActorLocation();
		OutRotation = CameraActor->GetActorRotation();
		OutFOV = -1.f;
		OutPPWeight = 0.f;
		return;
	}

	if (PreAnimatedStorage)
	{
		FPreAnimatedStorageIndex StorageIndex = PreAnimatedStorage->FindStorageIndex(InClient);
		if (ensureMsgf(StorageIndex.IsValid(), TEXT("Blending camera to or from editor but can't find pre-animated viewport info!")))
		{
			FPreAnimatedCameraCutEditorState CachedValue = PreAnimatedStorage->GetCachedValue(StorageIndex);
			OutLocation = CachedValue.ViewportLocation;
			OutRotation = CachedValue.ViewportRotation;
			OutFOV = InClient->FOVAngle;
			OutPPWeight = 0.f;
			return;
		}
	}
	
	// Provide sensible defaults even if we somehow have incorrect data.
	ensureMsgf(CameraComponent || CameraActor || PreAnimatedStorage, TEXT("Invalid viewport preview target: nothing was set!"));
	OutLocation = InClient->GetViewLocation();
	OutRotation = InClient->GetViewRotation();
	OutFOV = InClient->ViewFOV;
	OutPPWeight = 0.f;
}

FCameraCutViewportPreviewer::FCameraCutViewportPreviewer()
{
}

FCameraCutViewportPreviewer::~FCameraCutViewportPreviewer()
{
	if (bViewportModifiersRegistered)
	{
		ToggleViewportPreviewModifiers(false);
	}
}

void FCameraCutViewportPreviewer::ToggleViewportPreviewModifiers(bool bEnabled)
{
	if (GEditor == nullptr || bViewportModifiersRegistered == bEnabled)
	{
		return;
	}

	bViewportModifiersRegistered = bEnabled;

	if (bEnabled)
	{
		// We aren't registered with any viewport client, let's grab them all.
		RegisteredViewportClients = GEditor->GetLevelViewportClients();
		for (FLevelEditorViewportClient* LevelVC : RegisteredViewportClients)
		{
			LevelVC->ViewModifiers.AddRaw(this, &FCameraCutViewportPreviewer::ModifyViewportClientView);
		}

		// Also listen to viewports changing.
		GEditor->OnLevelViewportClientListChanged().AddRaw(this, &FCameraCutViewportPreviewer::OnLevelViewportClientListChanged);
	}
	else
	{
		// Unregister from all the viewport clients we know of.
		for (FLevelEditorViewportClient* LevelVC : RegisteredViewportClients)
		{
			LevelVC->ViewModifiers.RemoveAll(this);
		}
		RegisteredViewportClients.Reset();

		// Stop listening to viewports changing.
		GEditor->OnLevelViewportClientListChanged().RemoveAll(this);
	}
}

void FCameraCutViewportPreviewer::SetupBlend(const FCameraCutViewportPreviewerTarget& From, const FCameraCutViewportPreviewerTarget& To, float InBlendFactor)
{
	FromTarget = From;
	ToTarget = To;
	BlendFactor = InBlendFactor;
	bApplyViewModifier = true;
}

void FCameraCutViewportPreviewer::TeardownBlend()
{
	bApplyViewModifier = false;
}

void FCameraCutViewportPreviewer::ModifyViewportClientView(FEditorViewportViewModifierParams& Params)
{
	if (!bApplyViewModifier)
	{
		return;
	}

	if (!Params.ViewportClient->AllowsCinematicControl() || Params.ViewportClient->GetViewMode() == VMI_Unknown)
	{
		return;
	}

	FVector FromViewLocation, ToViewLocation;
	FRotator FromViewRotation, ToViewRotation;
	float FromViewFOV, ToViewFOV;
	FPostProcessSettings FromPP; FPostProcessSettings ToPP;
	float FromPPWeight, ToPPWeight;
	FLevelEditorViewportClient* ViewportClient = static_cast<FLevelEditorViewportClient*>(Params.ViewportClient);

	FromTarget.Get(ViewportClient, FromViewLocation, FromViewRotation, FromViewFOV, FromPP, FromPPWeight);
	ToTarget.Get(ViewportClient, ToViewLocation, ToViewRotation, ToViewFOV, ToPP, ToPPWeight);

	const FVector BlendedLocation = FMath::Lerp(FromViewLocation, ToViewLocation, BlendFactor);
	const FRotator BlendedRotation = FMath::Lerp(FromViewRotation, ToViewRotation, BlendFactor);
	const float BlendedFOV = FMath::Lerp(FromViewFOV, ToViewFOV, BlendFactor);

	Params.ViewInfo.Location = BlendedLocation;
	Params.ViewInfo.Rotation = BlendedRotation;
	Params.ViewInfo.FOV = BlendedFOV;

	const bool bHasFromPP = (FromPPWeight > 0.f);
	const bool bHasToPP = (ToPPWeight > 0.f);

	if (bHasFromPP)
	{
		FPostProcessUtils::OverridePostProcessSettings(Params.ViewInfo.PostProcessSettings, FromPP);
	}
	if (bHasToPP)
	{
		FPostProcessUtils::BlendPostProcessSettings(Params.ViewInfo.PostProcessSettings, ToPP, BlendFactor);
	}

	if (bHasFromPP && !bHasToPP)
	{
		Params.ViewInfo.PostProcessBlendWeight = (1.f - BlendFactor);
	}
	else if (!bHasFromPP && bHasToPP)
	{
		Params.ViewInfo.PostProcessBlendWeight = BlendFactor;
	}
	else if (bHasFromPP && bHasToPP)
	{
		Params.ViewInfo.PostProcessBlendWeight = 1.f;
	}
}

void FCameraCutViewportPreviewer::OnLevelViewportClientListChanged()
{
	TSet<FLevelEditorViewportClient*> NewVCs(GEditor->GetLevelViewportClients());
	TSet<FLevelEditorViewportClient*> OldVCs(RegisteredViewportClients);

	// Register our callback on the new clients, remove it from the retired clients.
	for (FLevelEditorViewportClient* NewVC : NewVCs.Difference(OldVCs))
	{
		NewVC->ViewModifiers.AddRaw(this, &FCameraCutViewportPreviewer::ModifyViewportClientView);
	}
	for (FLevelEditorViewportClient* OldVC : OldVCs.Difference(NewVCs))
	{
		OldVC->ViewModifiers.RemoveAll(this);
	}

	RegisteredViewportClients = NewVCs.Array();
}

}  // namespace UE::MovieScene

#endif  // WITH_EDITOR

