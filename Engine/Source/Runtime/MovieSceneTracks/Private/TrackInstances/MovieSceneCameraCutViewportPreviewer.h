// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "CoreMinimal.h"
#include "UObject/WeakObjectPtr.h"

class AActor;
class FLevelEditorViewportClient;
class UCameraComponent;
struct FEditorViewportViewModifierParams;
struct FPostProcessSettings;
struct FMovieSceneCameraCutParams;

namespace UE::MovieScene
{
	struct FCameraCutPlaybackCapability;
	struct FPreAnimatedCameraCutEditorStorage;

	/**
	 * View target for the camera viewport previewer.
	 */
	struct FCameraCutViewportPreviewerTarget
	{
		// Either an actor with an optional camera component...
		AActor* CameraActor = nullptr;
		UCameraComponent* CameraComponent = nullptr;

		// ...or a pre-animated viewport position, to be found in the given storage.
		TSharedPtr<FPreAnimatedCameraCutEditorStorage> PreAnimatedStorage;

		/** Returns the target's location, rotation, and FOV */
		void Get(FLevelEditorViewportClient* InClient, FVector& OutLocation, FRotator& OutRotation, float& OutFOV, FPostProcessSettings& OutPP, float& OutPPWeight) const;
	};

	/**
	 * Utility class for previewing camera cut blends in editor.
	 *
	 * This class is able to register view modifiers on editor viewports and, if needed,
	 * move the viewport's point-of-view into a blend between two targets. This is used 
	 * by FCameraCutEditorHandler to preview blending, but should be inactive when no
	 * blending is needed.
	 */
	class FCameraCutViewportPreviewer
	{
	public:
		FCameraCutViewportPreviewer();
		~FCameraCutViewportPreviewer();

		/**
		 * Toggle whether this previewer is running a blend preview.
		 * This will register/unregister the view modifiers on all editor viewports that
		 * have cinematic control enabled.
		 */
		void ToggleViewportPreviewModifiers(bool bEnable);

		/**
		 * Sets up a blend to be previewed between the specified targets, using the specified
		 * blend factor. This will only do something if ToggleViewportPreviewModifiers(true) has
		 * been called.
		 */
		void SetupBlend(const FCameraCutViewportPreviewerTarget& From, const FCameraCutViewportPreviewerTarget& To, float InBlendFactor);

		/**
		 * Stops previewing any blend. This will make the view modifier dormant but won't
		 * unregister it (that's only done with ToggleViewportPreviewModifiers(false)).
		 */
		void TeardownBlend();

	private:

		void UpdatePreviewLevelViewportClientFromCameraCut(FLevelEditorViewportClient& InViewportClient, UObject* InCameraObject, const FMovieSceneCameraCutParams& CameraCutParams);
		void ModifyViewportClientView(FEditorViewportViewModifierParams& Params);
		void OnLevelViewportClientListChanged();

	private:
		/** Target from which to blend */
		FCameraCutViewportPreviewerTarget FromTarget;
		/** Target towards which to blend */
		FCameraCutViewportPreviewerTarget ToTarget;
		/** Blend factor between both targets */
		float BlendFactor;

		/** Whether to apply the modifier with the blended view information */
		bool bApplyViewModifier;

		/** Whether we have registered viewport modifiers in the editor. */
		bool bViewportModifiersRegistered = false;
		/** Which viewport clients we have registered with */
		TArray<FLevelEditorViewportClient*> RegisteredViewportClients;
	};

}  // namespace UE::MovieScene

#endif  // WITH_EDITOR

