// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieScene3DAttachTrack.h"
#include "Sections/MovieScene3DAttachSection.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DAttachTrack)


#define LOCTEXT_NAMESPACE "MovieScene3DAttachTrack"


UMovieScene3DAttachTrack::UMovieScene3DAttachTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{ }

UMovieSceneSection* UMovieScene3DAttachTrack::AddConstraint(FFrameNumber KeyTime, int32 Duration, const FName SocketName, const FName ComponentName, const FMovieSceneObjectBindingID& ConstraintBindingID)
{
	// add the section
	UMovieScene3DAttachSection* NewSection = NewObject<UMovieScene3DAttachSection>(this, NAME_None, RF_Transactional);
	NewSection->SetAttachTargetID(ConstraintBindingID);
	NewSection->InitialPlacement(ConstraintSections, KeyTime, Duration, SupportsMultipleRows());
	NewSection->AttachSocketName = SocketName;
	NewSection->AttachComponentName = ComponentName;

	ConstraintSections.Add(NewSection);

	return NewSection;
}

bool UMovieScene3DAttachTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieScene3DAttachSection::StaticClass();
}

UMovieSceneSection* UMovieScene3DAttachTrack::CreateNewSection()
{
	return NewObject<UMovieScene3DAttachSection>(this, NAME_None, RF_Transactional);
}

#if WITH_EDITORONLY_DATA
FText UMovieScene3DAttachTrack::GetDisplayName() const
{
	return LOCTEXT("TrackName", "Attach");
}
#endif


#undef LOCTEXT_NAMESPACE

