// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieScene3DConstraintTrack.h"
#include "Sections/MovieScene3DConstraintSection.h"
#include "IMovieScenePlayer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DConstraintTrack)


#define LOCTEXT_NAMESPACE "MovieScene3DConstraintTrack"


UMovieScene3DConstraintTrack::UMovieScene3DConstraintTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(128, 90, 0, 65);
#endif
}

bool UMovieScene3DConstraintTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieScene3DConstraintSection::StaticClass();
}

UMovieSceneSection* UMovieScene3DConstraintTrack::CreateNewSection()
{
	return NewObject<UMovieScene3DConstraintSection>(this, NAME_None, RF_Transactional);
}


const TArray<UMovieSceneSection*>& UMovieScene3DConstraintTrack::GetAllSections() const
{
	return ConstraintSections;
}


void UMovieScene3DConstraintTrack::RemoveAllAnimationData()
{
	ConstraintSections.Empty();
}


bool UMovieScene3DConstraintTrack::HasSection(const UMovieSceneSection& Section) const
{
	return ConstraintSections.Contains(&Section);
}


void UMovieScene3DConstraintTrack::AddSection(UMovieSceneSection& Section)
{
	ConstraintSections.Add(&Section);
}


void UMovieScene3DConstraintTrack::RemoveSection(UMovieSceneSection& Section)
{
	ConstraintSections.Remove(&Section);
}


void UMovieScene3DConstraintTrack::RemoveSectionAt(int32 SectionIndex)
{
	ConstraintSections.RemoveAt(SectionIndex);
}


bool UMovieScene3DConstraintTrack::IsEmpty() const
{
	return ConstraintSections.Num() == 0;
}

#if WITH_EDITORONLY_DATA
FText UMovieScene3DConstraintTrack::GetDisplayName() const
{
	return LOCTEXT("TrackName", "Constraint");
}
#endif

#undef LOCTEXT_NAMESPACE

