// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieScene3DPathTrack.h"
#include "Sections/MovieScene3DPathSection.h"
#include "Evaluation/MovieScene3DPathTemplate.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DPathTrack)


#define LOCTEXT_NAMESPACE "MovieScene3DPathTrack"


UMovieScene3DPathTrack::UMovieScene3DPathTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{ }

bool UMovieScene3DPathTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieScene3DPathSection::StaticClass();
}

UMovieSceneSection* UMovieScene3DPathTrack::CreateNewSection()
{
	return NewObject<UMovieScene3DPathSection>(this, NAME_None, RF_Transactional);
}


FMovieSceneEvalTemplatePtr UMovieScene3DPathTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieScene3DPathSectionTemplate(*CastChecked<UMovieScene3DPathSection>(&InSection));
}


UMovieSceneSection* UMovieScene3DPathTrack::AddConstraint(FFrameNumber KeyTime, int32 Duration, const FName SocketName, const FName ComponentName, const FMovieSceneObjectBindingID& ConstraintBindingID)
{
	UMovieScene3DPathSection* NewSection = NewObject<UMovieScene3DPathSection>(this, NAME_None, RF_Transactional);
	{
		NewSection->SetPathBindingID( ConstraintBindingID );
		NewSection->InitialPlacement( ConstraintSections, KeyTime, Duration, SupportsMultipleRows() );
	}

	ConstraintSections.Add(NewSection);

	return NewSection;
}


#if WITH_EDITORONLY_DATA
FText UMovieScene3DPathTrack::GetDisplayName() const
{
	return LOCTEXT("TrackName", "Path");
}
#endif


#undef LOCTEXT_NAMESPACE

