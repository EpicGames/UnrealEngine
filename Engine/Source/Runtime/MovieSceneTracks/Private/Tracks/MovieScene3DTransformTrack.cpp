// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieScene3DTransformTrack.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "Compilation/MovieSceneTemplateInterrogation.h"
#include "Compilation/MovieSceneSegmentCompiler.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "Systems/MovieSceneQuaternionBlenderSystem.h"
#include "Systems/MovieScenePiecewiseDoubleBlenderSystem.h"
#include "MovieSceneCommonHelpers.h"
#include "Algo/BinarySearch.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScene3DTransformTrack)

UMovieScene3DTransformTrack::UMovieScene3DTransformTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	static FName Transform("Transform");
	SetPropertyNameAndPath(Transform, Transform.ToString());

	SupportedBlendTypes = FMovieSceneBlendTypeField::All();

#if WITH_EDITORONLY_DATA
	TrackTint = FColor(65, 173, 164, 65);
#endif

	EvalOptions.bEvaluateNearestSection_DEPRECATED = EvalOptions.bCanEvaluateNearestSection = true;
}

bool UMovieScene3DTransformTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieScene3DTransformSection::StaticClass();
}

UMovieSceneSection* UMovieScene3DTransformTrack::CreateNewSection()
{
	return NewObject<UMovieScene3DTransformSection>(this, NAME_None, RF_Transactional);
}

#if WITH_EDITORONLY_DATA
FText UMovieScene3DTransformTrack::GetDisplayName() const
{
	FText CustomDisplayName = Super::GetDisplayName();

	static FName Transform("Transform");
	if (CustomDisplayName.ToString() == Transform)
	{
		return NSLOCTEXT("MovieScene3DTransformTrack", "DefaultLabel", "Transform");
	}
	return CustomDisplayName;
}
#endif

TSubclassOf<UMovieSceneBlenderSystem> UMovieScene3DTransformTrack::GetBlenderSystem() const
{
	if (BlenderSystemClass == nullptr)
	{
		return UMovieScenePiecewiseDoubleBlenderSystem::StaticClass();
	}
	return BlenderSystemClass;
}

void UMovieScene3DTransformTrack::SetBlenderSystem(TSubclassOf<UMovieSceneBlenderSystem> InBlenderSystemClass)
{
	if (InBlenderSystemClass == UMovieScenePiecewiseDoubleBlenderSystem::StaticClass())
	{
		InBlenderSystemClass = nullptr;
	}

	BlenderSystemClass = InBlenderSystemClass;
}

void UMovieScene3DTransformTrack::GetSupportedBlenderSystems(TArray<TSubclassOf<UMovieSceneBlenderSystem>>& OutSystemClasses) const
{
	OutSystemClasses.Add(UMovieSceneQuaternionBlenderSystem::StaticClass());
	OutSystemClasses.Add(UMovieScenePiecewiseDoubleBlenderSystem::StaticClass());
}

#if WITH_EDITOR

uint32 GetDistanceFromTo(FFrameNumber TestValue, FFrameNumber TargetValue)
{
	static const int32 MaxInt = TNumericLimits<int32>::Max();
	if (TestValue == TargetValue)
	{
		return 0;
	}
	else if (TestValue > TargetValue)
	{
		return (uint32(TestValue.Value + MaxInt) - uint32(TargetValue.Value + MaxInt));
	}
	else
	{
		return (uint32(TargetValue.Value + MaxInt) - uint32(TestValue.Value + MaxInt));
	}
}

TArray<FTrajectoryKey> UMovieScene3DTransformTrack::GetTrajectoryData(FFrameNumber Time, int32 MaxNumDataPoints, TRange<FFrameNumber> ViewRange) const
{
	struct FCurveKeyIterator
	{
		FCurveKeyIterator(UMovieScene3DTransformSection* InSection, FMovieSceneDoubleChannel* InChannel, FName InChannelName, FFrameNumber StartTime, TRange<FFrameNumber> ViewRange)
			: Section(InSection), Channel(InChannel->GetData()), ChannelName(InChannelName), SectionRange(TRange<FFrameNumber>::Intersection(ViewRange, InSection->GetRange())), CurrentIndex(INDEX_NONE)
		{
			TArrayView<const FFrameNumber> Times = Channel.GetTimes();
			CurrentIndex = Algo::LowerBound(Times, StartTime);

			bIsLowerBound = false;
			bIsUpperBound = CurrentIndex == Times.Num() && SectionRange.GetUpperBound().IsClosed();
		}

		bool IsValid() const
		{
			return Channel.GetTimes().IsValidIndex(CurrentIndex);
		}

		FFrameNumber GetCurrentKeyTime() const
		{
			return Channel.GetTimes()[CurrentIndex];
		}

		FCurveKeyIterator& operator--()
		{
			TArrayView<const FFrameNumber> Times = Channel.GetTimes();

			if (bIsLowerBound)
			{
				bIsLowerBound = false;
				CurrentIndex = INDEX_NONE;
			}
			else
			{
				if (bIsUpperBound)
				{
					bIsUpperBound = false;
					CurrentIndex = Algo::LowerBound(Times, SectionRange.GetUpperBoundValue()) - 1;
				}
				else
				{
					--CurrentIndex;
				}

				bIsLowerBound = SectionRange.GetLowerBound().IsClosed() && (!Times.IsValidIndex(CurrentIndex) || !SectionRange.Contains(Channel.GetTimes()[CurrentIndex]) );
			}
			return *this;
		}

		FCurveKeyIterator& operator++()
		{
			TArrayView<const FFrameNumber> Times = Channel.GetTimes();

			if (bIsUpperBound)
			{
				bIsUpperBound = false;
				CurrentIndex = INDEX_NONE;
			}
			else
			{
				if (bIsLowerBound)
				{
					bIsLowerBound = false;
					CurrentIndex = Algo::UpperBound(Times, SectionRange.GetLowerBoundValue());
				}
				else 
				{
					++CurrentIndex;
				}

				bIsUpperBound = SectionRange.GetUpperBound().IsClosed() && (!Times.IsValidIndex(CurrentIndex) || !SectionRange.Contains(Times[CurrentIndex]) );
			}
			return *this;
		}

		explicit operator bool() const
		{
			TArrayView<const FFrameNumber> Times = Channel.GetTimes();
			return ( bIsLowerBound || bIsUpperBound ) || ( Times.IsValidIndex(CurrentIndex) && SectionRange.Contains(Times[CurrentIndex]) );
		}

		FFrameNumber GetTime() const
		{
			return bIsLowerBound ? SectionRange.GetLowerBoundValue() : bIsUpperBound ? SectionRange.GetUpperBoundValue() : Channel.GetTimes()[CurrentIndex];
		}

		ERichCurveInterpMode GetInterpMode() const
		{
			return ( bIsLowerBound || bIsUpperBound ) ? RCIM_None : Channel.GetValues()[CurrentIndex].InterpMode.GetValue();
		}

		UMovieScene3DTransformSection* GetSection() const
		{
			return Section;
		}

		FName GetChannelName() const
		{
			return ChannelName;
		}

		TOptional<FKeyHandle> GetKeyHandle()
		{
			return CurrentIndex == INDEX_NONE ? TOptional<FKeyHandle>() : Channel.GetHandle(CurrentIndex);
		}

	private:
		UMovieScene3DTransformSection* Section;
		TMovieSceneChannelData<FMovieSceneDoubleValue> Channel;
		FName ChannelName;
		TRange<FFrameNumber> SectionRange;
		int32 CurrentIndex;
		bool bIsUpperBound, bIsLowerBound;
	};

	TArray<FCurveKeyIterator> ForwardIters;
	TArray<FCurveKeyIterator> BackwardIters;

	for (UMovieSceneSection* Section : Sections)
	{
		UMovieScene3DTransformSection* TransformSection = Cast<UMovieScene3DTransformSection>(Section);
		if (TransformSection && MovieSceneHelpers::IsSectionKeyable(TransformSection))
		{
			FMovieSceneChannelProxy& SectionChannelProxy = TransformSection->GetChannelProxy();
			TMovieSceneChannelHandle<FMovieSceneDoubleChannel> ChannelHandles[] = {
				SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.X"),
				SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Y"),
				SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Location.Z"),
				SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.X"),
				SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Y"),
				SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Rotation.Z"),
				SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.X"),
				SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Y"),
				SectionChannelProxy.GetChannelByName<FMovieSceneDoubleChannel>("Scale.Z")
			};

			if (ChannelHandles[0].Get())
			{
				ForwardIters.Emplace(TransformSection, ChannelHandles[0].Get(), TEXT("Location.X"), Time, ViewRange);
				BackwardIters.Emplace(TransformSection, ChannelHandles[0].Get(), TEXT("Location.X"), Time, ViewRange);
			}
			if (ChannelHandles[1].Get())
			{
				ForwardIters.Emplace(TransformSection, ChannelHandles[1].Get(), TEXT("Location.Y"), Time, ViewRange);
				BackwardIters.Emplace(TransformSection, ChannelHandles[1].Get(), TEXT("Location.Y"), Time, ViewRange);
			}
			if (ChannelHandles[2].Get())
			{
				ForwardIters.Emplace(TransformSection, ChannelHandles[2].Get(), TEXT("Location.Z"), Time, ViewRange);
				BackwardIters.Emplace(TransformSection, ChannelHandles[2].Get(), TEXT("Location.Z"), Time, ViewRange);
			}
			if (ChannelHandles[3].Get())
			{
				ForwardIters.Emplace(TransformSection, ChannelHandles[3].Get(), TEXT("Rotation.X"), Time, ViewRange);
				BackwardIters.Emplace(TransformSection, ChannelHandles[3].Get(), TEXT("Rotation.X"), Time, ViewRange);
			}
			if (ChannelHandles[4].Get())
			{
				ForwardIters.Emplace(TransformSection, ChannelHandles[4].Get(), TEXT("Rotation.Y"), Time, ViewRange);
				BackwardIters.Emplace(TransformSection, ChannelHandles[4].Get(), TEXT("Rotation.Y"), Time, ViewRange);
			}
			if (ChannelHandles[5].Get())
			{
				ForwardIters.Emplace(TransformSection, ChannelHandles[5].Get(), TEXT("Rotation.Z"), Time, ViewRange);
				BackwardIters.Emplace(TransformSection, ChannelHandles[5].Get(), TEXT("Rotation.Z"), Time, ViewRange);
			}
		}
	}

	auto HasAnyValidIterators = [](const FCurveKeyIterator& It)
	{
		return bool(It);
	};

	TArray<FTrajectoryKey> Result;
	while (ForwardIters.ContainsByPredicate(HasAnyValidIterators) || BackwardIters.ContainsByPredicate(HasAnyValidIterators))
	{
		if (MaxNumDataPoints != 0 && Result.Num() >= MaxNumDataPoints)
		{
			break;
		}

		uint32 ClosestDistance = -1;
		TOptional<FFrameNumber> ClosestTime;

		// Find the closest key time
		for (const FCurveKeyIterator& Fwd : ForwardIters)
		{
			if (Fwd && ( !ClosestTime.IsSet() || GetDistanceFromTo(Fwd.GetTime(), Time) < ClosestDistance ) )
			{
				ClosestTime = Fwd.GetTime();
				ClosestDistance = GetDistanceFromTo(Fwd.GetTime(), Time);
			}
		}
		for (const FCurveKeyIterator& Bck : BackwardIters)
		{
			if (Bck && ( !ClosestTime.IsSet() || GetDistanceFromTo(Bck.GetTime(), Time) < ClosestDistance ) )
			{
				ClosestTime = Bck.GetTime();
				ClosestDistance = GetDistanceFromTo(Bck.GetTime(), Time);
			}
		}

		
		FTrajectoryKey& NewKey = Result[Result.Emplace(ClosestTime.GetValue())];
		for (FCurveKeyIterator& Fwd : ForwardIters)
		{
			if (Fwd && Fwd.GetTime() == NewKey.Time)
			{
				if (Fwd.IsValid())
				{
					// Add this key to the trajectory key
					NewKey.KeyData.Emplace(Fwd.GetSection(), Fwd.GetKeyHandle(), Fwd.GetInterpMode(), Fwd.GetChannelName());
				}
				// Move onto the next key in this curve
				++Fwd;
			}
		}
		
		for (FCurveKeyIterator& Bck : BackwardIters)
		{
			if (Bck && Bck.GetTime() == NewKey.Time)
			{
				if (Bck.IsValid())
				{
					// Add this key to the trajectory key
					NewKey.KeyData.Emplace(Bck.GetSection(), Bck.GetKeyHandle(), Bck.GetInterpMode(), Bck.GetChannelName());
				}
				// Move onto the next key in this curve
				--Bck;
			}
		}
	}

	Result.Sort(
		[](const FTrajectoryKey& A, const FTrajectoryKey& B)
		{
			return A.Time < B.Time;
		}
	);

	return Result;
}

FSlateColor UMovieScene3DTransformTrack::GetLabelColor(const FMovieSceneLabelParams& LabelParams) const
{
	return LabelParams.bIsDimmed ? FSlateColor::UseSubduedForeground() : FSlateColor::UseForeground();
}

#endif	// WITH_EDITOR

