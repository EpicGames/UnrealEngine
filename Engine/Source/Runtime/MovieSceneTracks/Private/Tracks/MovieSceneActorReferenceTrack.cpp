// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneActorReferenceTrack.h"
#include "Sections/MovieSceneActorReferenceSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneActorReferenceTrack)


UMovieSceneActorReferenceTrack::UMovieSceneActorReferenceTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{ }

bool UMovieSceneActorReferenceTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneActorReferenceSection::StaticClass();
}

UMovieSceneSection* UMovieSceneActorReferenceTrack::CreateNewSection()
{
	return NewObject<UMovieSceneActorReferenceSection>(this, NAME_None, RF_Transactional);
}

