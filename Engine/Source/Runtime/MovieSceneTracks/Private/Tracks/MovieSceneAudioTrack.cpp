// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneAudioTrack.h"
#include "Decorations/MovieSceneSectionAnchorsDecoration.h"
#include "Audio.h"
#include "Sound/SoundWave.h"
#include "MovieScene.h"
#include "Sections/MovieSceneAudioSection.h"
#include "Kismet/GameplayStatics.h"
#include "AudioDecompress.h"
#include "Evaluation/MovieSceneSegment.h"
#include "Compilation/MovieSceneSegmentCompiler.h"
#include "MovieSceneCommonHelpers.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneAudioTrack)

#define LOCTEXT_NAMESPACE "MovieSceneAudioTrack"


UMovieSceneAudioTrack::UMovieSceneAudioTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(93, 95, 136);
	RowHeight = 50;
#endif

	bAutoExpandToKeyframes = false;
}

const TArray<UMovieSceneSection*>& UMovieSceneAudioTrack::GetAllSections() const
{
	return AudioSections;
}


bool UMovieSceneAudioTrack::SupportsMultipleRows() const
{
	return true;
}

bool UMovieSceneAudioTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneAudioSection::StaticClass();
}

void UMovieSceneAudioTrack::RemoveAllAnimationData()
{
	AudioSections.Empty();
}


bool UMovieSceneAudioTrack::HasSection(const UMovieSceneSection& Section) const
{
	return AudioSections.Contains(&Section);
}


void UMovieSceneAudioTrack::AddSection(UMovieSceneSection& Section)
{
	AudioSections.Add(&Section);
}


void UMovieSceneAudioTrack::RemoveSection(UMovieSceneSection& Section)
{
	AudioSections.Remove(&Section);
}


void UMovieSceneAudioTrack::RemoveSectionAt(int32 SectionIndex)
{
	AudioSections.RemoveAt(SectionIndex);
}


bool UMovieSceneAudioTrack::IsEmpty() const
{
	return AudioSections.Num() == 0;
}


void UMovieSceneAudioTrack::GetCompatibleUserDecorationsImpl(TSet<UClass*>& OutClasses) const
{
	OutClasses.Add(UMovieSceneSectionAnchorsDecoration::StaticClass());
}


UMovieSceneSection* UMovieSceneAudioTrack::AddNewSoundOnRow(USoundBase* Sound, FFrameNumber Time, int32 RowIndex)
{
	check(Sound);
	
	FFrameRate FrameRate = GetTypedOuter<UMovieScene>()->GetTickResolution();

	// determine initial duration
	// @todo Once we have infinite sections, we can remove this
	// @todo ^^ Why? Infinte sections would mean there's no starting time?
	FFrameTime DurationToUse = 1.f * FrameRate; // if all else fails, use 1 second duration

	const float SoundDuration = MovieSceneHelpers::GetSoundDuration(Sound);
	if (SoundDuration != INDEFINITELY_LOOPING_DURATION && SoundDuration > 0)
	{
		DurationToUse = SoundDuration * FrameRate;
	}

	// add the section
	UMovieSceneAudioSection* NewSection = Cast<UMovieSceneAudioSection>(CreateNewSection());
	NewSection->InitialPlacementOnRow( AudioSections, Time, DurationToUse.FrameNumber.Value, RowIndex );
	NewSection->SetSound(Sound);

#if WITH_EDITORONLY_DATA
	// Use the timecode info from the sound wave if it's available to populate
	// the section's TimecodeSource property. Otherwise try the base sound's
	// timecode offset.
	TOptional<FSoundWaveTimecodeInfo> TimecodeInfo;
	if (const USoundWave* SoundWave = Cast<USoundWave>(Sound))
	{
		TimecodeInfo = SoundWave->GetTimecodeInfo();
	}

	if (TimecodeInfo.IsSet())
	{
		const double NumSecondsSinceMidnight = TimecodeInfo->GetNumSecondsSinceMidnight();
		const FTimecode Timecode(NumSecondsSinceMidnight, TimecodeInfo->TimecodeRate, TimecodeInfo->bTimecodeIsDropFrame, /* InbRollover = */ true);

		NewSection->TimecodeSource = FMovieSceneTimecodeSource(Timecode);
	}
	else
	{
		const TOptional<FSoundTimecodeOffset> TimecodeOffset = Sound->GetTimecodeOffset();
		if (TimecodeOffset.IsSet())
		{
			const double NumSecondsSinceMidnight = TimecodeOffset->NumOfSecondsSinceMidnight;

			// The timecode offset does not carry a rate with it, so just use the
			// display rate for this movie scene.
			const FFrameRate DisplayRate = GetTypedOuter<UMovieScene>()->GetDisplayRate();
			const FTimecode Timecode(NumSecondsSinceMidnight, DisplayRate, /* InbRollover = */ true);

			NewSection->TimecodeSource = FMovieSceneTimecodeSource(Timecode);
		}
	}
#endif

	AudioSections.Add(NewSection);

	return NewSection;
}

UMovieSceneSection* UMovieSceneAudioTrack::CreateNewSection()
{
	return NewObject<UMovieSceneAudioSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneAudioTrack::PostRename(UObject* OldOuter, const FName OldName)
{
	Super::PostRename(OldOuter, OldName);

	// Recache the channel proxy because attach in FAudioChannelEditorData is dependent upon the outer chain
	for (TObjectPtr<UMovieSceneSection>& Section : AudioSections)
	{
		if (UMovieSceneAudioSection* AudioSection = Cast<UMovieSceneAudioSection>(Section.Get()))
		{
			AudioSection->CacheChannelProxy();
		}
	}
}

#undef LOCTEXT_NAMESPACE

