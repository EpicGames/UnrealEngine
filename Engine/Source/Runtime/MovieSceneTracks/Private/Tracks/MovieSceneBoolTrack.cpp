// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneBoolTrack.h"
#include "MovieSceneCommonHelpers.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Evaluation/MovieScenePropertyTemplates.h"
#include "Algo/IndexOf.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneBoolTrack)

namespace UE::MovieScene
{
	MOVIESCENETRACKS_API bool GBooleanPropertyLegacyTemplate = false;
	static FAutoConsoleVariableRef CVarBooleanPropertyECS(
		TEXT("Sequencer.UseLegacyBooleanTemplate"),
		GBooleanPropertyLegacyTemplate,
		TEXT("(Default: false) DEPRECATED Whether to use Sequencer's legacy template evauation for boolean properties or not."));

	MOVIESCENETRACKS_API bool GCompileBooleanPropertyLegacyTemplate = true;
	static FAutoConsoleVariableRef CVarCompileBooleanPropertyECS(
		TEXT("Sequencer.CanUseLegacyBooleanTemplate"),
		GCompileBooleanPropertyLegacyTemplate,
		TEXT("(Default: true) DEPRECATED Whether to compile Sequencer's legacy template boolean properties at all. Disabling this cvar will completely disable the legacy template."));

}

bool UMovieSceneBoolTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneBoolSection::StaticClass();
}

UMovieSceneSection* UMovieSceneBoolTrack::CreateNewSection()
{
	return NewObject<UMovieSceneBoolSection>(this, NAME_None, RF_Transactional);
}


FMovieSceneEvalTemplatePtr UMovieSceneBoolTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	if (UE::MovieScene::GCompileBooleanPropertyLegacyTemplate)
	{
		return FMovieSceneBoolPropertySectionTemplate(*CastChecked<const UMovieSceneBoolSection>(&InSection), *this);
	}
	return FMovieSceneEvalTemplatePtr();
}

void UMovieSceneBoolTrack::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (GBooleanPropertyLegacyTemplate)
	{
		return;
	}

	const int32 SectionIndex = static_cast<int32>(Params.EntityID);

	UMovieSceneBoolSection* BoolSection = CastChecked<UMovieSceneBoolSection>(Sections[SectionIndex]);

	if (!BoolSection->GetChannel().HasAnyData())
	{
		return;
	}

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	// Entity ID for the track is the index of the section, but for FPropertyTrackEntityImportHelper it expects 0/1.
	//   Boolean tracks do not support edit condition overrides (for now), so we just force it to 0 for the main property entity
	FEntityImportParams SectionParams = Params;
	SectionParams.EntityID = 0;

	FPropertyTrackEntityImportHelper(TracksComponents->Bool)
		.Add(Components->BoolChannel, &BoolSection->GetChannel())
		.Add(Components->BlenderType, UMovieScenePiecewiseBoolBlenderSystem::StaticClass())
		.Commit(BoolSection, SectionParams, OutImportedEntity);

	BoolSection->BuildDefaultComponents(EntityLinker, Params, OutImportedEntity);
}

bool UMovieSceneBoolTrack::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& InEffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : GetEvaluationField().Entries)
	{
		if (Entry.Section && IsRowEvalDisabled(Entry.Section->GetRowIndex()))
		{
			continue;
		}

		// This codepath should only ever execute for the highest level so we do not need to do any transformations
		TRange<FFrameNumber> EffectiveRange = TRange<FFrameNumber>::Intersection(InEffectiveRange, Entry.Range);
		if (!EffectiveRange.IsEmpty())
		{
			const int32 SectionIndex = Algo::IndexOf(Sections, Entry.Section);
			if (SectionIndex == INDEX_NONE)
			{
				continue;
			}

			FMovieSceneEvaluationFieldEntityMetaData MetaData = InMetaData;

			MetaData.ForcedTime = Entry.ForcedTime;
			MetaData.Flags      = Entry.Flags;
			MetaData.bEvaluateInSequencePreRoll  = EvalOptions.bEvaluateInPreroll;
			MetaData.bEvaluateInSequencePostRoll = EvalOptions.bEvaluateInPostroll;
			MetaData.Condition = MovieSceneHelpers::GetSequenceCondition(this, Entry.Section, true);

			const int32 EntityIndex   = OutFieldBuilder->FindOrAddEntity(this, static_cast<uint32>(SectionIndex));
			const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(MetaData);

			OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
		}
	}

	return true;
}