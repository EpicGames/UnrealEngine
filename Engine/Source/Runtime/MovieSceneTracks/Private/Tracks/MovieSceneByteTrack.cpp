// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneByteTrack.h"
#include "Sections/MovieSceneByteSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneByteTrack)

UMovieSceneByteTrack::UMovieSceneByteTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
}

void UMovieSceneByteTrack::PostLoad()
{
	Super::PostLoad();

	SetEnum(Enum);
}

bool UMovieSceneByteTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneByteSection::StaticClass();
}

UMovieSceneSection* UMovieSceneByteTrack::CreateNewSection()
{
	UMovieSceneByteSection* NewByteSection = NewObject<UMovieSceneByteSection>(this, NAME_None, RF_Transactional);
	NewByteSection->ByteCurve.SetEnum(Enum);
	return NewByteSection;
}

void UMovieSceneByteTrack::SetEnum(UEnum* InEnum)
{
	Enum = InEnum;
	for (UMovieSceneSection* Section : Sections)
	{
		if (UMovieSceneByteSection* ByteSection = Cast<UMovieSceneByteSection>(Section))
		{
			ByteSection->ByteCurve.SetEnum(Enum);
		}
	}
}


UEnum* UMovieSceneByteTrack::GetEnum() const
{
	return Enum;
}

