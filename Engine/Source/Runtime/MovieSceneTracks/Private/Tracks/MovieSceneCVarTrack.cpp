// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneCVarTrack.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "Sections/MovieSceneCVarSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCVarTrack)


#define LOCTEXT_NAMESPACE "MovieSceneCVarTrack"

UMovieSceneCVarTrack::UMovieSceneCVarTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(23, 89, 145, 65);
#endif

	// By default, don't evaluate cvar values in pre and postroll as that would conflict with
	// the previous shot's desired values.
	EvalOptions.bEvaluateInPreroll = EvalOptions.bEvaluateInPostroll = false;
	// SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
}

/* UMovieSceneTrack interface
 *****************************************************************************/

void UMovieSceneCVarTrack::AddSection(UMovieSceneSection& Section)
{
	if (UMovieSceneCVarSection* CVarSection = Cast<UMovieSceneCVarSection>(&Section))
	{
		Sections.Add(CVarSection);
	}
	
	// todo: Should this call MovieSceneHelpers::SortConsecutiveSections(Sections); ?
}

bool UMovieSceneCVarTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneCVarSection::StaticClass();
}

UMovieSceneSection* UMovieSceneCVarTrack::CreateNewSection()
{
	return NewObject<UMovieSceneCVarSection>(this, NAME_None, RF_Transactional);
}

bool UMovieSceneCVarTrack::SupportsMultipleRows() const
{
	return true;
}

EMovieSceneTrackEasingSupportFlags UMovieSceneCVarTrack::SupportsEasing(FMovieSceneSupportsEasingParams& Params) const
{
	return EMovieSceneTrackEasingSupportFlags::None;
}

const TArray<UMovieSceneSection*>& UMovieSceneCVarTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneCVarTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}


bool UMovieSceneCVarTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

void UMovieSceneCVarTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneCVarTrack::RemoveSectionAt(int32 SectionIndex)
{
	UMovieSceneSection* SectionToDelete = Sections[SectionIndex];
	Sections.RemoveAt(SectionIndex);
	MovieSceneHelpers::SortConsecutiveSections(MutableView(Sections));
}

void UMovieSceneCVarTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneCVarTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Console Variable");
}
#endif

#undef LOCTEXT_NAMESPACE

