// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneCameraCutTrack.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTimeHelpers.h"
#include "Sections/MovieSceneCameraCutSection.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "IMovieSceneTracksModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraCutTrack)

#define LOCTEXT_NAMESPACE "MovieSceneCameraCutTrack"

/* UMovieSceneCameraCutTrack interface
 *****************************************************************************/
UMovieSceneCameraCutTrack::UMovieSceneCameraCutTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
	, bCanBlend(false)
	, bAutoArrangeSections(true)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(120, 120, 120, 65);
#endif

	// By default, don't evaluate camera cuts in pre and postroll
	EvalOptions.bEvaluateInPreroll = EvalOptions.bEvaluateInPostroll = false;

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
}

UMovieSceneCameraCutSection* UMovieSceneCameraCutTrack::AddNewCameraCut(const FMovieSceneObjectBindingID& CameraBindingID, FFrameNumber StartTime)
{
	Modify();

	FFrameNumber NewSectionEndTime = FindEndTimeForCameraCut(StartTime);

	// If there's an existing section, just swap the camera guid
	UMovieSceneCameraCutSection* ExistingSection = nullptr;
	for (auto Section : Sections)
	{
		if (Section->HasStartFrame() && Section->HasEndFrame() && Section->GetInclusiveStartFrame() == StartTime && Section->GetExclusiveEndFrame() == NewSectionEndTime)
		{
			ExistingSection = Cast<UMovieSceneCameraCutSection>(Section);
			break;
		}
	}

	UMovieSceneCameraCutSection* NewSection = ExistingSection;
	if (ExistingSection != nullptr)
	{
		ExistingSection->Modify();
		ExistingSection->SetCameraBindingID(CameraBindingID);
	}
	else
	{
		NewSection = NewObject<UMovieSceneCameraCutSection>(this, NAME_None, RF_Transactional);
		NewSection->SetRange(TRange<FFrameNumber>(StartTime, NewSectionEndTime));
		NewSection->SetCameraBindingID(CameraBindingID);

		AddSection(*NewSection);
	}

	// When a new CameraCut is added, sort all CameraCuts to ensure they are in the correct order
	MovieSceneHelpers::SortConsecutiveSections(MutableView(Sections));

	// Once CameraCuts are sorted fixup the surrounding CameraCuts to fix any gaps
	AutoArrangeSectionsIfNeeded(*NewSection, false);

	return NewSection;
}


/* UMovieSceneTrack interface
 *****************************************************************************/

void UMovieSceneCameraCutTrack::AddSection(UMovieSceneSection& Section)
{
	if (UMovieSceneCameraCutSection* CutSection = Cast<UMovieSceneCameraCutSection>(&Section))
	{
		Sections.Add(CutSection);
	}
}

bool UMovieSceneCameraCutTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneCameraCutSection::StaticClass();
}

UMovieSceneSection* UMovieSceneCameraCutTrack::CreateNewSection()
{
	return NewObject<UMovieSceneCameraCutSection>(this, NAME_None, RF_Transactional);
}

bool UMovieSceneCameraCutTrack::SupportsMultipleRows() const
{
	return true;
}

EMovieSceneTrackEasingSupportFlags UMovieSceneCameraCutTrack::SupportsEasing(FMovieSceneSupportsEasingParams& Params) const
{
	if (!bCanBlend)
	{
		return EMovieSceneTrackEasingSupportFlags::None;
	}
	if (Params.ForSection != nullptr)
	{
		const int32 NumSections = Sections.Num();
		if (NumSections == 1)
		{
			return EMovieSceneTrackEasingSupportFlags::AutomaticEasing | EMovieSceneTrackEasingSupportFlags::ManualEasing;
		}
		else if (NumSections > 1 && IsAutoManagingSections())
		{
			// Find the section with the earliest start time, and the section with the latest end time.
			int32 EdgeSections[2] = { 0, NumSections - 1 };
			TRange<FFrameNumber> EdgeRanges[2] = { Sections[0]->GetTrueRange(), Sections[NumSections - 1]->GetTrueRange() };
			const TRange<FFrameNumber> ForSectionRange = Params.ForSection->GetTrueRange();
			for (int32 Index = 0; Index < NumSections; ++Index)
			{
				const UMovieSceneSection* Section(Sections[Index]);
				const TRange<FFrameNumber> SectionRange(Section->GetTrueRange());
				if (Section != Params.ForSection && SectionRange.Contains(ForSectionRange))
				{
					return EMovieSceneTrackEasingSupportFlags::None;
				}

				if (EdgeRanges[0].HasLowerBound())
				{
					if (!SectionRange.HasLowerBound() || SectionRange.GetLowerBoundValue() < EdgeRanges[0].GetLowerBoundValue())
					{
						EdgeSections[0] = Index;
						EdgeRanges[0] = SectionRange;
					}
				}
				if (EdgeRanges[1].HasUpperBound())
				{
					if (!SectionRange.HasUpperBound() || SectionRange.GetUpperBoundValue() > EdgeRanges[1].GetUpperBoundValue())
					{
						EdgeSections[1] = Index;
						EdgeRanges[1] = SectionRange;
					}
				}
			}
			
			// Allow easing for these sections.
			EMovieSceneTrackEasingSupportFlags Flags = EMovieSceneTrackEasingSupportFlags::AutomaticEasing;
			if (Params.ForSection == Sections[EdgeSections[0]])
			{
				Flags |= EMovieSceneTrackEasingSupportFlags::ManualEaseIn;
			}
			if (Params.ForSection == Sections[EdgeSections[1]])
			{
				Flags |= EMovieSceneTrackEasingSupportFlags::ManualEaseOut;
			}
			return Flags;
		}
		else if (NumSections > 1 && !IsAutoManagingSections())
		{
			// The given section supports manual easing on one side only if it doesn't overlap with any other section
			// on that side.
			const TRange<FFrameNumber> ForSectionRange = Params.ForSection->GetTrueRange();
			EMovieSceneTrackEasingSupportFlags Flags = EMovieSceneTrackEasingSupportFlags::All;
			for (int32 Index = 0; Index < NumSections; ++Index)
			{
				const UMovieSceneSection* Section(Sections[Index]);
				const TRange<FFrameNumber> SectionRange(UE::MovieScene::MakeHullRange(Section->GetRange()));
				if (Section != Params.ForSection)
				{
					if (ForSectionRange.HasLowerBound() && SectionRange.Contains(ForSectionRange.GetLowerBoundValue()))
					{
						// Lower bound is contained inside another section... disable manual ease-in.
						Flags &= ~EMovieSceneTrackEasingSupportFlags::ManualEaseIn;
					}
					if (ForSectionRange.HasUpperBound() && SectionRange.Contains(ForSectionRange.GetUpperBoundValue()))
					{
						// Upper bound is contained inside another section... disable manual ease-out.
						Flags &= ~EMovieSceneTrackEasingSupportFlags::ManualEaseOut;
					}
				}
			}
			return Flags;
		}
	}
	return EMovieSceneTrackEasingSupportFlags::AutomaticEasing;
}

const TArray<UMovieSceneSection*>& UMovieSceneCameraCutTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneCameraCutTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}


bool UMovieSceneCameraCutTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

void UMovieSceneCameraCutTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
	AutoArrangeSectionsIfNeeded(Section, true);
	MovieSceneHelpers::SortConsecutiveSections(MutableView(Sections));

	// @todo Sequencer: The movie scene owned by the section is now abandoned.  Should we offer to delete it?  
}

void UMovieSceneCameraCutTrack::RemoveSectionAt(int32 SectionIndex)
{
	UMovieSceneSection* SectionToDelete = Sections[SectionIndex];
	Sections.RemoveAt(SectionIndex);
	AutoArrangeSectionsIfNeeded(*SectionToDelete, true);
	MovieSceneHelpers::SortConsecutiveSections(MutableView(Sections));
}

void UMovieSceneCameraCutTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneCameraCutTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Camera Cuts");
}
#endif


#if WITH_EDITOR
EMovieSceneSectionMovedResult UMovieSceneCameraCutTrack::OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params)
{
	const bool bCleanUp = (Params.MoveType == EPropertyChangeType::ValueSet);
	const bool bCleanUpDone = AutoArrangeSectionsIfNeeded(Section, false, bCleanUp);
	return bCleanUpDone ? EMovieSceneSectionMovedResult::SectionsChanged : EMovieSceneSectionMovedResult::None;
}
#endif

FFrameNumber UMovieSceneCameraCutTrack::FindEndTimeForCameraCut( FFrameNumber StartTime )
{
	UMovieScene* OwnerScene = GetTypedOuter<UMovieScene>();

	// End time should default to end where the movie scene ends. Ensure it is at least the same as start time (this should only happen when the movie scene has an initial time range smaller than the start time)
	FFrameNumber ExclusivePlayEnd = UE::MovieScene::DiscreteExclusiveUpper(OwnerScene->GetPlaybackRange());
	FFrameNumber ExclusiveEndTime = FMath::Max( ExclusivePlayEnd, StartTime );

	for( UMovieSceneSection* Section : Sections )
	{
		if( Section->HasStartFrame() && Section->GetInclusiveStartFrame() > StartTime )
		{
			ExclusiveEndTime = Section->GetInclusiveStartFrame();
			break;
		}
	}

	if( StartTime == ExclusiveEndTime )
	{
		// Give the CameraCut a reasonable length of time to start out with.  A 0 time CameraCut is not usable
		ExclusiveEndTime = (StartTime + .5f * OwnerScene->GetTickResolution()).FrameNumber;
	}

	return ExclusiveEndTime;
}

void UMovieSceneCameraCutTrack::PreCompileImpl(FMovieSceneTrackPreCompileResult& OutPreCompileResult)
{
	for (UMovieSceneSection* Section : Sections)
	{
		if (UMovieSceneCameraCutSection* CameraCutSection = CastChecked<UMovieSceneCameraCutSection>(Section))
		{
			CameraCutSection->ComputeInitialCameraCutTransform();
		}
	}
}

bool UMovieSceneCameraCutTrack::IsAutoManagingSections() const
{
	return bAutoArrangeSections && GetMaxRowIndex() == 0;
}

void UMovieSceneCameraCutTrack::SetIsAutoManagingSections(bool bInAutoArrangeSections)
{
	bAutoArrangeSections = bInAutoArrangeSections;
}

bool UMovieSceneCameraCutTrack::AutoArrangeSectionsIfNeeded(UMovieSceneSection& ChangedSection, bool bWasDeletion, bool bCleanUp)
{
	if (IsAutoManagingSections())
	{
		if (bCanBlend)
		{
			return MovieSceneHelpers::FixupConsecutiveBlendingSections(MutableView(Sections), ChangedSection, bWasDeletion, bCleanUp);
		}
		else
		{
			return MovieSceneHelpers::FixupConsecutiveSections(MutableView(Sections), ChangedSection, bWasDeletion, bCleanUp);
		}
	}
	return false;
}

void UMovieSceneCameraCutTrack::RearrangeAllSections()
{
	const UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	if (!ensureMsgf(MovieScene, TEXT("Can't auto-arrange sections on a track that doesn't belong to a MovieScene")))
	{
		return;
	}

	// Sort sections by time.
	MovieSceneHelpers::SortConsecutiveSections(MutableView(Sections));

	// Go over the sections and change anything we don't want to support.
	const bool bRemoveGaps = IsAutoManagingSections();
	const bool bRemoveOverlaps = !bCanBlend;
	const bool bRemoveEasings = !bCanBlend;

	const FFrameRate TickResolution = MovieScene->GetTickResolution();
	const FFrameRate DisplayRate = MovieScene->GetDisplayRate();

	for (int32 Idx = 1; Idx < Sections.Num(); ++Idx)
	{
		UMovieSceneSection* CurSection = Sections[Idx];
		UMovieSceneSection* PrevSection = Sections[Idx - 1];

		TRange<FFrameNumber> CurSectionRange = CurSection->GetRange();
		TRange<FFrameNumber> PrevSectionRange = PrevSection->GetRange();

		// See if these two sections overlap or have a gap between them.
		FFrameNumber OverlapOrGap(0);
		if (PrevSectionRange.HasUpperBound() && CurSectionRange.HasLowerBound())
		{
			// We need to handle the case of CurSection being completely overlapped by PrevSection.
			// In that case, PrevSection's upper bound is way past CurSection's upper bound, and if it's
			// far enough away, the mid-point that we find later (see MeetupFrame) can be past the end
			// of CurSection, resulting in setting an invalid range. So let's clamp this.
			FFrameNumber UpperBound = PrevSectionRange.GetUpperBoundValue();
			if (CurSectionRange.HasUpperBound())
			{
				UpperBound = FMath::Min(UpperBound, CurSectionRange.GetUpperBoundValue());
			}
			OverlapOrGap = (UpperBound - CurSectionRange.GetLowerBoundValue());
		}

		// If there's an overlap and we don't want it, resize the sections so that they start/stop
		// around the middle of the overlap zone (not the exact middle... we snap to the display rate)
		// Similarly, if there's a gap and we don't want it, make the sections meet in the middle of it.
		TOptional<FFrameNumber> MeetupFrame;
		if ((bRemoveOverlaps && OverlapOrGap > 0) || (bRemoveGaps && OverlapOrGap < 0))
		{
			const FFrameTime TimeAtHalf = CurSectionRange.GetLowerBoundValue() + FMath::FloorToInt(OverlapOrGap.Value / 2.f);
			MeetupFrame = FFrameRate::Snap(TimeAtHalf, TickResolution, DisplayRate).CeilToFrame();
		}

		if (MeetupFrame.IsSet())
		{
			PrevSectionRange.SetUpperBoundValue(MeetupFrame.GetValue());
			PrevSection->SetRange(PrevSectionRange);

			CurSectionRange.SetLowerBoundValue(MeetupFrame.GetValue());
			CurSection->SetRange(CurSectionRange);
		}

		// Remove any easings from blending overlaps or manual ease-in/out.
		if (bRemoveEasings)
		{
			CurSection->Easing.AutoEaseInDuration = 0;
			CurSection->Easing.ManualEaseInDuration = 0;

			PrevSection->Easing.AutoEaseOutDuration = 0;
			PrevSection->Easing.ManualEaseOutDuration = 0;
		}

		if (MeetupFrame.IsSet() || bRemoveEasings)
		{
			CurSection->Modify();
		}
	}

	// Remove begin/end easings.
	if (bRemoveEasings && Sections.Num() > 0)
	{
		UMovieSceneSection* FirstSection = Sections[0];

		FirstSection->Easing.AutoEaseInDuration = 0;
		FirstSection->Easing.ManualEaseInDuration = 0;

		FirstSection->Modify();
	}
	if (bRemoveEasings && Sections.Num() > 1)
	{
		UMovieSceneSection* LastSection = Sections.Last();

		LastSection->Easing.AutoEaseOutDuration = 0;
		LastSection->Easing.ManualEaseOutDuration = 0;

		LastSection->Modify();
	}
}

#undef LOCTEXT_NAMESPACE

