// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneCameraShakeSourceShakeTrack.h"
#include "Camera/CameraShakeSourceComponent.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "Sections/MovieSceneCameraShakeSourceShakeSection.h"
#include "MovieScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraShakeSourceShakeTrack)

#define LOCTEXT_NAMESPACE "MovieSceneCameraShakeSourceShakeTrack"

UMovieSceneSection* UMovieSceneCameraShakeSourceShakeTrack::AddNewCameraShake(const FFrameNumber KeyTime, const UCameraShakeSourceComponent& ShakeSourceComponent)
{
	return AddNewCameraShake(KeyTime, ShakeSourceComponent.CameraShake, true);
}

UMovieSceneSection* UMovieSceneCameraShakeSourceShakeTrack::AddNewCameraShake(const FFrameNumber KeyTime, const TSubclassOf<UCameraShakeBase> ShakeClass, bool bIsAutomaticShake)
{
	// It's OK to place an auto-shake on a shake source actor with no default shake set on it... 
	// Designers do that sometimes, using a shake source for a bunch of various shakes, but no
	// particular use for itself.
	check(ShakeClass || bIsAutomaticShake);

	Modify();

	UMovieSceneCameraShakeSourceShakeSection* const NewSection = Cast<UMovieSceneCameraShakeSourceShakeSection>(CreateNewSection());
	if (NewSection)
	{
		FCameraShakeDuration ShakeDuration;
		if (ShakeClass)
		{
			UCameraShakeBase::GetCameraShakeDuration(ShakeClass, ShakeDuration);
		}
		if (!ShakeDuration.IsFixed())
		{
			// Default endless shakes to 5 seconds totally arbitrarily.
			ShakeDuration = FCameraShakeDuration(5.f);
		}

		const FFrameTime ShakeDurationTime = ShakeDuration.Get() * GetTypedOuter<UMovieScene>()->GetTickResolution();
		NewSection->InitialPlacement(CameraShakeSections, KeyTime, ShakeDurationTime.FrameNumber.Value, SupportsMultipleRows());
		if (!bIsAutomaticShake)
		{
			// Only set the shake class if we want to specifically run the provided shake. Otherwise, we leave
			// it null and will automatically fallback to whatever is set on the shake source component.
			NewSection->ShakeData.ShakeClass = ShakeClass;
		}
		
		AddSection(*NewSection);
	}

	return NewSection;
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneCameraShakeSourceShakeTrack::GetDisplayName() const
{
	return LOCTEXT("TrackName", "Camera Shake");
}
#endif

const TArray<UMovieSceneSection*>& UMovieSceneCameraShakeSourceShakeTrack::GetAllSections() const
{
	return CameraShakeSections;
}

bool UMovieSceneCameraShakeSourceShakeTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneCameraShakeSourceShakeSection::StaticClass();
}

UMovieSceneSection* UMovieSceneCameraShakeSourceShakeTrack::CreateNewSection()
{
	return NewObject<UMovieSceneCameraShakeSourceShakeSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneCameraShakeSourceShakeTrack::RemoveAllAnimationData()
{
	CameraShakeSections.Empty();
}

bool UMovieSceneCameraShakeSourceShakeTrack::HasSection(const UMovieSceneSection& Section) const
{
	return CameraShakeSections.Contains(&Section);
}

void UMovieSceneCameraShakeSourceShakeTrack::AddSection(UMovieSceneSection& Section)
{
	CameraShakeSections.Add(&Section);
}

void UMovieSceneCameraShakeSourceShakeTrack::RemoveSection(UMovieSceneSection& Section)
{
	CameraShakeSections.Remove(&Section);
}

void UMovieSceneCameraShakeSourceShakeTrack::RemoveSectionAt(int32 SectionIndex)
{
	CameraShakeSections.RemoveAt(SectionIndex);
}

bool UMovieSceneCameraShakeSourceShakeTrack::IsEmpty() const
{
	return CameraShakeSections.Num() == 0;
}

#undef LOCTEXT_NAMESPACE


