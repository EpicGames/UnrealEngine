// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneCameraShakeSourceTriggerTrack.h"
#include "Sections/MovieSceneCameraShakeSourceTriggerSection.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraShakeSourceTriggerTrack)

#define LOCTEXT_NAMESPACE "MovieSceneCameraShakeSourceTrigger"

UMovieSceneCameraShakeSourceTriggerTrack::UMovieSceneCameraShakeSourceTriggerTrack(const FObjectInitializer& Obj)
	: Super(Obj)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(43, 43, 155, 65);
#endif
}

bool UMovieSceneCameraShakeSourceTriggerTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneCameraShakeSourceTriggerSection::StaticClass();
}

UMovieSceneSection* UMovieSceneCameraShakeSourceTriggerTrack::CreateNewSection()
{
	return NewObject<UMovieSceneCameraShakeSourceTriggerSection>(this, NAME_None, RF_Transactional);
}


bool UMovieSceneCameraShakeSourceTriggerTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.ContainsByPredicate([&](const UMovieSceneSection* In){ return In == &Section; });
}


void UMovieSceneCameraShakeSourceTriggerTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}


void UMovieSceneCameraShakeSourceTriggerTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.RemoveAll([&](const UMovieSceneSection* In) { return In == &Section; });
}

void UMovieSceneCameraShakeSourceTriggerTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

void UMovieSceneCameraShakeSourceTriggerTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}


bool UMovieSceneCameraShakeSourceTriggerTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}


const TArray<UMovieSceneSection*>& UMovieSceneCameraShakeSourceTriggerTrack::GetAllSections() const
{
	return Sections;
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneCameraShakeSourceTriggerTrack::GetDisplayName() const
{
	return LOCTEXT("TrackName", "Camera Shake Trigger");
}

#endif

#undef LOCTEXT_NAMESPACE


