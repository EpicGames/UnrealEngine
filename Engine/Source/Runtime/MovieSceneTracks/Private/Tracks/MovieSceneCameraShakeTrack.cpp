// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneCameraShakeTrack.h"
#include "Sections/MovieSceneCameraShakeSection.h"
#include "Compilation/MovieSceneSegmentCompiler.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "MovieScene.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCameraShakeTrack)

#define LOCTEXT_NAMESPACE "MovieSceneCameraShakeTrack"

UMovieSceneSection* UMovieSceneCameraShakeTrack::AddNewCameraShake(FFrameNumber KeyTime, TSubclassOf<UCameraShakeBase> ShakeClass)
{
	Modify();

	UMovieSceneCameraShakeSection* const NewSection = Cast<UMovieSceneCameraShakeSection>(CreateNewSection());
	if (NewSection)
	{
		// #fixme get length
		FFrameTime Duration = 5.0 * GetTypedOuter<UMovieScene>()->GetTickResolution();
		NewSection->InitialPlacement(CameraShakeSections, KeyTime, Duration.FrameNumber.Value, SupportsMultipleRows());
		NewSection->ShakeData.ShakeClass = ShakeClass;
		
		AddSection(*NewSection);
	}

	return NewSection;
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneCameraShakeTrack::GetDisplayName() const
{
	return LOCTEXT("TrackName", "Camera Shake");
}
#endif


/* UMovieSceneTrack interface
*****************************************************************************/


const TArray<UMovieSceneSection*>& UMovieSceneCameraShakeTrack::GetAllSections() const
{
	return CameraShakeSections;
}

bool UMovieSceneCameraShakeTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneCameraShakeSection::StaticClass();
}


UMovieSceneSection* UMovieSceneCameraShakeTrack::CreateNewSection()
{
	return NewObject<UMovieSceneCameraShakeSection>(this, NAME_None, RF_Transactional);
}


void UMovieSceneCameraShakeTrack::RemoveAllAnimationData()
{
	CameraShakeSections.Empty();
}


bool UMovieSceneCameraShakeTrack::HasSection(const UMovieSceneSection& Section) const
{
	return CameraShakeSections.Contains(&Section);
}


void UMovieSceneCameraShakeTrack::AddSection(UMovieSceneSection& Section)
{
	CameraShakeSections.Add(&Section);
}


void UMovieSceneCameraShakeTrack::RemoveSection(UMovieSceneSection& Section)
{
	CameraShakeSections.Remove(&Section);
}


void UMovieSceneCameraShakeTrack::RemoveSectionAt(int32 SectionIndex)
{
	CameraShakeSections.RemoveAt(SectionIndex);
}


bool UMovieSceneCameraShakeTrack::IsEmpty() const
{
	return CameraShakeSections.Num() == 0;
}


#undef LOCTEXT_NAMESPACE

