// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneCinematicShotTrack.h"
#include "MovieSceneSequence.h"
#include "MovieSceneCommonHelpers.h"
#include "Sections/MovieSceneCinematicShotSection.h"
#include "Compilation/MovieSceneCompilerRules.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCinematicShotTrack)


#define LOCTEXT_NAMESPACE "MovieSceneCinematicShotTrack"


/* UMovieSceneSubTrack interface
 *****************************************************************************/
UMovieSceneCinematicShotTrack::UMovieSceneCinematicShotTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(0, 0, 0, 127);
#endif
}

UMovieSceneSubSection* UMovieSceneCinematicShotTrack::AddSequenceOnRow(UMovieSceneSequence* Sequence, FFrameNumber StartTime, int32 Duration, int32 RowIndex)
{
	UMovieSceneSubSection* NewSection = UMovieSceneSubTrack::AddSequenceOnRow(Sequence, StartTime, Duration, RowIndex);

	UMovieSceneCinematicShotSection* NewShotSection = Cast<UMovieSceneCinematicShotSection>(NewSection);

	// When a new sequence is added, sort all sequences to ensure they are in the correct order
	MovieSceneHelpers::SortConsecutiveSections(MutableView(Sections));

	// Once sequences are sorted fixup the surrounding sequences to fix any gaps
	//MovieSceneHelpers::FixupConsecutiveSections(Sections, *NewSection, false);

	return NewSection;
}

/* UMovieSceneTrack interface
 *****************************************************************************/

void UMovieSceneCinematicShotTrack::AddSection(UMovieSceneSection& Section)
{
	if (Section.IsA<UMovieSceneCinematicShotSection>())
	{
		Sections.Add(&Section);
	}
}

bool UMovieSceneCinematicShotTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneCinematicShotSection::StaticClass();
}


UMovieSceneSection* UMovieSceneCinematicShotTrack::CreateNewSection()
{
	return NewObject<UMovieSceneCinematicShotSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneCinematicShotTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
	//MovieSceneHelpers::FixupConsecutiveSections(Sections, Section, true);
	MovieSceneHelpers::SortConsecutiveSections(MutableView(Sections));

	// @todo Sequencer: The movie scene owned by the section is now abandoned.  Should we offer to delete it?  
}

void UMovieSceneCinematicShotTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
	MovieSceneHelpers::SortConsecutiveSections(MutableView(Sections));
}

bool UMovieSceneCinematicShotTrack::SupportsMultipleRows() const
{
	return true;
}

namespace UE
{
namespace MovieScene
{
	struct FCinematicShotSectionSortData
	{
		int32 Row;
		int32 OverlapPriority;
		int32 SectionIndex;

		friend bool operator<(const FCinematicShotSectionSortData& A, const FCinematicShotSectionSortData& B)
		{
			if (A.Row != B.Row)
			{
				return A.Row < B.Row;
			}

			return A.OverlapPriority > B.OverlapPriority;
		}
	};
}
}

bool UMovieSceneCinematicShotTrack::PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const
{
	using namespace UE::MovieScene;

	TArray<FCinematicShotSectionSortData, TInlineAllocator<16>> SortedSections;

	for (int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
	{
		UMovieSceneSection* Section = Sections[SectionIndex];

		if (Section && Section->IsActive() && !IsRowEvalDisabled(Section->GetRowIndex()))
		{
			const TRange<FFrameNumber> SectionRange = Section->GetRange();
			if (!SectionRange.IsEmpty())
			{
				FCinematicShotSectionSortData SectionData{ Section->GetRowIndex(), Section->GetOverlapPriority(), SectionIndex };
				SortedSections.Emplace(SectionData);
			}
		}
	}

	SortedSections.Sort();

	for (const FCinematicShotSectionSortData& SectionData : SortedSections)
	{
		UMovieSceneSection* Section = Sections[SectionData.SectionIndex];
		OutData.AddIfEmpty(Section->GetRange(), FMovieSceneTrackEvaluationData::FromSection(Section));
	}

	return true;
}

int8 UMovieSceneCinematicShotTrack::GetEvaluationFieldVersion() const
{
	return 2;
}

#if WITH_EDITOR
EMovieSceneSectionMovedResult UMovieSceneCinematicShotTrack::OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params)
{
	//MovieSceneHelpers::FixupConsecutiveSections(Sections, Section, false);
	return EMovieSceneSectionMovedResult::None;
}
#endif

#if WITH_EDITORONLY_DATA
FText UMovieSceneCinematicShotTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Shots");
}
#endif

void UMovieSceneCinematicShotTrack::SortSections()
{
	MovieSceneHelpers::SortConsecutiveSections(MutableView(Sections));
}

#undef LOCTEXT_NAMESPACE

