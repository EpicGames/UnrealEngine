// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneColorTrack.h"
#include "Sections/MovieSceneColorSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneColorTrack)

UMovieSceneColorTrack::UMovieSceneColorTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieSceneColorTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneColorSection::StaticClass();
}

UMovieSceneSection* UMovieSceneColorTrack::CreateNewSection()
{
	return NewObject<UMovieSceneColorSection>(this, NAME_None, RF_Transactional);
}

