// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneCommonAnimationTrack.h"

#include "Animation/AnimationPoseData.h"
#include "MovieSceneAnimationRotationUtils.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimSequence.h"
#include "Animation/AttributesRuntime.h"
#include "AnimationRuntime.h"
#include "AnimSequencerInstanceProxy.h"
#include "BoneContainer.h"
#include "Components/SkeletalMeshComponent.h"
#include "MovieScene.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "SkeletalDebugRendering.h"

#if WITH_EDITORONLY_DATA
#include "AnimationBlueprintLibrary.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/Timecode.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCommonAnimationTrack)


UMovieSceneCommonAnimationTrack::UMovieSceneCommonAnimationTrack(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
	, bBlendFirstChildOfRoot(false)
{
#if WITH_EDITORONLY_DATA
	bShowRootMotionTrail = false;
#endif
}

UMovieSceneSection* UMovieSceneCommonAnimationTrack::CreateNewSection()
{
	return NewObject<UMovieSceneSkeletalAnimationSection>(this, NAME_None, RF_Transactional);
}

bool UMovieSceneCommonAnimationTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneSkeletalAnimationSection::StaticClass();
}

const TArray<UMovieSceneSection*>& UMovieSceneCommonAnimationTrack::GetAllSections() const
{
	return AnimationSections;
}

bool UMovieSceneCommonAnimationTrack::SupportsMultipleRows() const
{
	return true;
}

void UMovieSceneCommonAnimationTrack::RemoveAllAnimationData()
{
	AnimationSections.Empty();

}

bool UMovieSceneCommonAnimationTrack::HasSection(const UMovieSceneSection& Section) const
{
	return AnimationSections.Contains(&Section);
}

void UMovieSceneCommonAnimationTrack::AddSection(UMovieSceneSection& Section)
{
	Modify();
	AnimationSections.Add(&Section);
	SetUpRootMotions(true);
}

void UMovieSceneCommonAnimationTrack::RemoveSection(UMovieSceneSection& Section)
{
	Modify();
	AnimationSections.Remove(&Section);
	SetUpRootMotions(true);
}

void UMovieSceneCommonAnimationTrack::RemoveSectionAt(int32 SectionIndex)
{
	Modify();
	AnimationSections.RemoveAt(SectionIndex);
	SetUpRootMotions(true);
}

void UMovieSceneCommonAnimationTrack::UpdateEasing()
{
	Super::UpdateEasing();
	SetRootMotionsDirty();
}

bool UMovieSceneCommonAnimationTrack::IsEmpty() const
{
	return AnimationSections.Num() == 0;
}

UMovieSceneSection* UMovieSceneCommonAnimationTrack::AddNewAnimationOnRow(FFrameNumber KeyTime, UAnimSequenceBase* AnimSequence, int32 RowIndex)
{
	// If RowIndex is INDEX_NONE, compute the next available row
	if (RowIndex == INDEX_NONE)
	{
		RowIndex = ComputeNextAvailableRowIndex();
	}

	UMovieSceneSkeletalAnimationSection* NewSection = Cast<UMovieSceneSkeletalAnimationSection>(CreateNewSection());
	{
		FFrameTime AnimationLength = AnimSequence->GetPlayLength() * GetTypedOuter<UMovieScene>()->GetTickResolution();
		int32 IFrameNumber = AnimationLength.FrameNumber.Value + (int)(AnimationLength.GetSubFrame() + 0.5f) + 1;
		NewSection->InitialPlacementOnRow(AnimationSections, KeyTime, IFrameNumber, RowIndex);
		NewSection->Params.Animation = AnimSequence;

#if WITH_EDITORONLY_DATA
		FQualifiedFrameTime SourceStartFrameTime;
		if (UAnimationBlueprintLibrary::EvaluateRootBoneTimecodeAttributesAtTime(NewSection->Params.Animation, 0.0f, SourceStartFrameTime))
		{
			NewSection->TimecodeSource.Timecode = SourceStartFrameTime.ToTimecode();
		}
#endif
	}

	Super::AddSection(*NewSection);

	return NewSection;
}

void UMovieSceneCommonAnimationTrack::AddSectionWithAutoRow(UMovieSceneSection* Section)
{
	if (!Section)
	{
		return;
	}

	// Base implementation: just add the section
	// The section's row index should already be set, or will use default behavior
	AddSection(*Section);
}

int32 UMovieSceneCommonAnimationTrack::ComputeNextAvailableRowIndex() const
{
	// Find max row from existing sections
	int32 MaxRowIndex = -1;
	for (UMovieSceneSection* ExistingSection : AnimationSections)
	{
		MaxRowIndex = FMath::Max(MaxRowIndex, ExistingSection->GetRowIndex());
	}
	return MaxRowIndex + 1;
}

#if WITH_EDITOR
void UMovieSceneCommonAnimationTrack::PostEditImport()
{
	Super::PostEditImport();
	RootMotionParams.bRootMotionsDirty = true;
}

void UMovieSceneCommonAnimationTrack::PostEditUndo()
{
	Super::PostEditUndo();
	RootMotionParams.bRootMotionsDirty = true;
}

void UMovieSceneCommonAnimationTrack::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);
	RootMotionParams.bRootMotionsDirty = true;
}
EMovieSceneSectionMovedResult UMovieSceneCommonAnimationTrack::OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params)
{
	if (Params.MoveType == EPropertyChangeType::ValueSet)
	{
		RootMotionParams.bRootMotionsDirty = true;
	}
	
	return EMovieSceneSectionMovedResult::None;
}
#endif // WITH_EDITOR

void UMovieSceneCommonAnimationTrack::SortSections()
{
	AnimationSections.Sort(
		[](const UMovieSceneSection& A, const UMovieSceneSection& B)
		{
			TRange<FFrameNumber> RangeA = A.GetTrueRange();
			TRange<FFrameNumber> RangeB = B.GetTrueRange();
			if (RangeA.GetLowerBound().IsOpen())
			{
				return true;
			}
			else if (RangeB.GetLowerBound().IsOpen())
			{
				return false;
			}
			return (RangeA.GetLowerBoundValue() < RangeB.GetLowerBoundValue());
		}
	);
}

//expectation is the weights may be unnormalized.
static void BlendTheseTransformsByWeight(FTransform& OutTransform, const TArray<FTransform>& Transforms,  TArray<float>& Weights)
{
	if (Weights.Num() > 0)
	{
		float TotalWeight = 0.0f;
		for (int32 WeightIndex = 0; WeightIndex < Weights.Num(); ++WeightIndex)
		{
			TotalWeight += Weights[WeightIndex];
		}
		if (!FMath::IsNearlyEqual(TotalWeight, 1.0f))
		{
			for (int32 DivideIndex = 0; DivideIndex < Weights.Num(); ++DivideIndex)
			{
				Weights[DivideIndex] /= TotalWeight;
			}
		}
	}
	int32 NumBlends = Transforms.Num();
	check(Transforms.Num() == Weights.Num());
	if (NumBlends == 0)
	{
		OutTransform = FTransform::Identity;
	}
	else if (NumBlends == 1)
	{
		OutTransform = Transforms[0];
	}
	else
	{
		FVector		OutTranslation(0.0f, 0.0f, 0.0f);
		FVector		OutScale(0.0f, 0.0f, 0.0f);

		//rotation will get set to the first weighted and then made closest to that so linear interp works.
		FQuat FirstRot = Transforms[0].GetRotation();
		FQuat		OutRotation(FirstRot.X * Weights[0], FirstRot.Y * Weights[0], FirstRot.Z * Weights[0], FirstRot.W * Weights[0]);

		for (int32 Index = 0; Index < NumBlends; ++Index)
		{
			OutTranslation += Transforms[Index].GetTranslation() * Weights[Index];
			OutScale +=  Transforms[Index].GetScale3D() * Weights[Index];
			if (Index != 0)
			{
				FQuat Quat = Transforms[Index].GetRotation();
				Quat.EnforceShortestArcWith(FirstRot);
				Quat *= Weights[Index];
				OutRotation += Quat;
			}
		}

		OutRotation.Normalize();
		OutTransform = FTransform(OutRotation, OutTranslation, OutScale);
	}
}

void UMovieSceneCommonAnimationTrack::SetRootMotionsDirty()
{
	RootMotionParams.bRootMotionsDirty = true;
}

struct FSkelBoneLength
{
	FSkelBoneLength(FCompactPoseBoneIndex InPoseIndex, float InBL) :PoseBoneIndex(InPoseIndex), BoneLength(InBL) {};
	FCompactPoseBoneIndex PoseBoneIndex;
	float BoneLength; //squared
};

static void CalculateDistanceMap(USkeletalMeshComponent* SkelMeshComp, UAnimSequenceBase* FirstAnimSeq, UAnimSequenceBase* SecondAnimSeq, float StartFirstAnimTime, float FrameRate,
	TArray<TArray<float>>& OutDistanceDifferences)
{

	int32 FirstAnimNumFrames = (FirstAnimSeq->GetPlayLength() - StartFirstAnimTime) * FrameRate + 1;
	int32 SecondAnimNumFrames = SecondAnimSeq->GetPlayLength() * FrameRate + 1;
	OutDistanceDifferences.SetNum(FirstAnimNumFrames);
	float FirstAnimIndex = 0.0f;
	float FrameRateDiff = 1.0f / FrameRate;
	FCompactPose FirstAnimPose, SecondAnimPose;
	FCSPose<FCompactPose> FirstMeshPoses, SecondMeshPoses;
	FirstAnimPose.ResetToRefPose(SkelMeshComp->GetAnimInstance()->GetRequiredBones());
	SecondAnimPose.ResetToRefPose(SkelMeshComp->GetAnimInstance()->GetRequiredBones());

	FBlendedCurve FirstOutCurve, SecondOutCurve;
	UE::Anim::FStackAttributeContainer FirstTempAttributes, SecondTempAttributes;
	FAnimationPoseData FirstAnimPoseData(FirstAnimPose, FirstOutCurve, FirstTempAttributes);
	FAnimationPoseData SecondAnimPoseData(SecondAnimPose, SecondOutCurve, SecondTempAttributes);

	//sort by bone lengths just do the first half
	//this should avoid us overvalueing to many small values.
	/*
	TArray<FSkelBoneLength> BoneLengths;
	BoneLengths.SetNum(FirstAnimPose.GetNumBones());
	int32 Index = 0;
	for (FCompactPoseBoneIndex PoseBoneIndex : FirstAnimPose.ForEachBoneIndex())
	{
		FTransform LocalTransform = FirstMeshPoses.GetLocalSpaceTransform(PoseBoneIndex);
		float BoneLengthVal = LocalTransform.GetLocation().SizeSquared();
		BoneLengths[Index++] = FSkelBoneLength(PoseBoneIndex, BoneLengthVal);
	}
	BoneLengths.Sort([](const FSkelBoneLength& Item1, const FSkelBoneLength& Item2) {
		return Item1.BoneLength > Item2.BoneLength;
		});
		*/
	FBlendedCurve OutCurve;
	const FBoneContainer& RequiredBones = FirstAnimPoseData.GetPose().GetBoneContainer();
	for (TArray<float>& FloatArray : OutDistanceDifferences)
	{
		FloatArray.SetNum(SecondAnimNumFrames);
		float FirstAnimTime = FirstAnimIndex * FrameRateDiff + StartFirstAnimTime;
		FirstAnimIndex += 1.0f;
		FAnimExtractContext FirstExtractionContext(static_cast<double>(FirstAnimTime), false);
		FirstAnimSeq->GetAnimationPose(FirstAnimPoseData, FirstExtractionContext);
		FirstMeshPoses.InitPose(FirstAnimPoseData.GetPose());
		float SecondAnimIndex = 0.0f;
		for (float& DistVal : FloatArray)
		{
			DistVal = 0.0f;
			float SecondAnimTime = SecondAnimIndex * FrameRateDiff;
			SecondAnimIndex += 1.0f;
			FAnimExtractContext SecondExtractionContext(static_cast<double>(SecondAnimTime), false);
			SecondAnimSeq->GetAnimationPose(SecondAnimPoseData, SecondExtractionContext);
			SecondMeshPoses.InitPose(SecondAnimPoseData.GetPose());

			float DiffVal = 0.0f;
			for (FCompactPoseBoneIndex PoseBoneIndex : FirstAnimPoseData.GetPose().ForEachBoneIndex())
			{
				FTransform FirstTransform = FirstMeshPoses.GetComponentSpaceTransform(PoseBoneIndex);
				FTransform SecondTransform = SecondMeshPoses.GetComponentSpaceTransform(PoseBoneIndex);
				if (PoseBoneIndex != 0)
				{
					DistVal += (FirstTransform.GetTranslation() - SecondTransform.GetTranslation()).SizeSquared();
				}
			}
		}
	}
}
//outer is startanimtime to firstanim->seqlength...
//inner is 0 to secondanim->seqlength...
//for this function just find the smallest in the second...
//return the end anim time
static float GetBestBlendPointTimeAtStart(UAnimSequenceBase* FirstAnimSeq, UAnimSequenceBase* SecondAnimSeq, float StartFirstAnimTime, float FrameRate,
	TArray<TArray<float>>& DistanceDifferences)
{

	//int32 FirstAnimNumFrames = (FirstAnimSeq->SequenceLength - StartFirstAnimTime) * FrameRate + 1;
	int32 SecondAnimNumFrames = SecondAnimSeq->GetPlayLength() * FrameRate + 1;
	if (SecondAnimNumFrames <= 0)
	{
		return 0.0f;
	}
	TArray<float>& Distances = DistanceDifferences[0];
	float MinVal = Distances[0];
	int32 SmallIndex = 0;
	for (int32 Index = 1; Index < SecondAnimNumFrames; ++Index)
	{
		float NewMin = Distances[Index];
		if (NewMin < MinVal)
		{
			MinVal = NewMin;
			SmallIndex = Index;
		}
	}
	return SmallIndex * 1.0f / FrameRate;
}

TOptional<FTransform>  UMovieSceneCommonAnimationTrack::GetRootMotion(FFrameTime CurrentTime)
{
	TOptional<FTransform> Transform;

	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return Transform;
	}
	FFrameRate TickResolution = MovieScene->GetTickResolution();
	
	if (RootMotionParams.bRootMotionsDirty) 
	{
		SetUpRootMotions(true);
	}
	if (RootMotionParams.bHaveRootMotion == false)
	{
		return Transform;
	}
	SortSections();
	TArray<FTransform> CurrentTransforms;
	TArray<float> CurrentWeights;
	TArray<FTransform> CurrentAdditiveTransforms;
	TArray<float> CurrentAdditiveWeights;
	FTransform CurrentTransform = FTransform::Identity;

	// Use evaluation field to iterate, as it will take into account 'Evaluate Nearest Section'.
	for (const FMovieSceneTrackEvaluationFieldEntry& FieldEntry : GetEvaluationField().Entries)
	{
		if (FieldEntry.Range.Contains(CurrentTime.FrameNumber))
		{
			if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(FieldEntry.Section))
			{
				FFrameNumber EvaluationFrame = FieldEntry.ForcedTime == TNumericLimits<int32>::Lowest() ? CurrentTime.FrameNumber : FieldEntry.ForcedTime;
				if (AnimSection->Params.Animation)
				{
					UAnimSequenceBase* ValidAnimSequence = AnimSection->Params.Animation;
					FMemMark Mark(FMemStack::Get());
					FCompactPose OutPose;
					TArray<FBoneIndexType> RequiredBoneIndexArray;
					RequiredBoneIndexArray.AddUninitialized(ValidAnimSequence->GetSkeleton()->GetReferenceSkeleton().GetNum());
					for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
					{
						RequiredBoneIndexArray[BoneIndex] = BoneIndex;
					}

					FBoneContainer BoneContainer(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::None), *ValidAnimSequence->GetSkeleton());
					OutPose.ResetToRefPose(BoneContainer);
					FBlendedCurve OutCurve;
					OutCurve.InitFrom(BoneContainer);
					UE::Anim::FStackAttributeContainer TempAttributes;
					FAnimationPoseData AnimationPoseData(OutPose, OutCurve, TempAttributes);
					UMovieSceneSkeletalAnimationSection::FRootMotionTransformParam Param;
					Param.FrameRate = TickResolution;
					Param.CurrentTime = EvaluationFrame;

					if (AnimSection->GetRootMotionTransform(AnimationPoseData, Param))
					{
						if (!Param.bOutIsAdditive)
						{
							CurrentTransform = Param.OutTransform * AnimSection->PreviousTransform;
							CurrentTransforms.Add(CurrentTransform);
							CurrentWeights.Add(Param.OutWeight);
						}
						else
						{
							CurrentAdditiveTransforms.Add(Param.OutTransform);
							CurrentAdditiveWeights.Add(Param.OutWeight);
						}
					}
				}
			}
		}
	}

	BlendTheseTransformsByWeight(CurrentTransform, CurrentTransforms, CurrentWeights);
	//now handle additive onto the current
	if (CurrentAdditiveWeights.Num() > 0)
	{
		FTransform AdditiveTransform;
		BlendTheseTransformsByWeight(AdditiveTransform, CurrentAdditiveTransforms, CurrentAdditiveWeights);
		const ScalarRegister VBlendWeight(1.0f);
		FTransform::BlendFromIdentityAndAccumulate(CurrentTransform, AdditiveTransform, VBlendWeight);
	}
	Transform = CurrentTransform;
	return Transform;

}

void UMovieSceneCommonAnimationTrack::SetUpRootMotions(bool bForce)
{
	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	if (!MovieScene)
	{
		return;
	}
	
	if (bForce || RootMotionParams.bRootMotionsDirty)
	{
		RootMotionParams.bRootMotionsDirty = false;
		RootMotionParams.bHaveRootMotion = false;
		const FFrameRate MinDisplayRate(60, 1);

		FFrameRate DisplayRate =  MovieScene->GetDisplayRate().AsDecimal() < MinDisplayRate.AsDecimal() ? MinDisplayRate : MovieScene->GetDisplayRate();
		FFrameRate TickResolution = MovieScene->GetTickResolution();
		FFrameTime FrameTick = FFrameTime(FMath::Max(1, TickResolution.AsFrameNumber(1.0).Value / DisplayRate.AsFrameNumber(1.0).Value));
		if (AnimationSections.Num() == 0 || FrameTick.FrameNumber.Value == 0)
		{
#if WITH_EDITORONLY_DATA
			RootMotionParams.RootTransforms.SetNum(0);
#endif
			return;
		}
		
		SortSections();
		//Set the TempOffset.
		FTransform InitialTransform = FTransform::Identity;
		UMovieSceneSkeletalAnimationSection* PrevAnimSection = nullptr;
		//valid anim sequence to use to calculate bones.
		UAnimSequenceBase* ValidAnimSequence = nullptr;
		//if no transforms have offsets then don't do root motion caching.
		bool bAnySectionsHaveOffset = false;
		TArray<UMovieSceneSkeletalAnimationSection*> AnimSections;
		for (UMovieSceneSection* Section : AnimationSections)
		{
			UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section);
			if (AnimSection)
			{
				const bool bCurrentSectionIsMatched = !AnimSection->MatchedBoneName.IsNone();

				AnimSections.Add(AnimSection);
				if (AnimSection->StartLocationOffset.IsNearlyZero() == false || AnimSection->StartRotationOffset.IsNearlyZero() == false ||
					AnimSection->MatchedLocationOffset.IsNearlyZero() == false || AnimSection->MatchedRotationOffset.IsNearlyZero() == false)
				{
					bAnySectionsHaveOffset = true;
				}
				if (ValidAnimSequence == nullptr)
				{
					ValidAnimSequence = AnimSection->Params.Animation;
				}
				// Only propagate offsets as long as root motion offsets are matched. Non-matched offsets should operate directly on the transform values.
				if (PrevAnimSection && bCurrentSectionIsMatched)
				{
					if (UAnimSequenceBase* PrevAnimSequence = PrevAnimSection->Params.Animation)
					{	
						if (bAnySectionsHaveOffset)
						{ 
							RootMotionParams.RootMotionStartOffset = AnimSection->GetRootMotionStartOffset();
							FMemMark Mark(FMemStack::Get());
							FCompactPose OutPose;
							TArray<FBoneIndexType> RequiredBoneIndexArray;
							RequiredBoneIndexArray.AddUninitialized(PrevAnimSequence->GetSkeleton()->GetReferenceSkeleton().GetNum());
							for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
							{
								RequiredBoneIndexArray[BoneIndex] = BoneIndex;
							}

							FBoneContainer BoneContainer(RequiredBoneIndexArray, UE::Anim::FCurveFilterSettings(UE::Anim::ECurveFilterMode::None), *PrevAnimSequence->GetSkeleton());
							OutPose.ResetToRefPose(BoneContainer);
							FBlendedCurve OutCurve;
							OutCurve.InitFrom(BoneContainer);

							UE::Anim::FStackAttributeContainer TempAttributes;
							FAnimationPoseData AnimationPoseData(OutPose, OutCurve, TempAttributes);

							UMovieSceneSkeletalAnimationSection::FRootMotionTransformParam Param;
							Param.FrameRate = MovieScene->GetTickResolution();
							Param.CurrentTime = AnimSection->GetRange().GetLowerBoundValue();

							if (PrevAnimSection->GetRootMotionTransform(AnimationPoseData, Param))
							{
								AnimSection->PreviousTransform = Param.OutPoseTransform.GetRelativeTransformReverse(Param.OutTransform);
								AnimSection->PreviousTransform = AnimSection->PreviousTransform * InitialTransform;

							}
							else
							{
								AnimSection->PreviousTransform = FTransform::Identity;
							}
						}
						else
						{
							AnimSection->PreviousTransform = FTransform::Identity;
						}
					}
					else
					{
						AnimSection->PreviousTransform = FTransform::Identity;
					}
					InitialTransform = AnimSection->PreviousTransform;

				}
				else
				{
					AnimSection->PreviousTransform = FTransform::Identity;
				}
				PrevAnimSection = AnimSection;
				AnimSection->SetBoneIndexForRootMotionCalculations(bBlendFirstChildOfRoot);
			}
		}

		if (AnimSections.Num() == 0)
		{
#if WITH_EDITORONLY_DATA
			RootMotionParams.RootTransforms.SetNum(0);
#endif			
			return;
		}

		if (bAnySectionsHaveOffset == false && !ShouldUseRootMotions())
		{
#if WITH_EDITORONLY_DATA
			RootMotionParams.RootTransforms.SetNum(0);
#endif			
			return;
		}
		RootMotionParams.bHaveRootMotion = true;
		RootMotionParams.StartFrame = AnimSections[0]->GetInclusiveStartFrame();
		RootMotionParams.EndFrame = AnimSections.Last()->GetExclusiveEndFrame() - 1;
		RootMotionParams.FrameTick = FrameTick;
		
#if WITH_EDITORONLY_DATA
		if (RootMotionParams.bCacheRootTransforms == false)
		{
			return;
		}
		//set up pose from valid anim sequences.
		FMemMark Mark(FMemStack::Get());
		FCompactPose OutPose;

		if (ValidAnimSequence)
		{
			TArray<FBoneIndexType> RequiredBoneIndexArray;
			const UE::Anim::FCurveFilterSettings CurveFilterSettings;
			RequiredBoneIndexArray.AddUninitialized(ValidAnimSequence->GetSkeleton()->GetReferenceSkeleton().GetNum());
			for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
			{
				RequiredBoneIndexArray[BoneIndex] = BoneIndex;
			}

			FBoneContainer BoneContainer(RequiredBoneIndexArray, CurveFilterSettings, *ValidAnimSequence->GetSkeleton());
			OutPose.ResetToRefPose(BoneContainer);
			FBlendedCurve OutCurve;
			OutCurve.InitFrom(BoneContainer);
			TArray< UMovieSceneSkeletalAnimationSection*> SectionsAtCurrentTime;
			int32 NumTotal = (RootMotionParams.EndFrame.FrameNumber.Value - RootMotionParams.StartFrame.FrameNumber.Value) / (RootMotionParams.FrameTick.FrameNumber.Value) + 1;
			RootMotionParams.RootTransforms.SetNum(NumTotal);

			TArray<FTransform> CurrentTransforms;
			TArray<float> CurrentWeights;
			TArray<FTransform> CurrentAdditiveTransforms;
			TArray<float> CurrentAdditiveWeights;
			FFrameTime PreviousFrame = RootMotionParams.StartFrame;
			int32 Index = 0;
			for (FFrameTime FrameNumber = RootMotionParams.StartFrame; FrameNumber <= RootMotionParams.EndFrame; FrameNumber += RootMotionParams.FrameTick)
			{
				CurrentTransforms.SetNum(0);
				CurrentWeights.SetNum(0);
				FTransform CurrentTransform(FTransform::Identity), ParentTransform(FTransform::Identity);

				UMovieSceneSkeletalAnimationSection* PrevSection = nullptr;
				for (UMovieSceneSection* Section : AnimationSections)
				{
					UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section);
					if (AnimSection && AnimSection->GetRange().Contains(FrameNumber.FrameNumber))
					{
						UE::Anim::FStackAttributeContainer TempAttributes;
						FAnimationPoseData AnimationPoseData(OutPose, OutCurve, TempAttributes);
						UMovieSceneSkeletalAnimationSection::FRootMotionTransformParam Param;
						Param.FrameRate = TickResolution;
						Param.CurrentTime = FrameNumber.FrameNumber;

						if (AnimSection->GetRootMotionTransform(AnimationPoseData,Param))
						{
							if (!Param.bOutIsAdditive)
							{
								CurrentTransform = Param.OutTransform * AnimSection->PreviousTransform;
								CurrentTransforms.Add(CurrentTransform);
								CurrentWeights.Add(Param.OutWeight);
							}
							else
							{
								CurrentAdditiveTransforms.Add(Param.OutTransform);
								CurrentAdditiveWeights.Add(Param.OutWeight);
							}
						}
						PrevSection = AnimSection;
					}
				}

				BlendTheseTransformsByWeight(CurrentTransform, CurrentTransforms, CurrentWeights);
				//now handle additive onto the current
				if (CurrentAdditiveWeights.Num() > 0)
				{
					FTransform AdditiveTransform;
					BlendTheseTransformsByWeight(AdditiveTransform, CurrentAdditiveTransforms, CurrentAdditiveWeights);
					const ScalarRegister VBlendWeight(1.0f);
					FTransform::BlendFromIdentityAndAccumulate(CurrentTransform, AdditiveTransform, VBlendWeight);
				}
				RootMotionParams.RootTransforms[Index] = CurrentTransform;
				++Index;
				PreviousFrame = FrameNumber;

			}
		}
		else //no valid anim sequence just clear out
		{
			RootMotionParams.RootTransforms.SetNum(0);
		}
#endif
	}

}

static FTransform GetTransformForBoneRelativeToIndex(UAnimSequence* AnimSequence, USkeletalMeshComponent* MeshComponent, const FName& InBoneName,
	const FCompactPoseBoneIndex& ParentCPIndex, double Seconds)
{
	FTransform  WorldTransform = FTransform::Identity;
	//AnimSequence->GetBoneTransform doesn't seem to be as accurate as GetAnimationPose
	FMemMark Mark(FMemStack::Get());
	FCompactPose OutPose;
	TArray<FBoneIndexType> RequiredBoneIndexArray;
	const UE::Anim::FCurveFilterSettings CurveFilterSettings;
	RequiredBoneIndexArray.AddUninitialized(AnimSequence->GetSkeleton()->GetReferenceSkeleton().GetNum());
	for (int32 BoneIndex = 0; BoneIndex < RequiredBoneIndexArray.Num(); ++BoneIndex)
	{
		RequiredBoneIndexArray[BoneIndex] = BoneIndex;
	}

	FBoneContainer BoneContainer(RequiredBoneIndexArray, CurveFilterSettings, *AnimSequence->GetSkeleton());
	OutPose.ResetToRefPose(BoneContainer);
	FBlendedCurve OutCurve;
	OutCurve.InitFrom(BoneContainer);
	UE::Anim::FStackAttributeContainer TempAttributes;
	FAnimationPoseData AnimationPoseData(OutPose, OutCurve, TempAttributes);
	FAnimExtractContext ExtractionContext(Seconds, false);
#if WITH_EDITOR
	ExtractionContext.bIgnoreRootLock = true;
#endif
	AnimSequence->GetAnimationPose(AnimationPoseData, ExtractionContext);
	int32 MeshIndex = AnimationPoseData.GetPose().GetBoneContainer().GetPoseBoneIndexForBoneName(InBoneName);
	if (MeshIndex != INDEX_NONE)
	{
		FCompactPoseBoneIndex CPIndex = AnimationPoseData.GetPose().GetBoneContainer().MakeCompactPoseIndex(FMeshPoseBoneIndex(MeshIndex));
		if (CPIndex != INDEX_NONE )
		{
			FTransform BoneTransform = FTransform::Identity;
			WorldTransform *= BoneTransform;
			do
			{
				BoneTransform = AnimationPoseData.GetPose()[CPIndex];
				WorldTransform *= BoneTransform;
				if (CPIndex == ParentCPIndex)  //if we are the parent then we stop
				{
					CPIndex = FCompactPoseBoneIndex(INDEX_NONE);
				}
				else
				{
					CPIndex = AnimationPoseData.GetPose().GetBoneContainer().GetParentBoneIndex(CPIndex);
				}
			} while (CPIndex.IsValid());
		}
	}
	return WorldTransform;
}

using ERotationOrder = UE::MovieScene::ERotationOrder;
using UE::MovieScene::QuatFromEuler;
using UE::MovieScene::EulerFromQuat;
using UE::MovieScene::FindBestRotationOrder;

void UMovieSceneCommonAnimationTrack::MatchSectionByBoneTransform(bool bMatchWithPrevious, USkeletalMeshComponent* SkelMeshComp, UMovieSceneSkeletalAnimationSection* CurrentSection, FFrameTime CurrentFrame, FFrameRate InTickResolution,
	const FName& BoneName, FTransform& SecondSectionRootDiff, FVector& TranslationDiff, FQuat& RotationDiff) //add options for z and for rotation.
{
	SortSections();
	UMovieSceneSection* PrevSection = nullptr;
	UMovieSceneSection* NextSection = nullptr;
	for (int32 Index = 0; Index < AnimationSections.Num(); ++Index)
	{
		UMovieSceneSection* Section = AnimationSections[Index];
		if (Section == CurrentSection)
		{
			if (++Index < AnimationSections.Num())
			{
				NextSection = AnimationSections[Index];
			}
			break;
		}
		PrevSection = Section;
	}

	TranslationDiff = FVector(0.0f, 0.0f, 0.0f);
	RotationDiff = FQuat::Identity;
	SecondSectionRootDiff = FTransform::Identity;

	if (bMatchWithPrevious && PrevSection)
	{
		UMovieSceneSkeletalAnimationSection* FirstSection = Cast<UMovieSceneSkeletalAnimationSection>(PrevSection);
		UAnimSequence* FirstAnimSequence = Cast<UAnimSequence>(FirstSection->Params.Animation);
		UAnimSequence* SecondAnimSequence = Cast<UAnimSequence>(CurrentSection->Params.Animation);

		if (FirstAnimSequence && SecondAnimSequence)
		{
			double FirstSectionTime = FirstSection->MapTimeToAnimation(CurrentFrame, InTickResolution);
			//use same index for all
			int32 Index = CurrentSection->SetBoneIndexForRootMotionCalculations(bBlendFirstChildOfRoot);
			FCompactPoseBoneIndex ParentIndex(Index);
			
			FTransform  FirstTransform = GetTransformForBoneRelativeToIndex(FirstAnimSequence, SkelMeshComp, BoneName, ParentIndex, FirstSectionTime);
			double SecondSectionTime = CurrentSection->MapTimeToAnimation(CurrentFrame, InTickResolution);
			FTransform  SecondTransform = GetTransformForBoneRelativeToIndex(SecondAnimSequence, SkelMeshComp, BoneName, ParentIndex,SecondSectionTime);
			//Need to match the translations and rotations here 
			//First need to get the correct rotation order based upon what's matching, otherwise if not all are matched 
			//and one rotation is set last we will get errors.
			ERotationOrder RotationOrder = FindBestRotationOrder(CurrentSection->bMatchRotationRoll, CurrentSection->bMatchRotationPitch, CurrentSection->bMatchRotationYaw);

			FVector FirstTransformTranslation = FirstTransform.GetTranslation();
			FVector SecondTransformTranslation = SecondTransform.GetTranslation();
			FQuat FirstTransformQuat = FirstTransform.GetRotation();
			FQuat SecondTransformQuat = SecondTransform.GetRotation();

			FirstTransformQuat.EnforceShortestArcWith(SecondTransformQuat);

			FRotator FirstTransformRotation(FRotator::MakeFromEuler(EulerFromQuat(FirstTransformQuat, RotationOrder)));
			FRotator SecondTransformRotation(FRotator::MakeFromEuler(EulerFromQuat(SecondTransformQuat, RotationOrder)));
			SecondTransformRotation.SetClosestToMe(FirstTransformRotation);

			if (!CurrentSection->bMatchTranslation)
			{
				FirstTransformTranslation.X = SecondTransformTranslation.X;
				FirstTransformTranslation.Y = SecondTransformTranslation.Y;
				FirstTransformTranslation.Z = SecondTransformTranslation.Z;
			}
			if (!CurrentSection->bMatchIncludeZHeight)
			{
				FirstTransformTranslation.Z = SecondTransformTranslation.Z;
			}
			FirstTransform.SetTranslation(FirstTransformTranslation);

			if (!CurrentSection->bMatchRotationYaw)
			{
				FirstTransformRotation.Yaw = SecondTransformRotation.Yaw;
			}
			if (!CurrentSection->bMatchRotationPitch)
			{
				FirstTransformRotation.Pitch = SecondTransformRotation.Pitch;
			}
			if (!CurrentSection->bMatchRotationRoll)
			{
				FirstTransformRotation.Roll = SecondTransformRotation.Roll;
			}
			FirstTransformQuat = QuatFromEuler(FirstTransformRotation.Euler(), RotationOrder);
			FirstTransform.SetRotation(FirstTransformQuat);

			// Below is the match but we need to use GetRelativeTransformReverse since Inverse doesn't work as expected.
			//	* GetRelativeTransformReverse returns this(-1)* Other, and parameter is Other.
			SecondSectionRootDiff = SecondTransform.GetRelativeTransformReverse(FirstTransform);
			TranslationDiff = SecondSectionRootDiff.GetTranslation();
			RotationDiff = SecondSectionRootDiff.GetRotation();

		}
	}
	else if (bMatchWithPrevious == false && NextSection) //match with next
	{
		UMovieSceneSkeletalAnimationSection* SecondSection = Cast<UMovieSceneSkeletalAnimationSection>(NextSection);
		UAnimSequence* FirstAnimSequence = Cast<UAnimSequence>(CurrentSection->Params.Animation);
		UAnimSequence* SecondAnimSequence = Cast<UAnimSequence>(SecondSection->Params.Animation);

		if (FirstAnimSequence && SecondAnimSequence)
		{
			//use same index for all
			int32 Index = CurrentSection->SetBoneIndexForRootMotionCalculations(bBlendFirstChildOfRoot);
			FCompactPoseBoneIndex ParentIndex(Index);
			float FirstSectionTime = static_cast<float>(CurrentSection->MapTimeToAnimation(CurrentFrame, InTickResolution));
			FTransform  FirstTransform = GetTransformForBoneRelativeToIndex(FirstAnimSequence, SkelMeshComp, BoneName,ParentIndex, FirstSectionTime);
			float SecondSectionTime = static_cast<float>(SecondSection->MapTimeToAnimation(CurrentFrame, InTickResolution));
			FTransform  SecondTransform = GetTransformForBoneRelativeToIndex(SecondAnimSequence, SkelMeshComp, BoneName,ParentIndex, SecondSectionTime);

			//Need to match the translations and rotations here 
			//First need to get the correct rotation order based upon what's matching, otherwise if not all are matched 
			//and one rotation is set last we will get errors.
			ERotationOrder RotationOrder = FindBestRotationOrder(CurrentSection->bMatchRotationRoll, CurrentSection->bMatchRotationPitch, CurrentSection->bMatchRotationYaw);

			FVector FirstTransformTranslation = FirstTransform.GetTranslation();
			FVector SecondTransformTranslation = SecondTransform.GetTranslation();
			FQuat FirstTransformQuat = FirstTransform.GetRotation();
			FQuat SecondTransformQuat = SecondTransform.GetRotation();

			SecondTransformQuat.EnforceShortestArcWith(FirstTransformQuat);

			FRotator FirstTransformRotation(FRotator::MakeFromEuler(EulerFromQuat(FirstTransformQuat, RotationOrder)));
			FRotator SecondTransformRotation(FRotator::MakeFromEuler(EulerFromQuat(SecondTransformQuat, RotationOrder)));
			FirstTransformRotation.SetClosestToMe(SecondTransformRotation);

			if (!CurrentSection->bMatchTranslation)
			{
				SecondTransformTranslation.X = FirstTransformTranslation.X;
				SecondTransformTranslation.Y = FirstTransformTranslation.Y;
				SecondTransformTranslation.Z = FirstTransformTranslation.Z;
			}
			if (!CurrentSection->bMatchIncludeZHeight)
			{
				SecondTransformTranslation.Z = FirstTransformTranslation.Z;
			}
			SecondTransform.SetTranslation(SecondTransformTranslation);

			if (!CurrentSection->bMatchRotationYaw)
			{
				SecondTransformRotation.Yaw = FirstTransformRotation.Yaw;
			}
			if (!CurrentSection->bMatchRotationPitch)
			{
				SecondTransformRotation.Pitch = FirstTransformRotation.Pitch;
			}
			if (!CurrentSection->bMatchRotationRoll)
			{
				SecondTransformRotation.Roll = FirstTransformRotation.Roll;
			}
			SecondTransformQuat = QuatFromEuler(SecondTransformRotation.Euler(), RotationOrder);
			SecondTransform.SetRotation(SecondTransformQuat);

			//GetRelativeTransformReverse returns this(-1)* Other, and parameter is Other.
			SecondSectionRootDiff = FirstTransform.GetRelativeTransformReverse(SecondTransform);
			TranslationDiff = SecondSectionRootDiff.GetTranslation();
			RotationDiff = SecondSectionRootDiff.GetRotation();

		}
	}
}

#if WITH_EDITORONLY_DATA

void UMovieSceneCommonAnimationTrack::ToggleShowRootMotionTrail()
{
	bShowRootMotionTrail = bShowRootMotionTrail ? false : true;
}
#endif


//MZ To Do need way to get passed the skelmeshcomp when we add or move a section.
void UMovieSceneCommonAnimationTrack::AutoMatchSectionRoot(UMovieSceneSkeletalAnimationSection* CurrentSection)
{
	return;
#if 0
	UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
	if (AnimationSections.Num() > 0 && MovieScene && CurrentSection)
	{
		SortSections();

		for (int32 Index = 0; Index < AnimationSections.Num(); ++Index)
		{
			UMovieSceneSection* Section = AnimationSections[Index];
			if (Section && Section == CurrentSection)
			{
				CurrentSection->bMatchWithPrevious = (Index == 0) ? false : true;
				FFrameTime FrameTime = (Index == 0) ? CurrentSection->GetRange().GetUpperBoundValue() : CurrentSection->GetRange().GetLowerBoundValue();
				USkeletalMeshComponent* SkelMeshComp = nullptr;
				CurrentSection->MatchSectionByBoneTransform(SkelMeshComp, FrameTime, MovieScene->GetTickResolution(), CurrentSection->MatchedBoneName);
			}
		}
	}
#endif
}
