// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneCustomPrimitiveDataTrack.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneCustomPrimitiveDataSection.h"
#include "Materials/MaterialParameters.h"
#include "Materials/MaterialInterface.h"
#include "Components/PrimitiveComponent.h"
#include "Components/MeshComponent.h"

#if WITH_EDITORONLY_DATA
#include "IMovieScenePlayer.h"
#endif

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneCustomPrimitiveDataTrack)
#define LOCTEXT_NAMESPACE "CustomPrimitiveDataTrack"

UMovieSceneCustomPrimitiveDataTrack::UMovieSceneCustomPrimitiveDataTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(64, 192, 64, 65);
#endif

	BuiltInTreePopulationMode = ETreePopulationMode::Blended;

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
	SupportedBlendTypes.Add(EMovieSceneBlendType::Additive);
	SupportedBlendTypes.Add(EMovieSceneBlendType::AdditiveFromBase);
}


bool UMovieSceneCustomPrimitiveDataTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneCustomPrimitiveDataSection::StaticClass();
}


UMovieSceneSection* UMovieSceneCustomPrimitiveDataTrack::CreateNewSection()
{
	UMovieSceneSection* NewSection = NewObject<UMovieSceneCustomPrimitiveDataSection>(this, NAME_None, RF_Transactional);
	NewSection->SetBlendType(EMovieSceneBlendType::Absolute);
	return NewSection;
}


void UMovieSceneCustomPrimitiveDataTrack::RemoveAllAnimationData()
{
	Sections.Empty();
	SectionToKey = nullptr;
}


bool UMovieSceneCustomPrimitiveDataTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}


void UMovieSceneCustomPrimitiveDataTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);

	if (Sections.Num() > 1)
	{
		SetSectionToKey(&Section);
	}
}


void UMovieSceneCustomPrimitiveDataTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
	if (SectionToKey == &Section)
	{
		if (Sections.Num() > 0)
		{
			SectionToKey = Sections[0];
		}
		else
		{
			SectionToKey = nullptr;
		}
	}
}

void UMovieSceneCustomPrimitiveDataTrack::RemoveSectionAt(int32 SectionIndex)
{
	bool bResetSectionToKey = (SectionToKey == Sections[SectionIndex]);

	Sections.RemoveAt(SectionIndex);

	if (bResetSectionToKey)
	{
		SectionToKey = Sections.Num() > 0 ? Sections[0] : nullptr;
	}
}


bool UMovieSceneCustomPrimitiveDataTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

bool UMovieSceneCustomPrimitiveDataTrack::SupportsMultipleRows() const
{
	return true;
}

void UMovieSceneCustomPrimitiveDataTrack::SetSectionToKey(UMovieSceneSection* InSection)
{
	SectionToKey = InSection;
}

UMovieSceneSection* UMovieSceneCustomPrimitiveDataTrack::GetSectionToKey() const
{
	return SectionToKey;
}

const TArray<UMovieSceneSection*>& UMovieSceneCustomPrimitiveDataTrack::GetAllSections() const
{
	return Sections;
}


void UMovieSceneCustomPrimitiveDataTrack::AddScalarParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, float Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddScalarParameterKey(CustomPrimitiveDataStartIndex, Time, INDEX_NONE, Value, DefaultInterpolation);
}

void UMovieSceneCustomPrimitiveDataTrack::AddScalarParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, int32 RowIndex, float Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	UMovieSceneSection* NearestSection = SectionToKey;
	if (NearestSection == nullptr || (RowIndex != INDEX_NONE && NearestSection->GetRowIndex() != RowIndex))
	{
		NearestSection = MovieSceneHelpers::FindNearestSectionAtTime(Sections, Time, RowIndex);
	}
	if (NearestSection == nullptr)
	{
		NearestSection = Cast<UMovieSceneCustomPrimitiveDataSection>(CreateNewSection());

		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		check(MovieScene);

		NearestSection->SetRange(MovieScene->GetPlaybackRange());
		Sections.Add(NearestSection);
	}
	if (NearestSection != nullptr && NearestSection->TryModify())
	{
		if (UMovieSceneCustomPrimitiveDataSection* NearestCustomPrimitiveDataSection = Cast<UMovieSceneCustomPrimitiveDataSection>(NearestSection))
		{
			NearestCustomPrimitiveDataSection->AddScalarParameterKey(*FString::FromInt(CustomPrimitiveDataStartIndex), Time, Value, DefaultInterpolation);
		}
	}
}

void UMovieSceneCustomPrimitiveDataTrack::AddVector2DParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, FVector2D Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddVector2DParameterKey(CustomPrimitiveDataStartIndex, Time, INDEX_NONE, Value, DefaultInterpolation);
}

void UMovieSceneCustomPrimitiveDataTrack::AddVector2DParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, int32 RowIndex, FVector2D Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	UMovieSceneSection* NearestSection = SectionToKey;
	if (NearestSection == nullptr || (RowIndex != INDEX_NONE && NearestSection->GetRowIndex() != RowIndex))
	{
		NearestSection = MovieSceneHelpers::FindNearestSectionAtTime(Sections, Time, RowIndex);
	}
	if (NearestSection == nullptr)
	{
		NearestSection = Cast<UMovieSceneCustomPrimitiveDataSection>(CreateNewSection());

		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		check(MovieScene);

		NearestSection->SetRange(MovieScene->GetPlaybackRange());
		Sections.Add(NearestSection);
	}
	if (NearestSection != nullptr && NearestSection->TryModify())
	{
		if (UMovieSceneCustomPrimitiveDataSection* NearestCustomPrimitiveDataSection = Cast<UMovieSceneCustomPrimitiveDataSection>(NearestSection))
		{
			NearestCustomPrimitiveDataSection->AddVector2DParameterKey(*FString::FromInt(CustomPrimitiveDataStartIndex), Time, Value, DefaultInterpolation);
		}
	}
}

void UMovieSceneCustomPrimitiveDataTrack::AddVectorParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, FVector Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddVectorParameterKey(CustomPrimitiveDataStartIndex, Time, INDEX_NONE, Value, DefaultInterpolation);
}

void UMovieSceneCustomPrimitiveDataTrack::AddVectorParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, int32 RowIndex, FVector Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	UMovieSceneSection* NearestSection = SectionToKey;
	if (NearestSection == nullptr || (RowIndex != INDEX_NONE && NearestSection->GetRowIndex() != RowIndex))
	{
		NearestSection = MovieSceneHelpers::FindNearestSectionAtTime(Sections, Time, RowIndex);
	}
	if (NearestSection == nullptr)
	{
		NearestSection = Cast<UMovieSceneCustomPrimitiveDataSection>(CreateNewSection());

		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		check(MovieScene);

		NearestSection->SetRange(MovieScene->GetPlaybackRange());
		Sections.Add(NearestSection);
	}
	if (NearestSection != nullptr && NearestSection->TryModify())
	{
		if (UMovieSceneCustomPrimitiveDataSection* NearestCustomPrimitiveDataSection = Cast<UMovieSceneCustomPrimitiveDataSection>(NearestSection))
		{
			NearestCustomPrimitiveDataSection->AddVectorParameterKey(*FString::FromInt(CustomPrimitiveDataStartIndex), Time, Value, DefaultInterpolation);
		}
	}
}

void UMovieSceneCustomPrimitiveDataTrack::AddColorParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, FLinearColor Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddColorParameterKey(CustomPrimitiveDataStartIndex, Time, INDEX_NONE, Value, DefaultInterpolation);
}

void UMovieSceneCustomPrimitiveDataTrack::AddColorParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, int32 RowIndex, FLinearColor Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	UMovieSceneSection* NearestSection = SectionToKey;
	if (NearestSection == nullptr || (RowIndex != INDEX_NONE && NearestSection->GetRowIndex() != RowIndex))
	{
		NearestSection = MovieSceneHelpers::FindNearestSectionAtTime(Sections, Time, RowIndex);
	}
	if (NearestSection == nullptr)
	{
		NearestSection = Cast<UMovieSceneCustomPrimitiveDataSection>(CreateNewSection());

		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		check(MovieScene);

		NearestSection->SetRange(MovieScene->GetPlaybackRange());
		Sections.Add(NearestSection);
	}
	if (NearestSection != nullptr && NearestSection->TryModify())
	{
		if (UMovieSceneCustomPrimitiveDataSection* NearestCustomPrimitiveDataSection = Cast<UMovieSceneCustomPrimitiveDataSection>(NearestSection))
		{
			NearestCustomPrimitiveDataSection->AddColorParameterKey(*FString::FromInt(CustomPrimitiveDataStartIndex), Time, Value, DefaultInterpolation);
		}
	}
}


void UMovieSceneCustomPrimitiveDataTrack::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	// These tracks don't define any entities for themselves
	checkf(false, TEXT("This track should never have created entities for itself - this assertion indicates an error in the entity-component field"));
}

void UMovieSceneCustomPrimitiveDataTrack::ExtendEntityImpl(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	// Custom primitive data parameters are always absolute blends for the time being
	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.AddTag(TracksComponents->Tags.CustomPrimitiveData)
		// If the section has no valid blend type (legacy data), make it use absolute blending.
		// Otherwise, the base section class will add the appropriate blend type tag in BuildDefaultComponents.
		.AddTagConditional(BuiltInComponents->Tags.AbsoluteBlend, !Section->GetBlendType().IsValid())
	);
}

bool UMovieSceneCustomPrimitiveDataTrack::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	const FMovieSceneTrackEvaluationField& LocalEvaluationField = GetEvaluationField();

	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : LocalEvaluationField.Entries)
	{
		UMovieSceneCustomPrimitiveDataSection* CPDSection = Cast<UMovieSceneCustomPrimitiveDataSection>(Entry.Section);
		if (CPDSection)
		{
			if (IsRowEvalDisabled(Entry.Section->GetRowIndex()))
			{
				continue;
			}

			TRange<FFrameNumber> SectionEffectiveRange = TRange<FFrameNumber>::Intersection(EffectiveRange, Entry.Range);
			if (!SectionEffectiveRange.IsEmpty())
			{
				FMovieSceneEvaluationFieldEntityMetaData SectionMetaData = InMetaData;
				SectionMetaData.Flags = Entry.Flags;
				if (Entry.Section)
				{
					SectionMetaData.Condition = MovieSceneHelpers::GetSequenceCondition(this, Entry.Section, true);
				}
				CPDSection->ExternalPopulateEvaluationField(SectionEffectiveRange, SectionMetaData, OutFieldBuilder);
			}
		}
	}

	return true;
}


#if WITH_EDITORONLY_DATA

void UMovieSceneCustomPrimitiveDataTrack::GetCPDMaterialData(IMovieScenePlayer& Player, FGuid BoundObjectId, FMovieSceneSequenceID SequenceID, TSortedMap<uint8, TArray<FCustomPrimitiveDataMaterialParametersData>>& OutCPDMaterialData)
{
	// Empty the map
	OutCPDMaterialData.Empty();

	auto FindMaterialData = [this, &OutCPDMaterialData](EMaterialParameterType ParameterType, UMaterialInterface* MaterialInterface, const FComponentMaterialInfo& MaterialInfo)
	{
		TMap<FMaterialParameterInfo, FMaterialParameterMetadata> Parameters;

		MaterialInterface->GetAllParametersOfType(ParameterType, Parameters);

		for (const TPair<FMaterialParameterInfo, FMaterialParameterMetadata>& Parameter : Parameters)
		{
			const FMaterialParameterInfo& ParameterInfo = Parameter.Key;
			const FMaterialParameterMetadata& ParameterMetadata = Parameter.Value;

			if (ParameterMetadata.PrimitiveDataIndex > INDEX_NONE)
			{
				OutCPDMaterialData.FindOrAdd(ParameterMetadata.PrimitiveDataIndex).Add(
					{
						ParameterType,
						MaterialInfo,
						ParameterInfo,
						MaterialInterface
					});
			}
		}
	};

	// Find the bound object
	for (TWeakObjectPtr<> WeakObject : Player.FindBoundObjects(BoundObjectId, SequenceID))
	{
		if (USceneComponent* SceneComponent = Cast<USceneComponent>(WeakObject.Get()))
		{
			// Find each type of component with materials we can, and search for parameters of scalar and vector types that specify PrimitiveDataIndex
			if (UPrimitiveComponent* PrimitiveComponent = Cast<UPrimitiveComponent>(SceneComponent))
			{
				int32 NumMaterials = PrimitiveComponent->GetNumMaterials();
				TArray<FName> MaterialSlotNames = PrimitiveComponent->GetMaterialSlotNames();
				UMeshComponent* MeshComponent = Cast<UMeshComponent>(SceneComponent);
				if (NumMaterials > 0 || MeshComponent)
				{
					for (int32 MaterialIndex = 0; MaterialIndex < NumMaterials; MaterialIndex++)
					{
						if (UMaterialInterface* IndexedMaterial = PrimitiveComponent->GetMaterial(MaterialIndex))
						{
							FName MaterialSlotName = MaterialSlotNames.IsValidIndex(MaterialIndex) ? MaterialSlotNames[MaterialIndex] : FName();
							FComponentMaterialInfo MaterialInfo{ MaterialSlotName, MaterialIndex, EComponentMaterialType::IndexedMaterial };
							FindMaterialData(EMaterialParameterType::Scalar, IndexedMaterial, MaterialInfo);
							FindMaterialData(EMaterialParameterType::Vector, IndexedMaterial, MaterialInfo);
						}
					}
					if (MeshComponent)
					{
						if (UMaterialInterface* OverlayMaterial = MeshComponent->GetOverlayMaterial())
						{
							FComponentMaterialInfo MaterialInfo{ FName(), 0, EComponentMaterialType::OverlayMaterial };
							FindMaterialData(EMaterialParameterType::Scalar, OverlayMaterial, MaterialInfo);
							FindMaterialData(EMaterialParameterType::Vector, OverlayMaterial, MaterialInfo);
						}
					}
				}
			}
		}
	}
}


FText UMovieSceneCustomPrimitiveDataTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("CustomPrimitiveDataTrackName", "Custom Primitive Data");
}

#endif

#undef LOCTEXT_NAMESPACE