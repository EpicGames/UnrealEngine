// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneDataLayerTrack.h"
#include "Sections/MovieSceneDataLayerSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDataLayerTrack)

#define LOCTEXT_NAMESPACE "MovieSceneDataLayerTrack"

UMovieSceneDataLayerTrack::UMovieSceneDataLayerTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvalOptions.bEvaluateInPreroll = EvalOptions.bEvaluateInPostroll = true;
}

bool UMovieSceneDataLayerTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

void UMovieSceneDataLayerTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

void UMovieSceneDataLayerTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneDataLayerTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

bool UMovieSceneDataLayerTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneDataLayerSection::StaticClass();
}

UMovieSceneSection* UMovieSceneDataLayerTrack::CreateNewSection()
{
	return NewObject<UMovieSceneDataLayerSection>(this, NAME_None, RF_Transactional);
}

const TArray<UMovieSceneSection*>& UMovieSceneDataLayerTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneDataLayerTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}


#if WITH_EDITORONLY_DATA
FText UMovieSceneDataLayerTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("DisplayName", "Data Layer");
}
#endif

#undef LOCTEXT_NAMESPACE

