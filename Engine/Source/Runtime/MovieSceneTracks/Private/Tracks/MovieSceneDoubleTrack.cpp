// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneDoubleTrack.h"
#include "Sections/MovieSceneDoubleSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneDoubleTrack)

UMovieSceneDoubleTrack::UMovieSceneDoubleTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieSceneDoubleTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneDoubleSection::StaticClass();
}

UMovieSceneSection* UMovieSceneDoubleTrack::CreateNewSection()
{
	return NewObject<UMovieSceneDoubleSection>(this, NAME_None, RF_Transactional);
}


