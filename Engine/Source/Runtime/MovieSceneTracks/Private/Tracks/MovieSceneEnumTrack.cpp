// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneEnumTrack.h"
#include "Sections/MovieSceneEnumSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEnumTrack)

UMovieSceneEnumTrack::UMovieSceneEnumTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
}

void UMovieSceneEnumTrack::PostLoad()
{
	Super::PostLoad();
	SetEnum(Enum);
}

bool UMovieSceneEnumTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneEnumSection::StaticClass();
}

UMovieSceneSection* UMovieSceneEnumTrack::CreateNewSection()
{
	UMovieSceneEnumSection* NewEnumSection = NewObject<UMovieSceneEnumSection>(this, NAME_None, RF_Transactional);
	NewEnumSection->EnumCurve.SetEnum(Enum);
	return NewEnumSection;
}

void UMovieSceneEnumTrack::SetEnum(UEnum* InEnum)
{
	Enum = InEnum;

	for (UMovieSceneSection* Section : Sections)
	{
		if (UMovieSceneEnumSection* EnumSection = Cast<UMovieSceneEnumSection>(Section))
		{
			EnumSection->EnumCurve.SetEnum(Enum);
		}
	}
}

UEnum* UMovieSceneEnumTrack::GetEnum() const
{
	return Enum;
}

