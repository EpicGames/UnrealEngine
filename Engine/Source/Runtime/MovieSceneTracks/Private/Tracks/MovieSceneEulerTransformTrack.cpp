// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneEulerTransformTrack.h"
#include "Sections/MovieScene3DTransformSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEulerTransformTrack)


UMovieSceneEulerTransformTrack::UMovieSceneEulerTransformTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(65, 173, 164, 65);
#endif

	SupportedBlendTypes = FMovieSceneBlendTypeField::All();

	EvalOptions.bEvaluateNearestSection_DEPRECATED = EvalOptions.bCanEvaluateNearestSection = true;
}

bool UMovieSceneEulerTransformTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieScene3DTransformSection::StaticClass();
}

UMovieSceneSection* UMovieSceneEulerTransformTrack::CreateNewSection()
{
	return NewObject<UMovieScene3DTransformSection>(this, NAME_None, RF_Transactional);
}


