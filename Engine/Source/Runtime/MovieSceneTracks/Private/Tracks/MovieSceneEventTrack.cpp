// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneEventTrack.h"
#include "MovieSceneCommonHelpers.h"
#include "Sections/MovieSceneEventSection.h"
#include "Sections/MovieSceneEventSectionBase.h"
#include "Sections/MovieSceneEventTriggerSection.h"
#include "Sections/MovieSceneEventRepeaterSection.h"
#include "Evaluation/MovieSceneEventTemplate.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Compilation/IMovieSceneTemplateGenerator.h"
#include "IMovieSceneTracksModule.h"
#include "Evaluation/MovieSceneEvaluationCustomVersion.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneEventTrack)

#define LOCTEXT_NAMESPACE "MovieSceneEventTrack"


/* UMovieSceneTrack interface
 *****************************************************************************/

#if WITH_EDITORONLY_DATA
void UMovieSceneEventTrack::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FMovieSceneEvaluationCustomVersion::GUID);

	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FMovieSceneEvaluationCustomVersion::GUID) < FMovieSceneEvaluationCustomVersion::EntityManager)
		{
			// Default for legacy content was AfterSpawn
			EventPosition = EFireEventsAtPosition::AfterSpawn;
		}
	}

	Super::Serialize(Ar);
}

void UMovieSceneEventTrack::PostRename(UObject* OldOuter, const FName OldName)
{
	if (OldOuter != GetOuter())
	{
		Super::PostRename(OldOuter, OldName);

		for (UMovieSceneSection* Section : Sections)
		{
			if (UMovieSceneEventSectionBase* EventSection = Cast<UMovieSceneEventSectionBase>(Section))
			{
				EventSection->PostDuplicateSectionEvent.Execute(EventSection);
			}
		}
	}
}
#endif// WITH_EDITOR

void UMovieSceneEventTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}


bool UMovieSceneEventTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneEventSection::StaticClass();
}

UMovieSceneSection* UMovieSceneEventTrack::CreateNewSection()
{
	return NewObject<UMovieSceneEventTriggerSection>(this, NAME_None, RF_Transactional);
}


const TArray<UMovieSceneSection*>& UMovieSceneEventTrack::GetAllSections() const
{
	return Sections;
}


bool UMovieSceneEventTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}


bool UMovieSceneEventTrack::IsEmpty() const
{
	return (Sections.Num() == 0);
}


void UMovieSceneEventTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}


void UMovieSceneEventTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneEventTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

EMovieSceneCompileResult UMovieSceneEventTrack::CustomCompile(FMovieSceneEvaluationTrack& Track, const FMovieSceneTrackCompilerArgs& Args) const
{
	// Don't run the legacy track compile for tracks that don't contain contain legacy event sections
	const bool bContainsLegacySections = Sections.ContainsByPredicate([](UMovieSceneSection* InSection){ return InSection && InSection->IsA<UMovieSceneEventSection>(); });
	if (!bContainsLegacySections)
	{
		return EMovieSceneCompileResult::Failure;
	}

	return Compile(Track, Args);
}

FMovieSceneEvalTemplatePtr UMovieSceneEventTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	if (const UMovieSceneEventSection* LegacyEventSection = Cast<const UMovieSceneEventSection>(&InSection))
	{
		return FMovieSceneEventSectionTemplate(*LegacyEventSection, *this);
	}

	return FMovieSceneEvalTemplatePtr();
}

void UMovieSceneEventTrack::PostCompile(FMovieSceneEvaluationTrack& Track, const FMovieSceneTrackCompilerArgs& Args) const
{
	switch (EventPosition)
	{
	case EFireEventsAtPosition::AtStartOfEvaluation:
		Track.SetEvaluationGroup(IMovieSceneTracksModule::GetEvaluationGroupName(EBuiltInEvaluationGroup::PreEvaluation));
		break;

	case EFireEventsAtPosition::AtEndOfEvaluation:
		Track.SetEvaluationGroup(IMovieSceneTracksModule::GetEvaluationGroupName(EBuiltInEvaluationGroup::PostEvaluation));
		break;

	default:
		Track.SetEvaluationGroup(IMovieSceneTracksModule::GetEvaluationGroupName(EBuiltInEvaluationGroup::SpawnObjects));
		Track.SetEvaluationPriority(UMovieSceneSpawnTrack::GetEvaluationPriority() - 100);
		break;
	}

	Track.SetEvaluationMethod(EEvaluationMethod::Swept);
}

void UMovieSceneEventTrack::PopulateDeterminismData(FMovieSceneDeterminismData& OutData, const TRange<FFrameNumber>& Range) const
{
	OutData.bParentSequenceRequiresLowerFence = true;
	OutData.bParentSequenceRequiresUpperFence = true;
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneEventTrack::GetDefaultDisplayName() const
{ 
	return LOCTEXT("TrackName", "Events"); 
}

#endif


#undef LOCTEXT_NAMESPACE

