// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneFadeTrack.h"
#include "Sections/MovieSceneFadeSection.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneFadeTrack)

#define LOCTEXT_NAMESPACE "MovieSceneFadeTrack"


/* UMovieSceneFadeTrack interface
 *****************************************************************************/
UMovieSceneFadeTrack::UMovieSceneFadeTrack(const FObjectInitializer& Init)
	: Super(Init)
{
	EvalOptions.bEvaluateNearestSection_DEPRECATED = EvalOptions.bCanEvaluateNearestSection = true;
}

bool UMovieSceneFadeTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneFadeSection::StaticClass();
}

UMovieSceneSection* UMovieSceneFadeTrack::CreateNewSection()
{
	return NewObject<UMovieSceneFadeSection>(this, NAME_None, RF_Transactional);
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneFadeTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Fade");
}
#endif


#undef LOCTEXT_NAMESPACE

