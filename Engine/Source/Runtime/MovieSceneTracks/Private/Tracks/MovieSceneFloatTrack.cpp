// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneFloatTrack.h"
#include "Sections/MovieSceneFloatSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneFloatTrack)

UMovieSceneFloatTrack::UMovieSceneFloatTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieSceneFloatTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneFloatSection::StaticClass();
}

UMovieSceneSection* UMovieSceneFloatTrack::CreateNewSection()
{
	return NewObject<UMovieSceneFloatSection>(this, NAME_None, RF_Transactional);
}


