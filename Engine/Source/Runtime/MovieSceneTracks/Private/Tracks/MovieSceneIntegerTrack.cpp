// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneIntegerTrack.h"
#include "Sections/MovieSceneIntegerSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneIntegerTrack)

UMovieSceneIntegerTrack::UMovieSceneIntegerTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieSceneIntegerTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneIntegerSection::StaticClass();
}

UMovieSceneSection* UMovieSceneIntegerTrack::CreateNewSection()
{
	return NewObject<UMovieSceneIntegerSection>(this, NAME_None, RF_Transactional);
}

