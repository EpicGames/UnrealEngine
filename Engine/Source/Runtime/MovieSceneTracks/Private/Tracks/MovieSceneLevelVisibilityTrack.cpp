// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneLevelVisibilityTrack.h"
#include "Sections/MovieSceneLevelVisibilitySection.h"
#include "Evaluation/PersistentEvaluationData.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "IMovieSceneTracksModule.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneLevelVisibilityTrack)

#define LOCTEXT_NAMESPACE "MovieSceneLevelVisibilityTrack"

UMovieSceneLevelVisibilityTrack::UMovieSceneLevelVisibilityTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
}


bool UMovieSceneLevelVisibilityTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}


void UMovieSceneLevelVisibilityTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}


void UMovieSceneLevelVisibilityTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneLevelVisibilityTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

bool UMovieSceneLevelVisibilityTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneLevelVisibilitySection::StaticClass();
}

UMovieSceneSection* UMovieSceneLevelVisibilityTrack::CreateNewSection()
{
	return NewObject<UMovieSceneLevelVisibilitySection>(this, NAME_None, RF_Transactional);
}


const TArray<UMovieSceneSection*>& UMovieSceneLevelVisibilityTrack::GetAllSections() const
{
	return Sections;
}


bool UMovieSceneLevelVisibilityTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}


#if WITH_EDITORONLY_DATA
FText UMovieSceneLevelVisibilityTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("DisplayName", "Level Visibility");
}
#endif

#undef LOCTEXT_NAMESPACE

