// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneMaterialParameterCollectionTrack.h"
#include "Materials/MaterialParameterCollection.h"

#include "Evaluation/MovieSceneEvaluationField.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneSequence.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneComponentMaterialParameterSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMaterialParameterCollectionTrack)

#define LOCTEXT_NAMESPACE "MovieSceneMaterialParameterCollectionTrack"

UMovieSceneMaterialParameterCollectionTrack::UMovieSceneMaterialParameterCollectionTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(64,192,64,65);
#endif
}

UMovieSceneSection* UMovieSceneMaterialParameterCollectionTrack::CreateNewSection()
{
	UMovieSceneSection* NewSection = NewObject<UMovieSceneParameterSection>(this, NAME_None, RF_Transactional);
	NewSection->SetBlendType(EMovieSceneBlendType::Absolute);
	NewSection->SetRange(TRange<FFrameNumber>::All());
	return NewSection;
}

void UMovieSceneMaterialParameterCollectionTrack::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	// These tracks don't define any entities for themselves
	checkf(false, TEXT("This track should never have created entities for itself - this assertion indicates an error in the entity-component field"));
}

void UMovieSceneMaterialParameterCollectionTrack::ExtendEntityImpl(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	// Material parameters are always absolute blends for the time being
	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(TracksComponents->MPC, MPC)
		.AddTag(BuiltInComponents->Tags.Root)
		// If the section has no valid blend type (legacy data), make it use absolute blending.
		// Otherwise, the base section class will add the appropriate blend type tag in BuildDefaultComponents.
		.AddTagConditional(BuiltInComponents->Tags.AbsoluteBlend, !Section->GetBlendType().IsValid())
	);
}

bool UMovieSceneMaterialParameterCollectionTrack::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	const FMovieSceneTrackEvaluationField& LocalEvaluationField = GetEvaluationField();

	// Define entities for the old style parameter sections. ComponentMaterialParameterSections define their own.
	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : LocalEvaluationField.Entries)
	{
		UMovieSceneParameterSection* ParameterSection = Cast<UMovieSceneParameterSection>(Entry.Section);
		UMovieSceneComponentMaterialParameterSection* ComponentMaterialParameterSection = Cast<UMovieSceneComponentMaterialParameterSection>(Entry.Section);
		if (ParameterSection || ComponentMaterialParameterSection)
		{
			if (IsRowEvalDisabled(Entry.Section->GetRowIndex()))
			{
				continue;
			}

			TRange<FFrameNumber> SectionEffectiveRange = TRange<FFrameNumber>::Intersection(EffectiveRange, Entry.Range);
			if (!SectionEffectiveRange.IsEmpty())
			{
				FMovieSceneEvaluationFieldEntityMetaData SectionMetaData = InMetaData;
				SectionMetaData.Flags = Entry.Flags;
				if (Entry.Section)
				{
					SectionMetaData.Condition = MovieSceneHelpers::GetSequenceCondition(this, Entry.Section, true);
				}
				if (ParameterSection)
				{
					ParameterSection->ExternalPopulateEvaluationField(SectionEffectiveRange, SectionMetaData, OutFieldBuilder);
				}
				else if (ComponentMaterialParameterSection)
				{
					ComponentMaterialParameterSection->ExternalPopulateEvaluationField(SectionEffectiveRange, SectionMetaData, OutFieldBuilder);
				}
			}
		}
	}

	return true;
}

bool UMovieSceneMaterialParameterCollectionTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneComponentMaterialParameterSection::StaticClass() || SectionClass == UMovieSceneParameterSection::StaticClass();
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneMaterialParameterCollectionTrack::GetDefaultDisplayName() const
{
	if (MPC)
	{
		return FText::FromString(MPC->GetName());
	}

	return LOCTEXT("DefaultTrackName", "Material Parameter Collection");
}
#endif

#if WITH_EDITOR

void UMovieSceneMaterialParameterCollectionTrack::PostLoad()
{
	if (!GetDisplayName_Internal().IsEmpty() && MPC && GetDisplayName_Internal().EqualTo(FText::FromString(MPC->GetName())))
	{
		UMovieSceneSequence* Sequence = GetTypedOuter<UMovieSceneSequence>();
		UE_LOGF(LogMovieScene, Display, "Material Parameter Collection Track: Removing display name %ls from %ls because it is identical to the material name and track names are now derived from the material name", *MPC->GetName(), *GetNameSafe(Sequence));
		SetDisplayName(FText::GetEmpty());
	}

	Super::PostLoad();
}

#endif

#undef LOCTEXT_NAMESPACE

