// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneMaterialTrack.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Sections/MovieSceneComponentMaterialParameterSection.h"
#include "IMovieScenePlayer.h"
#include "Components/DecalComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/MeshComponent.h"
#include "Materials/MaterialInterface.h"
#include "Components/VolumetricCloudComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneMaterialTrack)

#define LOCTEXT_NAMESPACE "MaterialTrack"

UMovieSceneMaterialTrack::UMovieSceneMaterialTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(64,192,64,65);
#endif

	BuiltInTreePopulationMode = ETreePopulationMode::Blended;

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
	SupportedBlendTypes.Add(EMovieSceneBlendType::Additive);
	SupportedBlendTypes.Add(EMovieSceneBlendType::AdditiveFromBase);
}


bool UMovieSceneMaterialTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneComponentMaterialParameterSection::StaticClass() || SectionClass == UMovieSceneParameterSection::StaticClass();
}


UMovieSceneSection* UMovieSceneMaterialTrack::CreateNewSection()
{
	UMovieSceneSection* NewSection = NewObject<UMovieSceneComponentMaterialParameterSection>(this, NAME_None, RF_Transactional);
	NewSection->SetBlendType(EMovieSceneBlendType::Absolute);
	return NewSection;
}


void UMovieSceneMaterialTrack::RemoveAllAnimationData()
{
	Sections.Empty();
	SectionToKey = nullptr;
}


bool UMovieSceneMaterialTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}


void UMovieSceneMaterialTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);

	if (Sections.Num() > 1)
	{
		SetSectionToKey(&Section);
	}
}


void UMovieSceneMaterialTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
	if (SectionToKey == &Section)
	{
		if (Sections.Num() > 0)
		{
			SectionToKey = Sections[0];
		}
		else
		{
			SectionToKey = nullptr;
		}
	}
}

void UMovieSceneMaterialTrack::RemoveSectionAt(int32 SectionIndex)
{
	bool bResetSectionToKey = (SectionToKey == Sections[SectionIndex]);

	Sections.RemoveAt(SectionIndex);

	if (bResetSectionToKey)
	{
		SectionToKey = Sections.Num() > 0 ? Sections[0] : nullptr;
	}
}


bool UMovieSceneMaterialTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

bool UMovieSceneMaterialTrack::SupportsMultipleRows() const
{
	return true;
}

void UMovieSceneMaterialTrack::SetSectionToKey(UMovieSceneSection* InSection)
{
	SectionToKey = InSection;
}

UMovieSceneSection* UMovieSceneMaterialTrack::GetSectionToKey() const
{
	return SectionToKey;
}

const TArray<UMovieSceneSection*>& UMovieSceneMaterialTrack::GetAllSections() const
{
	return Sections;
}


void UMovieSceneMaterialTrack::AddScalarParameterKey(FName ParameterName, FFrameNumber Time, float Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddScalarParameterKey(FMaterialParameterInfo(ParameterName), Time, INDEX_NONE, Value, FString(), FString(), DefaultInterpolation);
}

void UMovieSceneMaterialTrack::AddScalarParameterKey(FName ParameterName, FFrameNumber Time, int32 RowIndex, float Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddScalarParameterKey(FMaterialParameterInfo(ParameterName), Time, RowIndex, Value, FString(), FString(), DefaultInterpolation);
}

void UMovieSceneMaterialTrack::AddScalarParameterKey(const FMaterialParameterInfo& ParameterInfo, FFrameNumber Time, float Value, const FString& InLayerName, const FString& InAssetName, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddScalarParameterKey(ParameterInfo, Time, INDEX_NONE, Value, InLayerName, InAssetName, DefaultInterpolation);
}

void UMovieSceneMaterialTrack::AddScalarParameterKey(const FMaterialParameterInfo& ParameterInfo, FFrameNumber Time, int32 RowIndex, float Value, const FString& InLayerName, const FString& InAssetName, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	UMovieSceneSection* NearestSection = SectionToKey;
	if (NearestSection == nullptr || (RowIndex != INDEX_NONE && NearestSection->GetRowIndex() != RowIndex))
	{
		NearestSection = MovieSceneHelpers::FindNearestSectionAtTime(Sections, Time, RowIndex);
	}
	if (NearestSection == nullptr)
	{
		NearestSection = CreateNewSection();

		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		check(MovieScene);

		NearestSection->SetRange(MovieScene->GetPlaybackRange());
		Sections.Add(NearestSection);
	}
	if (NearestSection != nullptr && NearestSection->TryModify())
	{
		// If we have an old parameter section already, fall back to the old style section
		if (UMovieSceneParameterSection* NearestParameterSection = Cast<UMovieSceneParameterSection>(NearestSection))
		{
			NearestParameterSection->AddScalarParameterKey(ParameterInfo.Name, Time, Value, DefaultInterpolation);
		}
		else if (UMovieSceneComponentMaterialParameterSection* NearestComponentMaterialParameterSection = Cast<UMovieSceneComponentMaterialParameterSection>(NearestSection))
		{
			NearestComponentMaterialParameterSection->AddScalarParameterKey(ParameterInfo, Time, Value, InLayerName, InAssetName, DefaultInterpolation);
		}
	}
}


void UMovieSceneMaterialTrack::AddColorParameterKey(FName ParameterName, FFrameNumber Time, FLinearColor Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddColorParameterKey(FMaterialParameterInfo(ParameterName), Time, INDEX_NONE, Value, FString(), FString(), DefaultInterpolation);
}


void UMovieSceneMaterialTrack::AddColorParameterKey(FName ParameterName, FFrameNumber Time, int32 RowIndex, FLinearColor Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddColorParameterKey(FMaterialParameterInfo(ParameterName), Time, RowIndex, Value, FString(), FString(), DefaultInterpolation);
}

void UMovieSceneMaterialTrack::AddColorParameterKey(const FMaterialParameterInfo& ParameterInfo, FFrameNumber Time, FLinearColor Value, const FString& InLayerName, const FString& InAssetName, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddColorParameterKey(ParameterInfo, Time, INDEX_NONE, Value, InLayerName, InAssetName, DefaultInterpolation);
}


void UMovieSceneMaterialTrack::AddColorParameterKey(const FMaterialParameterInfo& ParameterInfo, FFrameNumber Time, int32 RowIndex, FLinearColor Value, const FString& InLayerName, const FString& InAssetName, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	AddColorParameterKey(ParameterInfo, Time, RowIndex, Value, InLayerName, InAssetName, FParameterChannelNames(), DefaultInterpolation);
}

void UMovieSceneMaterialTrack::AddColorParameterKey(const FMaterialParameterInfo& ParameterInfo, FFrameNumber Time, int32 RowIndex, FLinearColor Value, const FString& InLayerName, const FString& InAssetName, const FParameterChannelNames& InChannelNames, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	UMovieSceneSection* NearestSection = SectionToKey;
	if (NearestSection == nullptr || (RowIndex != INDEX_NONE && NearestSection->GetRowIndex() != RowIndex))
	{
		NearestSection = MovieSceneHelpers::FindNearestSectionAtTime(Sections, Time, RowIndex);
	}
	if (NearestSection == nullptr)
	{
		NearestSection = CreateNewSection();

		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		check(MovieScene);

		NearestSection->SetRange(MovieScene->GetPlaybackRange());
		Sections.Add(NearestSection);
	}
	if (NearestSection != nullptr && NearestSection->TryModify())
	{
		// If we have an old parameter section already, fall back to the old style section
		if (UMovieSceneParameterSection* NearestParameterSection = Cast<UMovieSceneParameterSection>(NearestSection))
		{
			NearestParameterSection->AddColorParameterKey(ParameterInfo.Name, Time, Value, DefaultInterpolation);
		}
		else if (UMovieSceneComponentMaterialParameterSection* NearestComponentMaterialParameterSection = Cast<UMovieSceneComponentMaterialParameterSection>(NearestSection))
		{
			NearestComponentMaterialParameterSection->AddColorParameterKey(ParameterInfo, Time, Value, InLayerName, InAssetName, InChannelNames, DefaultInterpolation);
		}
	}
}


UMovieSceneComponentMaterialTrack::UMovieSceneComponentMaterialTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	BuiltInTreePopulationMode = ETreePopulationMode::Blended;
}

void UMovieSceneComponentMaterialTrack::ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity)
{
	// These tracks don't define any entities for themselves
	checkf(false, TEXT("This track should never have created entities for itself - this assertion indicates an error in the entity-component field"));
}

void UMovieSceneComponentMaterialTrack::ExtendEntityImpl(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	// Material parameters are always absolute blends for the time being
	OutImportedEntity->AddBuilder(
		FEntityBuilder()
		.Add(TracksComponents->ComponentMaterialInfo, MaterialInfo)
		// If the section has no valid blend type (legacy data), make it use absolute blending.
		// Otherwise, the base section class will add the appropriate blend type tag in BuildDefaultComponents.
		.AddTagConditional(BuiltInComponents->Tags.AbsoluteBlend, !Section->GetBlendType().IsValid())
	);
}

bool UMovieSceneComponentMaterialTrack::PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	const FMovieSceneTrackEvaluationField& LocalEvaluationField = GetEvaluationField();

	// Define entities for the old style parameter sections. ComponentMaterialParameterSections define their own.
	for (const FMovieSceneTrackEvaluationFieldEntry& Entry : LocalEvaluationField.Entries)
	{
		UMovieSceneParameterSection* ParameterSection = Cast<UMovieSceneParameterSection>(Entry.Section);
		UMovieSceneComponentMaterialParameterSection* ComponentMaterialParameterSection = Cast<UMovieSceneComponentMaterialParameterSection>(Entry.Section);
		if (ParameterSection || ComponentMaterialParameterSection)
		{
			if (IsRowEvalDisabled(Entry.Section->GetRowIndex()))
			{
				continue;
			}

			TRange<FFrameNumber> SectionEffectiveRange = TRange<FFrameNumber>::Intersection(EffectiveRange, Entry.Range);
			if (!SectionEffectiveRange.IsEmpty())
			{
				FMovieSceneEvaluationFieldEntityMetaData SectionMetaData = InMetaData;
				SectionMetaData.Flags = Entry.Flags;
				if (Entry.Section)
				{
					SectionMetaData.Condition = MovieSceneHelpers::GetSequenceCondition(this, Entry.Section, true);
				}
				if (ParameterSection)
				{
					ParameterSection->ExternalPopulateEvaluationField(SectionEffectiveRange, SectionMetaData, OutFieldBuilder);
				}
				else if (ComponentMaterialParameterSection)
				{
					ComponentMaterialParameterSection->ExternalPopulateEvaluationField(SectionEffectiveRange, SectionMetaData, OutFieldBuilder);
				}
			}
		}
	}

	return true;
}


#if WITH_EDITORONLY_DATA

FText UMovieSceneComponentMaterialTrack::GetDefaultDisplayName() const
{
	// Old track name before we started naming directly from editor
	return FText::FromString(FString::Printf(TEXT("Material Element %i"), MaterialIndex_DEPRECATED));
}

FText UMovieSceneComponentMaterialTrack::GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const
{
	if (LabelParams.Player)
	{
		UMaterialInterface* Material = nullptr;
		for (TWeakObjectPtr<> WeakObject : LabelParams.Player->FindBoundObjects(LabelParams.BindingID, LabelParams.SequenceID))
		{
			switch (MaterialInfo.MaterialType)
			{
			case EComponentMaterialType::Empty:
				break;
			case EComponentMaterialType::IndexedMaterial:
				if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(WeakObject.Get()))
				{
					if (!MaterialInfo.MaterialSlotName.IsNone())
					{
						Material = Component->GetMaterialByName(MaterialInfo.MaterialSlotName);
						int32 FoundMaterialIndex = Component->GetMaterialIndex(MaterialInfo.MaterialSlotName);
						if (Material)
						{
							// Found a binding by slot name
							return (FoundMaterialIndex == MaterialInfo.MaterialSlotIndex) ? FText::Format(LOCTEXT("SlotMaterialTrackTooltip", "Material parameter track for slot {0} at index {1}, asset {2}"), FText::FromName(MaterialInfo.MaterialSlotName), FText::AsNumber(MaterialInfo.MaterialSlotIndex), FText::FromString(Material->GetName()))
								: FText::Format(LOCTEXT("SlotMaterialTrackTooltip_WrongIndexCached", "Material parameter track for slot {0} at index {1}, asset {2}. \n Asset has cached material index {3} that does not match material found with slot name. This can be corrected in right-click menu."), FText::FromName(MaterialInfo.MaterialSlotName), FText::AsNumber(FoundMaterialIndex), FText::FromString(Material->GetName()), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
						}
					}
					if (!Material)
					{
						// Couldn't find binding by slot name or one wasn't specified, try by index
						Material = Component->GetMaterial(MaterialInfo.MaterialSlotIndex);
						if (Material)
						{
							// Found by index, but may want to rebind to new slot name
							return MaterialInfo.MaterialSlotName.IsNone() ? FText::Format(LOCTEXT("IndexedMaterialTrackTooltip_NoSlotName", "Material parameter track for element at index {0}, asset {1}. \nBinding is missing Material Slot Name. This can be corrected in right-click menu."), FText::AsNumber(MaterialInfo.MaterialSlotIndex), FText::FromString(Material->GetName()))
								: FText::Format(LOCTEXT("IndexedMaterialTrackTooltip_IncorrectSlotName", "Material parameter track for element at index {0}, asset {1}. \n Binding references missing Material Slot Name {2}. This can be rebound in right-click menu."), FText::AsNumber(MaterialInfo.MaterialSlotIndex), FText::FromString(Material->GetName()), FText::FromName(MaterialInfo.MaterialSlotName));
						}
						else
						{
							// Couldn't find binding by slot or index
							return FText::Format(LOCTEXT("SlotMaterialTrackTooltip_MissingMaterial", "Material could not be found for slot {0} and cached index {1}"), FText::FromName(MaterialInfo.MaterialSlotName), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
						}
					}
				}
				break;
			case EComponentMaterialType::OverlayMaterial:
				if (UMeshComponent* Component = Cast<UMeshComponent>(WeakObject.Get()))
				{
					Material = Component->GetOverlayMaterial();
					if (Material)
					{
						return FText::Format(LOCTEXT("OverlayMaterialTrackTooltip", "Material parameter track for overlay material {0}"), FText::FromString(Material->GetName()));
					}
					else
					{
						return LOCTEXT("OverlayMaterialTrackTooltip_MissingMaterial", "No overlay material could be found");
					}
				}
				break;
			case EComponentMaterialType::DecalMaterial:
				if (UDecalComponent* DecalComponent = Cast<UDecalComponent>(WeakObject.Get()))
				{
					Material = DecalComponent->GetDecalMaterial();
					if (Material)
					{
						return FText::Format(LOCTEXT("DecalMaterialTrackTooltip", "Material parameter track for decal material {0}"), FText::FromString(Material->GetName()));
					}
					else
					{
						return LOCTEXT("DecalMaterialTrackTooltip_MissingMaterial", "No decal material could be found");
					}
				}
				break;			
			case EComponentMaterialType::VolumetricCloudMaterial:
					if (UVolumetricCloudComponent* CloudComponent = Cast<UVolumetricCloudComponent>(WeakObject.Get()))
					{
						Material = CloudComponent->GetMaterial();
						if (Material)
						{
							return FText::Format(LOCTEXT("CloudMaterialTrackTooltip", "Material parameter track for volumetric cloud material {0}"), FText::FromString(Material->GetName()));
						}
						else
						{
							return LOCTEXT("CloudMaterialTrackTooltip_MissingMaterial", "No volumetric cloud material could be found");
						}
					}
					break;
			case EComponentMaterialType::SlotOverlayMaterial:
				if (UMeshComponent* Component = Cast<UMeshComponent>(WeakObject.Get()))
				{
					Material = Component->GetOverlayMaterial(true, MaterialInfo.MaterialSlotIndex);
					if (Material)
					{
						return !MaterialInfo.MaterialSlotName.IsNone() ?
							FText::Format(LOCTEXT("SlotOverlayMaterialTrackTooltip", "Material parameter track for overlay slot {0} at index {1}, asset {2}"), FText::FromName(MaterialInfo.MaterialSlotName), FText::AsNumber(MaterialInfo.MaterialSlotIndex), FText::FromString(Material->GetName())) :
							FText::Format(LOCTEXT("IndexedSlotOverlayMaterialTrackTooltip", "Material parameter track for overlay element at index {0}, asset {1}"), FText::AsNumber(MaterialInfo.MaterialSlotIndex), FText::FromString(Material->GetName()));
					}
					else
					{
						return !MaterialInfo.MaterialSlotName.IsNone() ?
							FText::Format(LOCTEXT("SlotOverlayMaterialTrackTooltip_Missing", "No overlay material found for slot {0} at index {1}"), FText::FromName(MaterialInfo.MaterialSlotName), FText::AsNumber(MaterialInfo.MaterialSlotIndex)) :
							FText::Format(LOCTEXT("IndexedSlotOverlayMaterialTrackTooltip_Missing", "No overlay material found at index {0}"), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
					}
				}
				break;
			default:
				break;
			}
		}
	}
	return FText();
}

FSlateColor UMovieSceneComponentMaterialTrack::GetLabelColor(const FMovieSceneLabelParams& LabelParams) const
{
	auto GetDimmedColor = [&LabelParams](FLinearColor LinearColor)
	{
		return FSlateColor(LabelParams.bIsDimmed ? LinearColor.Desaturate(0.6f) : LinearColor);
	};
	if (LabelParams.Player)
	{
		for (TWeakObjectPtr<> WeakObject : LabelParams.Player->FindBoundObjects(LabelParams.BindingID, LabelParams.SequenceID))
		{
			switch (MaterialInfo.MaterialType)
			{
			case EComponentMaterialType::Empty:
				break;
			case EComponentMaterialType::IndexedMaterial:
				if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(WeakObject.Get()))
				{
					if (MaterialInfo.MaterialSlotName.IsNone())
					{
						// No material slot name. Can we find by index?
						if (Component->GetMaterial(MaterialInfo.MaterialSlotIndex))
						{
							// Found by index, but show yellow because missing slot name
							return GetDimmedColor(FLinearColor::Yellow);
						}
						else
						{
							// No material found at all
							return GetDimmedColor(FLinearColor::Red);
						}
					}
					else
					{
						UMaterialInterface* Material = Component->GetMaterialByName(MaterialInfo.MaterialSlotName);
						int32 FoundMaterialIndex = Component->GetMaterialIndex(MaterialInfo.MaterialSlotName);
						if (Material)
						{
							if (FoundMaterialIndex != MaterialInfo.MaterialSlotIndex)
							{
								// Indices don't match- yellow
								return GetDimmedColor(FLinearColor::Yellow);
							}
						}
						else
						{
							// Can't find material by name, try by index
							Material = Component->GetMaterial(MaterialInfo.MaterialSlotIndex);
							if (Material)
							{
								// Found, but didn't match name. Yellow
								return GetDimmedColor(FLinearColor::Yellow);
							}
							else
							{
								// Nothing found. Red.
								return GetDimmedColor(FLinearColor::Red);
							}
						}
					}
				}
				break;
			case EComponentMaterialType::OverlayMaterial:
				if (UMeshComponent* Component = Cast<UMeshComponent>(WeakObject.Get()))
				{
					UMaterialInterface* Material = Component->GetOverlayMaterial();
					if (!Material)
					{
						return GetDimmedColor(FLinearColor::Red);
					}
				}
				else
				{
					return GetDimmedColor(FLinearColor::Red);
				}
				break;
			case EComponentMaterialType::DecalMaterial:
				if (UDecalComponent* DecalComponent = Cast<UDecalComponent>(WeakObject.Get()))
				{
					UMaterialInterface* Material = DecalComponent->GetDecalMaterial();
					if (!Material)
					{
						return GetDimmedColor(FLinearColor::Red);
					}
				}
				else
				{
					return GetDimmedColor(FLinearColor::Red);
				}
				break;
			case EComponentMaterialType::VolumetricCloudMaterial:
				if (UVolumetricCloudComponent* CloudComponent = Cast<UVolumetricCloudComponent>(WeakObject.Get()))
				{
					UMaterialInterface* Material = CloudComponent->GetMaterial();
					if (!Material)
					{
						return GetDimmedColor(FLinearColor::Red);
					}
				}
				else
				{
					return GetDimmedColor(FLinearColor::Red);
				}
				break;
			case EComponentMaterialType::SlotOverlayMaterial:
				if (UMeshComponent* Component = Cast<UMeshComponent>(WeakObject.Get()))
				{
					UMaterialInterface* Material = Component->GetOverlayMaterial(true, MaterialInfo.MaterialSlotIndex);
					if (!Material)
					{
						return GetDimmedColor(FLinearColor::Red);
					}
				}
				else
				{
					return GetDimmedColor(FLinearColor::Red);
				}
				break;
			default:
				break;
			}
		}
	}
	return LabelParams.bIsDimmed ? FSlateColor::UseSubduedForeground() : FSlateColor::UseForeground();
}

void UMovieSceneComponentMaterialTrack::PostLoad()
{
	Super::PostLoad();
	// Backwards compatibility with MaterialIndex alone as a way to reference materials.
	if (MaterialInfo.MaterialType == EComponentMaterialType::Empty)
	{
		MaterialInfo.MaterialType = EComponentMaterialType::IndexedMaterial;
		MaterialInfo.MaterialSlotIndex = MaterialIndex_DEPRECATED;
	}
}
#endif

#undef LOCTEXT_NAMESPACE
