// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneObjectPropertyTrack.h"
#include "Sections/MovieSceneObjectPropertySection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneObjectPropertyTrack)


UMovieSceneObjectPropertyTrack::UMovieSceneObjectPropertyTrack(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	PropertyClass = nullptr;
}

bool UMovieSceneObjectPropertyTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneObjectPropertySection::StaticClass();
}


UMovieSceneSection* UMovieSceneObjectPropertyTrack::CreateNewSection()
{
	UMovieSceneObjectPropertySection* Section = NewObject<UMovieSceneObjectPropertySection>(this, NAME_None, RF_Transactional);
	Section->ObjectChannel.SetPropertyClass(PropertyClass);
	return Section;
}
