// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneParticleParameterTrack.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "Evaluation/MovieSceneParticleParameterTemplate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneParticleParameterTrack)

#define LOCTEXT_NAMESPACE "ParticleParameterTrack"

UMovieSceneParticleParameterTrack::UMovieSceneParticleParameterTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(0, 170, 255, 65);
#endif
}

FMovieSceneEvalTemplatePtr UMovieSceneParticleParameterTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneParticleParameterSectionTemplate(*CastChecked<UMovieSceneParameterSection>(&InSection), *this);
}

bool UMovieSceneParticleParameterTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneParameterSection::StaticClass();
}

UMovieSceneSection* UMovieSceneParticleParameterTrack::CreateNewSection()
{
	return NewObject<UMovieSceneParameterSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneParticleParameterTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

bool UMovieSceneParticleParameterTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

void UMovieSceneParticleParameterTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

void UMovieSceneParticleParameterTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneParticleParameterTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

bool UMovieSceneParticleParameterTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

const TArray<UMovieSceneSection*>& UMovieSceneParticleParameterTrack::GetAllSections() const
{
	return Sections;
}


#if WITH_EDITORONLY_DATA
FText UMovieSceneParticleParameterTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("DisplayName", "Particle Parameter");
}
#endif


void UMovieSceneParticleParameterTrack::AddScalarParameterKey( FName ParameterName, FFrameNumber Time, float Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	UMovieSceneParameterSection* NearestSection = Cast<UMovieSceneParameterSection>(MovieSceneHelpers::FindNearestSectionAtTime(Sections, Time));
	if ( NearestSection == nullptr )
	{
		NearestSection = Cast<UMovieSceneParameterSection>(CreateNewSection());

		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		check(MovieScene);

		NearestSection->SetRange(MovieScene->GetPlaybackRange());

		Sections.Add( NearestSection );
	}
	NearestSection->AddScalarParameterKey(ParameterName, Time, Value, DefaultInterpolation);
}

void UMovieSceneParticleParameterTrack::AddVectorParameterKey( FName ParameterName, FFrameNumber Time, FVector Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	UMovieSceneParameterSection* NearestSection = Cast<UMovieSceneParameterSection>( MovieSceneHelpers::FindNearestSectionAtTime( Sections, Time ) );
	if ( NearestSection == nullptr )
	{
		NearestSection = Cast<UMovieSceneParameterSection>( CreateNewSection() );

		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		check(MovieScene);

		NearestSection->SetRange(MovieScene->GetPlaybackRange());

		Sections.Add( NearestSection );
	}
	NearestSection->AddVectorParameterKey( ParameterName, Time, Value, DefaultInterpolation);
}

void UMovieSceneParticleParameterTrack::AddColorParameterKey( FName ParameterName, FFrameNumber Time, FLinearColor Value, EMovieSceneKeyInterpolation DefaultInterpolation)
{
	UMovieSceneParameterSection* NearestSection = Cast<UMovieSceneParameterSection>( MovieSceneHelpers::FindNearestSectionAtTime( Sections, Time ) );
	if ( NearestSection == nullptr )
	{
		NearestSection = Cast<UMovieSceneParameterSection>( CreateNewSection() );

		UMovieScene* MovieScene = GetTypedOuter<UMovieScene>();
		check(MovieScene);

		NearestSection->SetRange(MovieScene->GetPlaybackRange());

		Sections.Add( NearestSection );
	}
	NearestSection->AddColorParameterKey( ParameterName, Time, Value, DefaultInterpolation);
}
#undef LOCTEXT_NAMESPACE

