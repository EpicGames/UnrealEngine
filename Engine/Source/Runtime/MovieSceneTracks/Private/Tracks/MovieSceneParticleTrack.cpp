// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneParticleTrack.h"
#include "Evaluation/MovieSceneParticleTemplate.h"
#include "MovieSceneCommonHelpers.h"
#include "Sections/MovieSceneParticleSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneParticleTrack)


#define LOCTEXT_NAMESPACE "MovieSceneParticleTrack"


UMovieSceneParticleTrack::UMovieSceneParticleTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(255,255,255,160);
#endif
}


FMovieSceneEvalTemplatePtr UMovieSceneParticleTrack::CreateTemplateForSection(const UMovieSceneSection& InSection) const
{
	return FMovieSceneParticleSectionTemplate(*CastChecked<const UMovieSceneParticleSection>(&InSection));
}


const TArray<UMovieSceneSection*>& UMovieSceneParticleTrack::GetAllSections() const
{
	return ParticleSections;
}


void UMovieSceneParticleTrack::RemoveAllAnimationData()
{
	// do nothing
}


bool UMovieSceneParticleTrack::HasSection(const UMovieSceneSection& Section) const
{
	return ParticleSections.Contains(&Section);
}


void UMovieSceneParticleTrack::AddSection(UMovieSceneSection& Section)
{
	ParticleSections.Add(&Section);
}


void UMovieSceneParticleTrack::RemoveSection(UMovieSceneSection& Section)
{
	ParticleSections.Remove(&Section);
}


void UMovieSceneParticleTrack::RemoveSectionAt(int32 SectionIndex)
{
	ParticleSections.RemoveAt(SectionIndex);
}


bool UMovieSceneParticleTrack::IsEmpty() const
{
	return ParticleSections.Num() == 0;
}


void UMovieSceneParticleTrack::AddNewSection( FFrameNumber SectionTime )
{
	if ( MovieSceneHelpers::FindSectionAtTime( ParticleSections, SectionTime ) == nullptr )
	{
		UMovieSceneParticleSection* NewSection = Cast<UMovieSceneParticleSection>( CreateNewSection() );
		ParticleSections.Add(NewSection);
	}
}

bool UMovieSceneParticleTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneParticleSection::StaticClass();
}

UMovieSceneSection* UMovieSceneParticleTrack::CreateNewSection()
{
	return NewObject<UMovieSceneParticleSection>(this, NAME_None, RF_Transactional);
}

#if WITH_EDITORONLY_DATA
FText UMovieSceneParticleTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("DisplayName", "Particle System");
}
#endif

#undef LOCTEXT_NAMESPACE

