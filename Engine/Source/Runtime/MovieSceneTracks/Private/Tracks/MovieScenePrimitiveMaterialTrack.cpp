// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieScenePrimitiveMaterialTrack.h"
#include "Tracks/MovieSceneMaterialTrack.h"
#include "Sections/MovieScenePrimitiveMaterialSection.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Components/DecalComponent.h"
#include "Components/PrimitiveComponent.h"
#include "Components/MeshComponent.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePrimitiveMaterialTrack)

#define LOCTEXT_NAMESPACE "PrimitiveMaterialTrack"


UMovieScenePrimitiveMaterialTrack::UMovieScenePrimitiveMaterialTrack(const FObjectInitializer& ObjInit)
	: Super(ObjInit)
{
	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);
	SupportedBlendTypes.Add(EMovieSceneBlendType::Additive);

#if WITH_EDITORONLY_DATA
	TrackTint = FColor(64,192,64,75);
#endif
}

UMovieSceneSection* UMovieScenePrimitiveMaterialTrack::CreateNewSection()
{
	return NewObject<UMovieScenePrimitiveMaterialSection>(this, NAME_None, RF_Transactional);
}

bool UMovieScenePrimitiveMaterialTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieScenePrimitiveMaterialSection::StaticClass();
}

#if WITH_EDITORONLY_DATA

FText UMovieScenePrimitiveMaterialTrack::GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const
{
	if (LabelParams.Player)
	{
		UMaterialInterface* Material = nullptr;
		for (TWeakObjectPtr<> WeakObject : LabelParams.Player->FindBoundObjects(LabelParams.BindingID, LabelParams.SequenceID))
		{
			switch (MaterialInfo.MaterialType)
			{
			case EComponentMaterialType::Empty:
				break;
			case EComponentMaterialType::IndexedMaterial:
				if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(WeakObject.Get()))
				{
					if (!MaterialInfo.MaterialSlotName.IsNone())
					{
						Material = Component->GetMaterialByName(MaterialInfo.MaterialSlotName);
						int32 FoundMaterialIndex = Component->GetMaterialIndex(MaterialInfo.MaterialSlotName);
						if (Material)
						{
							// Found a binding by slot name
							return FText::Format(LOCTEXT("SlotMaterialSwitcherTrackTooltip", "Material switcher for {0} at index {1}"), FText::FromName(MaterialInfo.MaterialSlotName), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
						}
					}
					if (!Material)
					{
						// Couldn't find binding by slot name or one wasn't specified, try by index
						Material = Component->GetMaterial(MaterialInfo.MaterialSlotIndex);
						if (Material)
						{
							// Found by index, but may want to rebind to new slot name
							return MaterialInfo.MaterialSlotName.IsNone() ? FText::Format(LOCTEXT("IndexedMaterialSwitcherTrackTooltip_MissingSlotName", "Material switcher for element at index {0}\nBinding is missing Material Slot Name. This can be corrected in right-click menu."), FText::AsNumber(MaterialInfo.MaterialSlotIndex))
								: FText::Format(LOCTEXT("IndexedMaterialSwitcherTrackTooltip_IncorrectSlotName", "Material switcher for element at index {0} \n Binding references missing Material Slot Name {1}. This can be rebound in right-click menu."), FText::AsNumber(MaterialInfo.MaterialSlotIndex), FText::FromName(MaterialInfo.MaterialSlotName));
						}
						else
						{
							// Couldn't find binding by slot or index
							return FText::Format(LOCTEXT("SlotMaterialSwitcherTrackTooltip_MissingMaterial", "Material could not be found for slot {0} and cached index {1}"), FText::FromName(MaterialInfo.MaterialSlotName), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
						}
					}
				}
				break;
			case EComponentMaterialType::OverlayMaterial:
				if (UMeshComponent* Component = Cast<UMeshComponent>(WeakObject.Get()))
				{
					return LOCTEXT("OverlayMaterialSwitcherTrackTooltip", "Material switcher for overlay material");
				}
				break;
			case EComponentMaterialType::DecalMaterial:
				if (UDecalComponent* DecalComponent = Cast<UDecalComponent>(WeakObject.Get()))
				{
					return LOCTEXT("DecalMaterialSwitcherTrackTooltip", "Material switcher for decal material");
				}
				break;
			case EComponentMaterialType::SlotOverlayMaterial:
				if (UMeshComponent* Component = Cast<UMeshComponent>(WeakObject.Get()))
				{
					return !MaterialInfo.MaterialSlotName.IsNone() ?
						FText::Format(LOCTEXT("SlotOverlayMaterialSwitcherTooltip", "Material switcher for overlay slot {0} at index {1}"), FText::FromName(MaterialInfo.MaterialSlotName), FText::AsNumber(MaterialInfo.MaterialSlotIndex)) :
						FText::Format(LOCTEXT("IndexedSlotOverlayMaterialSwitcherTooltip", "Material switcher for overlay element at index {0}"), FText::AsNumber(MaterialInfo.MaterialSlotIndex));
				}
				break;
			default:
				break;
			}
		}
	}
	return FText();
}

FSlateColor UMovieScenePrimitiveMaterialTrack::GetLabelColor(const FMovieSceneLabelParams& LabelParams) const
{
	auto GetDimmedColor = [&LabelParams](FLinearColor LinearColor)
	{
		return FSlateColor(LabelParams.bIsDimmed ? LinearColor.Desaturate(0.6f) : LinearColor);
	};
	if (LabelParams.Player)
	{
		for (TWeakObjectPtr<> WeakObject : LabelParams.Player->FindBoundObjects(LabelParams.BindingID, LabelParams.SequenceID))
		{
			switch (MaterialInfo.MaterialType)
			{
			case EComponentMaterialType::Empty:
				break;
			case EComponentMaterialType::IndexedMaterial:
				if (UPrimitiveComponent* Component = Cast<UPrimitiveComponent>(WeakObject.Get()))
				{
					if (MaterialInfo.MaterialSlotName.IsNone())
					{
						// No material slot name. Can we find by index?
						if (Component->GetMaterial(MaterialInfo.MaterialSlotIndex))
						{
							// Found by index, but show yellow because missing slot name
							return GetDimmedColor(FLinearColor::Yellow);
						}
						else
						{
							// No material found at all
							return GetDimmedColor(FLinearColor::Red);
						}
					}
					else
					{
						UMaterialInterface* Material = Component->GetMaterialByName(MaterialInfo.MaterialSlotName);
						int32 FoundMaterialIndex = Component->GetMaterialIndex(MaterialInfo.MaterialSlotName);
						if (Material)
						{
							if (FoundMaterialIndex != MaterialInfo.MaterialSlotIndex)
							{
								// Indices don't match- yellow
								return GetDimmedColor(FLinearColor::Yellow);
							}
						}
						else
						{
							// Can't find material by name, try by index
							Material = Component->GetMaterial(MaterialInfo.MaterialSlotIndex);
							if (Material)
							{
								// Found, but didn't match name. Yellow
								return GetDimmedColor(FLinearColor::Yellow);
							}
							else
							{
								// Nothing found. Red.
								return GetDimmedColor(FLinearColor::Red);
							}
						}
					}
				}
				break;
			case EComponentMaterialType::OverlayMaterial:
			{
				UMeshComponent* MeshComponent = Cast<UMeshComponent>(WeakObject.Get());
				if (!MeshComponent)
				{
					return GetDimmedColor(FLinearColor::Red);
				}
			}
				break;
			case EComponentMaterialType::DecalMaterial:
			{
				UDecalComponent* DecalComponent = Cast<UDecalComponent>(WeakObject.Get());
				if (!DecalComponent)
				{
					return GetDimmedColor(FLinearColor::Red);
				}
			}
				break;
			case EComponentMaterialType::SlotOverlayMaterial:
			{
				UMeshComponent* MeshComponent = Cast<UMeshComponent>(WeakObject.Get());
				if (!MeshComponent)
				{
					return GetDimmedColor(FLinearColor::Red);
				}
			}
				break;
			default:
				break;
			}
		}
	}
	return LabelParams.bIsDimmed ? FSlateColor::UseSubduedForeground() : FSlateColor::UseForeground();
}

void UMovieScenePrimitiveMaterialTrack::PostLoad()
{
	Super::PostLoad();
	// Backwards compatibility with MaterialIndex alone as a way to reference materials.
	if (MaterialInfo.MaterialType == EComponentMaterialType::Empty)
	{
		MaterialInfo.MaterialType = EComponentMaterialType::IndexedMaterial;
		MaterialInfo.MaterialSlotIndex = MaterialIndex_DEPRECATED;
	}
}
#endif

int32 UMovieScenePrimitiveMaterialTrack::GetMaterialIndex() const
{
	return MaterialInfo.MaterialSlotIndex;
}

void UMovieScenePrimitiveMaterialTrack::SetMaterialInfo(const FComponentMaterialInfo& InMaterialInfo)
{
	MaterialInfo = InMaterialInfo;
}

const FComponentMaterialInfo& UMovieScenePrimitiveMaterialTrack::GetMaterialInfo() const
{
	return MaterialInfo;
}

void UMovieScenePrimitiveMaterialTrack::SetMaterialIndex(int32 InMaterialIndex)
{
	MaterialInfo.MaterialSlotIndex = InMaterialIndex;
	// Assumption is if this is being called by old code, we should be indexed
	MaterialInfo.MaterialType = EComponentMaterialType::IndexedMaterial;
}

#undef LOCTEXT_NAMESPACE
