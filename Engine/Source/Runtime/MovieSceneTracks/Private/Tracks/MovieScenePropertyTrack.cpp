// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieScenePropertyTrack.h"

#include "IMovieScenePlayer.h"
#include "Channels/MovieSceneSectionChannelOverrideRegistry.h"
#include "Algo/Sort.h"
#include "MovieScene.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneTracksComponentTypes.h"
#include "PropertyPathHelpers.h"
#include "Systems/MovieScenePiecewiseBoolBlenderSystem.h"
#include "ObjectEditorUtils.h"
#include "UObject/Field.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieScenePropertyTrack)

#define LOCTEXT_NAMESPACE "MovieScenePropertyTrack"

UMovieScenePropertyTrack::UMovieScenePropertyTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	EvalOptions.bEvaluateNearestSection_DEPRECATED = EvalOptions.bCanEvaluateNearestSection = true;
}


void UMovieScenePropertyTrack::SetPropertyNameAndPath(FName InPropertyName, const FString& InPropertyPath)
{
	check((InPropertyName != NAME_None) && !InPropertyPath.IsEmpty());

	PropertyBinding = FMovieScenePropertyBinding(InPropertyName, InPropertyPath);
}


const TArray<UMovieSceneSection*>& UMovieScenePropertyTrack::GetAllSections() const
{
	return Sections;
}


void UMovieScenePropertyTrack::PostLoad()
{
	Super::PostLoad();
}

void UMovieScenePropertyTrack::Serialize(FArchive& Ar)
{
	Ar.UsingCustomVersion(FMovieSceneEvaluationCustomVersion::GUID);

	Super::Serialize(Ar);
#if WITH_EDITORONLY_DATA
	if (Ar.IsLoading())
	{
		if (Ar.CustomVer(FMovieSceneEvaluationCustomVersion::GUID) < FMovieSceneEvaluationCustomVersion::EntityManager)
		{
			if (PropertyName_DEPRECATED != NAME_None && !PropertyPath_DEPRECATED.IsEmpty())
			{
				PropertyBinding = FMovieScenePropertyBinding(PropertyName_DEPRECATED, PropertyPath_DEPRECATED);
			}
		}
	}
#endif
}

#if WITH_EDITORONLY_DATA
FText UMovieScenePropertyTrack::GetDefaultDisplayName() const
{
	return FText::FromName(PropertyBinding.PropertyName);
}

FText UMovieScenePropertyTrack::GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const
{
	if (!LabelParams.BindingID.IsValid() || !LabelParams.Player)
	{
		return FText();
	}
	const TArrayView<TWeakObjectPtr<>> FoundBoundObjects = LabelParams.Player->FindBoundObjects(LabelParams.BindingID, LabelParams.SequenceID);
	for (const TWeakObjectPtr<> BoundObject : FoundBoundObjects)
	{
		if (!BoundObject.IsValid())
		{
			continue;
		}

		FTrackInstancePropertyBindings InstancePropertyBinding(GetPropertyName(), GetPropertyPath().ToString());
		if (FProperty* BoundProperty = InstancePropertyBinding.GetProperty(*BoundObject))
		{
			return FText::Format(LOCTEXT("DisplayNameTooltipFormat", "{0} \u00BB {1}\n(Path: {2})"),
				FObjectEditorUtils::GetCategoryText(BoundProperty),
				BoundProperty->GetDisplayNameText(),
				FText::FromString(InstancePropertyBinding.GetPropertyPath()));
		}
	}
	
	return FText::FromName(PropertyBinding.PropertyPath);
}

FSlateColor UMovieScenePropertyTrack::GetLabelColor(const FMovieSceneLabelParams& LabelParams) const
{
	// If there is no object binding extension, don't tint it
	if (!LabelParams.BindingID.IsValid() || !LabelParams.Player)
	{
		return LabelParams.bIsDimmed ? FSlateColor::UseSubduedForeground() : FSlateColor::UseForeground();
	}

	// Return a normal colour if we have at least one bound object for which the property binding resolves
	// correctly. Otherwise, return a red colour indicating a binding issue.
	// TODO: if the property is custom, we are going to allocate and throw away a custom handler every frame here :(
	FTrackInstancePropertyBindings InstancePropertyBinding(GetPropertyName(), GetPropertyPath().ToString());
	const TArrayView<TWeakObjectPtr<>> FoundBoundObjects = LabelParams.Player->FindBoundObjects(LabelParams.BindingID, LabelParams.SequenceID);
	for (const TWeakObjectPtr<> BoundObject : FoundBoundObjects)
	{
		if (!BoundObject.IsValid())
		{
			continue;
		}

		if (InstancePropertyBinding.HasValidBinding(*BoundObject))
		{
			return LabelParams.bIsDimmed ? FSlateColor::UseSubduedForeground() : FSlateColor::UseForeground();
		}
	}
	return LabelParams.bIsDimmed ? FSlateColor(FLinearColor::Red.Desaturate(0.6f)) : FLinearColor::Red;
}
#endif

FName UMovieScenePropertyTrack::GetTrackName() const
{
	return PropertyBinding.PropertyPath;
}

void UMovieScenePropertyTrack::RemoveAllAnimationData()
{
	Sections.Empty();
	SectionToKey = nullptr;
}


bool UMovieScenePropertyTrack::HasSection(const UMovieSceneSection& Section) const 
{
	return Sections.Contains(&Section);
}


void UMovieScenePropertyTrack::AddSection(UMovieSceneSection& Section) 
{
	Sections.Add(&Section);

	if (Sections.Num() > 1)
	{
		SetSectionToKey(&Section);
	}
}


void UMovieScenePropertyTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
	if (SectionToKey == &Section)
	{
		if (Sections.Num() > 0)
		{
			SectionToKey = Sections[0];
		}
		else
		{
			SectionToKey = nullptr;
		}
	}
}

void UMovieScenePropertyTrack::RemoveSectionAt(int32 SectionIndex)
{
	bool bResetSectionToKey = (SectionToKey == Sections[SectionIndex]);

	Sections.RemoveAt(SectionIndex);

	if (bResetSectionToKey)
	{
		SectionToKey = Sections.Num() > 0 ? Sections[0] : nullptr;
	}
}


bool UMovieScenePropertyTrack::IsEmpty() const
{
	return Sections.Num() == 0;
}

TArray<UMovieSceneSection*, TInlineAllocator<4>> UMovieScenePropertyTrack::FindAllSections(FFrameNumber Time)
{
	TArray<UMovieSceneSection*, TInlineAllocator<4>> OverlappingSections;

	for (UMovieSceneSection* Section : Sections)
	{
		if (MovieSceneHelpers::IsSectionKeyable(Section) && Section->GetRange().Contains(Time))
		{
			OverlappingSections.Add(Section);
		}
	}

	Algo::Sort(OverlappingSections, MovieSceneHelpers::SortOverlappingSections);

	return OverlappingSections;
}


UMovieSceneSection* UMovieScenePropertyTrack::FindSection(FFrameNumber Time)
{
	TArray<UMovieSceneSection*, TInlineAllocator<4>> OverlappingSections = FindAllSections(Time);

	if (OverlappingSections.Num())
	{
		if (SectionToKey && OverlappingSections.Contains(SectionToKey))
		{
			return SectionToKey;
		}
		else
		{
			return OverlappingSections[0];
		}
	}

	return nullptr;
}


UMovieSceneSection* UMovieScenePropertyTrack::FindOrExtendSection(FFrameNumber Time, float& Weight)
{
	Weight = 1.0f;
	TArray<UMovieSceneSection*, TInlineAllocator<4>> OverlappingSections = FindAllSections(Time);
	if (SectionToKey && MovieSceneHelpers::IsSectionKeyable(SectionToKey))
	{
		bool bCalculateWeight = false;
		if (!OverlappingSections.Contains(SectionToKey))
		{
			if (SectionToKey->HasEndFrame() && SectionToKey->GetExclusiveEndFrame() <= Time)
			{
				if (SectionToKey->GetExclusiveEndFrame() != Time)
				{
					SectionToKey->SetEndFrame(Time);
				}
			}
			else
			{
				SectionToKey->SetStartFrame(Time);
			}
			if (OverlappingSections.Num() > 0)
			{
				bCalculateWeight = true;
			}
		}
		else
		{
			if (OverlappingSections.Num() > 1)
			{
				bCalculateWeight = true;
			}
		}
		//we need to calculate weight also possibly
		FOptionalMovieSceneBlendType BlendType = SectionToKey->GetBlendType();
		if (bCalculateWeight)
		{
			Weight = MovieSceneHelpers::CalculateWeightForBlending(SectionToKey, Time);
		}
		return SectionToKey;
	}
	else
	{
		if (OverlappingSections.Num() > 0)
		{
			return OverlappingSections[0];
		}
	}

	// Find a spot for the section so that they are sorted by start time
	TOptional<int32> MinDiff;
	int32 ClosestSectionIndex = -1;
	bool bStartFrame = false;
	for(int32 SectionIndex = 0; SectionIndex < Sections.Num(); ++SectionIndex)
	{
		UMovieSceneSection* Section = Sections[SectionIndex];

		if (Section->HasStartFrame())
		{
			int32 Diff = FMath::Abs(Time.Value - Section->GetInclusiveStartFrame().Value);

			if (!MinDiff.IsSet())
			{
				MinDiff = Diff;
				ClosestSectionIndex = SectionIndex;
				bStartFrame = true;
			}
			else if (Diff < MinDiff.GetValue())
			{
				MinDiff = Diff;
				ClosestSectionIndex = SectionIndex;
				bStartFrame = true;
			}
		}

		if (Section->HasEndFrame())
		{
			int32 Diff = FMath::Abs(Time.Value - Section->GetExclusiveEndFrame().Value);

			if (!MinDiff.IsSet())
			{
				MinDiff = Diff;
				ClosestSectionIndex = SectionIndex;
				bStartFrame = false;
			}
			else if (Diff < MinDiff.GetValue())
			{
				MinDiff = Diff;
				ClosestSectionIndex = SectionIndex;
				bStartFrame = false;
			}
		}
	}

	if (ClosestSectionIndex != -1)
	{
		UMovieSceneSection* ClosestSection = Sections[ClosestSectionIndex];
		if (bStartFrame)
		{
			ClosestSection->SetStartFrame(Time);
		}
		else
		{
			ClosestSection->SetEndFrame(Time);
		}

		return ClosestSection;
	}

	return nullptr;
}

UMovieSceneSection* UMovieScenePropertyTrack::FindOrAddSection(FFrameNumber Time, bool& bSectionAdded)
{
	bSectionAdded = false;

	UMovieSceneSection* FoundSection = FindSection(Time);
	if (FoundSection)
	{
		return FoundSection;
	}

	// Add a new section that starts and ends at the same time
	UMovieSceneSection* NewSection = CreateNewSection();
	ensureAlwaysMsgf(NewSection->HasAnyFlags(RF_Transactional), TEXT("CreateNewSection must return an instance with RF_Transactional set! (pass RF_Transactional to NewObject)"));
	NewSection->SetFlags(RF_Transactional);
	NewSection->SetRange(TRange<FFrameNumber>::Inclusive(Time, Time));

	UMovieSceneTrack::AddSection(FSectionParameter(*NewSection));
	
	bSectionAdded = true;

	return NewSection;
}

void UMovieScenePropertyTrack::SetSectionToKey(UMovieSceneSection* InSection)
{
	SectionToKey = InSection;
}

UMovieSceneSection* UMovieScenePropertyTrack::GetSectionToKey() const
{
	return SectionToKey;
}

const int32 FMovieScenePropertyTrackEntityImportHelper::SectionPropertyValueImportingID = 0;
const int32 FMovieScenePropertyTrackEntityImportHelper::SectionEditConditionToggleImportingID = 1;

void FMovieScenePropertyTrackEntityImportHelper::PopulateEvaluationField(UMovieSceneSection& Section, const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder)
{
	using namespace UE::MovieScene;

	// Parse the property path, and get the expected object type for the property binding, if any.
	TArray<FString> PropertyPathSegments;
#if WITH_EDITORONLY_DATA
	const UClass* ParentBoundClass = nullptr;
#endif

	UMovieScenePropertyTrack* PropertyTrack = Section.GetTypedOuter<UMovieScenePropertyTrack>();
	UMovieScene* MovieScene = Section.GetTypedOuter<UMovieScene>();
	if (PropertyTrack && MovieScene)
	{
		const FMovieScenePropertyBinding& PropertyBinding = PropertyTrack->GetPropertyBinding();
		PropertyBinding.PropertyPath.ToString().ParseIntoArray(PropertyPathSegments, TEXT("."), true);

#if WITH_EDITORONLY_DATA
		FGuid ParentBindingGuid = OutFieldBuilder->GetSharedMetaData().ObjectBindingID;
		if (ParentBindingGuid.IsValid())
		{
			if (const FMovieScenePossessable* ParentPossessable = MovieScene->FindPossessable(ParentBindingGuid))
			{
				ParentBoundClass = ParentPossessable->GetPossessedObjectClass();
			}
			else if (const FMovieSceneSpawnable* ParentSpawnable = MovieScene->FindSpawnable(ParentBindingGuid))
			{
				ParentBoundClass = ParentSpawnable->GetObjectTemplate() ? 
					ParentSpawnable->GetObjectTemplate()->GetClass() : nullptr;
			}
		}
#endif
	}

	FMovieSceneEvaluationFieldEntityMetaData MainMetaData(InMetaData);

#if WITH_EDITORONLY_DATA

	// Check if this section is animating a property with a notify function. If so, add that information to the metadata.
	if (ParentBoundClass && PropertyPathSegments.Num() > 0)
	{
		if (FProperty* HeadProperty = ParentBoundClass->FindPropertyByName(*PropertyPathSegments[0]))
		{
			const FString InterpNotifyMetaData = HeadProperty->GetMetaData(TEXT("InterpNotify"));
			if (!InterpNotifyMetaData.IsEmpty())
			{
				MainMetaData.NotifyFunctionName = FName(InterpNotifyMetaData);
			}
		}
	}

#else

	// No support for notify functions for sequences dynamically built at runtime.

#endif  // WITH_EDITORONLY_DATA

	int32 NumOverridenChannels = 0;
	IMovieSceneChannelOverrideProvider* RegistryProvider = Cast<IMovieSceneChannelOverrideProvider>(&Section);
	if (RegistryProvider)
	{
		if (UMovieSceneSectionChannelOverrideRegistry* OverrideRegistry = RegistryProvider->GetChannelOverrideRegistry(false))
		{
			NumOverridenChannels = OverrideRegistry->NumChannels();
			OverrideRegistry->PopulateEvaluationFieldImpl(EffectiveRange, MainMetaData, OutFieldBuilder, Section);
		}
	}

	const int32 NumChannels = Section.GetChannelProxy().NumChannels();
	if (NumChannels > NumOverridenChannels)
	{
		// Add the default entity for this section.
		const int32 EntityIndex = OutFieldBuilder->FindOrAddEntity(&Section, SectionPropertyValueImportingID);
		const int32 MetaDataIndex = OutFieldBuilder->AddMetaData(MainMetaData);
		OutFieldBuilder->AddPersistentEntity(EffectiveRange, EntityIndex, MetaDataIndex);
	}

	// Check if this section is animating a property with an edit-condition. If so, we need to also animate a boolean toggle
	// that will be set to true while the main property is animated.
	if (PropertyPathSegments.Num() > 0)
	{
		// Prepare the edit condition toggle property path by taking the beginning part of the
		// main property path. We'll append the toggle name to it. This is necessary for nested stuff, like:
		//
		// (this.)PostProcessSettings.bOverride_FooBar 
		//
		// ...where the main property was:
		//
		// (this.)PostProcessSettings.FooBar
		//
		FString EditConditionPropertyPath = FString::Join(
				MakeArrayView(PropertyPathSegments.GetData(), PropertyPathSegments.Num() - 1),
				TEXT("."));
		bool bHasEditCondition = false;

#if WITH_EDITORONLY_DATA

		if (ParentBoundClass)
		{
			FCachedPropertyPath PropertyPath(PropertyPathSegments);
			PropertyPath.Resolve(ParentBoundClass->GetDefaultObject());
			if (const FProperty* LeafProperty = PropertyPath.GetFProperty())
			{
				const FString EditConditionPropertyName = LeafProperty->GetMetaData("EditCondition");
				if (!EditConditionPropertyName.IsEmpty())
				{
					if (!EditConditionPropertyPath.IsEmpty())
					{
						EditConditionPropertyPath.Append(".");
					}
					EditConditionPropertyPath.Append(EditConditionPropertyName);
					bHasEditCondition = true;
				}
			}
		}

#else

		// HACK: We don't have the metadata info in non-editor builds, so we need to hard-code some well-known
		//       stuff that uses edit-conditions... until we find a better solution.
		//       For now, this is only for PostProcessSettings properties.
		if (PropertyPathSegments.Num() >= 2 && PropertyPathSegments[PropertyPathSegments.Num() - 2] == "PostProcessSettings")
		{
			FString EditConditionPropertyName = PropertyPathSegments[PropertyPathSegments.Num() - 1];
			EditConditionPropertyName.InsertAt(0, TEXT("bOverride_"));

			if (!EditConditionPropertyPath.IsEmpty())
			{
				EditConditionPropertyPath.Append(".");
			}
			EditConditionPropertyPath.Append(EditConditionPropertyName);
			bHasEditCondition = true;
		}

#endif  // WITH_EDITORONLY_DATA

		if (bHasEditCondition)
		{
			FMovieSceneEvaluationFieldEntityMetaData OverrideToggleMetaData(InMetaData);
			OverrideToggleMetaData.OverrideBoundPropertyPath = EditConditionPropertyPath;

			const int32 OverrideToggleEntityIndex = OutFieldBuilder->FindOrAddEntity(&Section, SectionEditConditionToggleImportingID);
			const int32 OverrideToggleMetaDataIndex = OutFieldBuilder->AddMetaData(OverrideToggleMetaData);
			OutFieldBuilder->AddPersistentEntity(EffectiveRange, OverrideToggleEntityIndex, OverrideToggleMetaDataIndex);
		}

	}
}

bool FMovieScenePropertyTrackEntityImportHelper::IsPropertyValueID(const UE::MovieScene::FEntityImportParams& Params)
{
	return Params.EntityID == SectionPropertyValueImportingID;
}

bool FMovieScenePropertyTrackEntityImportHelper::IsEditConditionToggleID(const UE::MovieScene::FEntityImportParams& Params)
{
	return Params.EntityID == SectionEditConditionToggleImportingID;
}

void FMovieScenePropertyTrackEntityImportHelper::ImportEditConditionToggleEntity(const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity)
{
	using namespace UE::MovieScene;

	if (!ensure(Params.EntityMetaData))
	{
		return;
	}

	const FString& OverrideBoundPropertyPath = Params.EntityMetaData->OverrideBoundPropertyPath;
	if (!ensure(!OverrideBoundPropertyPath.IsEmpty()))
	{
		return;
	}

	// TODO: The interrogation property instantiator doesn't support multiple unrelated entities for a given interrogation so let's not add the bool override setter.
	if (Params.InterrogationKey.IsValid())
	{
		return;
	}

	int32 LastDotIndex = INDEX_NONE;
	OverrideBoundPropertyPath.FindLastChar('.', LastDotIndex);
	const FName OverrideBoundPropertyName = SanitizeBoolPropertyName(
			(LastDotIndex != INDEX_NONE) ?
				FName(OverrideBoundPropertyPath.RightChop(LastDotIndex + 1)) :
				FName(OverrideBoundPropertyPath)
			);

	const FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
	const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

	const FGuid ObjectBindingID = Params.GetObjectBindingID();
	const FMovieScenePropertyBinding OverrideTogglePropertyBinding(OverrideBoundPropertyName, OverrideBoundPropertyPath);

	OutImportedEntity->AddBuilder(
			FEntityBuilder()
			.Add(Components->BoolResult, true)
			.Add(Components->BlenderType, UMovieScenePiecewiseBoolBlenderSystem::StaticClass())
			.Add(Components->PropertyBinding, OverrideTogglePropertyBinding)
			.AddConditional(Components->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
			.AddTag(TracksComponents->Bool.PropertyTag)
			);
}

FName FMovieScenePropertyTrackEntityImportHelper::SanitizeBoolPropertyName(FName InPropertyName)
{
	FString PropertyVarName = InPropertyName.ToString();
	PropertyVarName.RemoveFromStart("b", ESearchCase::CaseSensitive);
	return FName(*PropertyVarName);
}

#undef LOCTEXT_NAMESPACE
