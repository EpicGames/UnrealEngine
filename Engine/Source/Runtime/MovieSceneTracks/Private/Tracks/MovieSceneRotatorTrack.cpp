// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneRotatorTrack.h"

#include "Evaluation/Blending/MovieSceneBlendType.h"
#include "Sections/MovieSceneRotatorSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneRotatorTrack)

UMovieSceneRotatorTrack::UMovieSceneRotatorTrack(const FObjectInitializer& InObjectInitializer)
	: Super(InObjectInitializer)
{
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}

bool UMovieSceneRotatorTrack::SupportsType(TSubclassOf<UMovieSceneSection> InSectionClass) const
{
	return InSectionClass == UMovieSceneRotatorSection::StaticClass();
}

UMovieSceneSection* UMovieSceneRotatorTrack::CreateNewSection()
{
	return NewObject<UMovieSceneRotatorSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneRotatorTrack::AddSection(UMovieSceneSection& InSection)
{
	Sections.Add(&InSection);
}

const TArray<UMovieSceneSection*>& UMovieSceneRotatorTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneRotatorTrack::HasSection(const UMovieSceneSection& InSection) const
{
	return Sections.Contains(&InSection);
}

bool UMovieSceneRotatorTrack::IsEmpty() const
{
	return Sections.IsEmpty();
}

void UMovieSceneRotatorTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

void UMovieSceneRotatorTrack::RemoveSection(UMovieSceneSection& InSection)
{
	Sections.Remove(&InSection);
}

void UMovieSceneRotatorTrack::RemoveSectionAt(int32 InSectionIndex)
{
	Sections.RemoveAt(InSectionIndex);
}

