// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneSkeletalAnimationTrack.h"
#include "Evaluation/MovieSceneEvaluationCustomVersion.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Compilation/MovieSceneCompilerRules.h"
#include "Components/SkeletalMeshComponent.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Compilation/MovieSceneEvaluationTreePopulationRules.h"
#include "MovieScene.h"
#include "AnimationRuntime.h"
#include "Animation/AnimInstance.h"
#include "Animation/AnimSequence.h"
#include "Animation/AnimationPoseData.h"
#include "Animation/AttributesRuntime.h"
#include "SkeletalDebugRendering.h"
#include "Rendering/SkeletalMeshRenderData.h"
#include "BoneContainer.h"
#include "AnimSequencerInstanceProxy.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSkeletalAnimationTrack)

#if WITH_EDITORONLY_DATA
#include "AnimationBlueprintLibrary.h"
#include "Misc/QualifiedFrameTime.h"
#include "Misc/Timecode.h"
#endif

#define LOCTEXT_NAMESPACE "MovieSceneSkeletalAnimationTrack"

FColor UMovieSceneSkeletalAnimationTrack::LinkedAnimSeqTrackColor = FColor(60, 4, 46);

/* UMovieSceneSkeletalAnimationTrack structors
 *****************************************************************************/

UMovieSceneSkeletalAnimationTrack::UMovieSceneSkeletalAnimationTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
	, bUseLegacySectionIndexBlend(false)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(124, 15, 124, 65);
	bSupportsDefaultSections = false;
#endif

	SupportedBlendTypes.Add(EMovieSceneBlendType::Absolute);

	EvalOptions.bEvaluateNearestSection_DEPRECATED = EvalOptions.bCanEvaluateNearestSection = true;
	SwapRootBone = ESwapRootBone::SwapRootBone_None;
}


/* UMovieSceneSkeletalAnimationTrack interface
 *****************************************************************************/

TArray<UMovieSceneSection*> UMovieSceneSkeletalAnimationTrack::GetAnimSectionsAtTime(FFrameNumber Time)
{
	TArray<UMovieSceneSection*> Sections;
	for (UMovieSceneSection* Section : AnimationSections)
	{
		if (Section->IsTimeWithinSection(Time))
		{
			Sections.Add(Section);
		}
	}

	return Sections;
}

bool UMovieSceneSkeletalAnimationTrack::ShouldUseRootMotions() const
{
	// Turn on root motion caching if any swap-root-bone target is active, either at the track
	// level (right-click menu path) or on any individual section (Details-panel path). Without
	// the per-section check, a Details-panel edit leaves the track-level RootMotionParams cache
	// empty, so the root bone is extracted from the pose but no motion is applied to the actor.
	if (SwapRootBone != ESwapRootBone::SwapRootBone_None)
	{
		return true;
	}
	for (const UMovieSceneSection* Section : AnimationSections)
	{
		if (const UMovieSceneSkeletalAnimationSection* AnimSection = Cast<const UMovieSceneSkeletalAnimationSection>(Section))
		{
			if (AnimSection->Params.SwapRootBone != ESwapRootBone::SwapRootBone_None)
			{
				return true;
			}
		}
	}
	return false;
}

/* UMovieSceneTrack interface
 *****************************************************************************/

void UMovieSceneSkeletalAnimationTrack::PostLoad()
{
	// UMovieSceneTrack::PostLoad removes null sections. However, RemoveAtSection requires SetupRootMotions, which accesses AnimationSections, so remove null sections here before anything else 
	TOptional<ESwapRootBone> SectionsSwapRootBone;
	bool bAllSectionsSameSwapRootBone = false;
	for (int32 SectionIndex = 0; SectionIndex < AnimationSections.Num(); )
	{
		UMovieSceneSection* Section = AnimationSections[SectionIndex];

		if (Section == nullptr)
		{
#if WITH_EDITOR
			UE_LOGF(LogMovieScene, Warning, "Removing null section from %ls:%ls", *GetPathName(), *GetDisplayName().ToString());
#endif
			AnimationSections.RemoveAt(SectionIndex);
		}
		else if (Section->GetRange().IsEmpty())
		{
#if WITH_EDITOR
			//UE_LOGF(LogMovieScene, Warning, "Removing section %ls:%ls with empty range", *GetPathName(), *GetDisplayName().ToString());
#endif
			AnimationSections.RemoveAt(SectionIndex);
		}
		else
		{
			++SectionIndex;
			if (UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section))
			{
				if (SectionsSwapRootBone.IsSet())
				{
					if (SectionsSwapRootBone.GetValue() != AnimSection->Params.SwapRootBone)
					{
						bAllSectionsSameSwapRootBone = false;
					}
				}
				else
				{
					SectionsSwapRootBone = AnimSection->Params.SwapRootBone;
					bAllSectionsSameSwapRootBone = true;
				}
			}
		}
	}
	//if we have all sections with the same swap root bone, set that, probably from a previous version
	if (bAllSectionsSameSwapRootBone && SectionsSwapRootBone.IsSet())
	{
		SwapRootBone = SectionsSwapRootBone.GetValue();
	}

	Super::PostLoad();

	if (GetLinkerCustomVersion(FMovieSceneEvaluationCustomVersion::GUID) < FMovieSceneEvaluationCustomVersion::AddBlendingSupport)
	{
		bUseLegacySectionIndexBlend = true;
	}
}

void UMovieSceneSkeletalAnimationTrack::SetSwapRootBone(ESwapRootBone InValue)
{
	SwapRootBone = InValue;
	for (UMovieSceneSection* Section : AnimationSections)
	{
		UMovieSceneSkeletalAnimationSection* AnimSection = Cast<UMovieSceneSkeletalAnimationSection>(Section);
		if (AnimSection)
		{
			AnimSection->Params.SwapRootBone = SwapRootBone;
		}
	}
	RootMotionParams.bRootMotionsDirty = true;
}

ESwapRootBone UMovieSceneSkeletalAnimationTrack::GetSwapRootBone() const
{
	return SwapRootBone;
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneSkeletalAnimationTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Animation");
}

#endif

bool UMovieSceneSkeletalAnimationTrack::PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const
{
	using namespace UE::MovieScene;

	if (!bUseLegacySectionIndexBlend)
	{
		FEvaluationTreePopulationRules::HighPassPerRow(AnimationSections, OutData);
	}
	else
	{
		// Use legacy blending... when there's overlapping, the section that makes it into the evaluation tree is
		// the one that appears later in the container arary of section data.
		//
		auto SortByLatestInArrayAndRow = [](const FEvaluationTreePopulationRules::FSortedSection& A, const FEvaluationTreePopulationRules::FSortedSection& B)
		{
			if (A.Row() == B.Row())
			{
				return A.Index > B.Index;
			}
			
			return A.Row() < B.Row();
		};

		UE::MovieScene::FEvaluationTreePopulationRules::HighPassCustomPerRow(AnimationSections, OutData, SortByLatestInArrayAndRow);
	}
	return true;
}

#undef LOCTEXT_NAMESPACE

