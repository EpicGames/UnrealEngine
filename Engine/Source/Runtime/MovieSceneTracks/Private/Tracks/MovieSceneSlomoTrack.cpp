// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneSlomoTrack.h"
#include "Sections/MovieSceneSlomoSection.h"
#include "Evaluation/MovieSceneEvaluationTrack.h"
#include "Templates/Casts.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneSlomoTrack)

#define LOCTEXT_NAMESPACE "MovieSceneSlomoTrack"


/* UMovieSceneEventTrack interface
 *****************************************************************************/
UMovieSceneSlomoTrack::UMovieSceneSlomoTrack(const FObjectInitializer& Init)
	: Super(Init)
{
	EvalOptions.bCanEvaluateNearestSection = true;
}

bool UMovieSceneSlomoTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneSlomoSection::StaticClass();
}

UMovieSceneSection* UMovieSceneSlomoTrack::CreateNewSection()
{
	return NewObject<UMovieSceneSlomoSection>(this, NAME_None, RF_Transactional);
}

#if WITH_EDITORONLY_DATA

FText UMovieSceneSlomoTrack::GetDefaultDisplayName() const
{
	return LOCTEXT("TrackName", "Time Dilation");
}

#endif

#undef LOCTEXT_NAMESPACE

