// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneStringTrack.h"
#include "MovieSceneCommonHelpers.h"
#include "Sections/MovieSceneStringSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneStringTrack)


#define LOCTEXT_NAMESPACE "MovieSceneStringTrack"


/* UMovieSceneTrack interface
 *****************************************************************************/

void UMovieSceneStringTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}


bool UMovieSceneStringTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneStringSection::StaticClass();
}


UMovieSceneSection* UMovieSceneStringTrack::CreateNewSection()
{
	return NewObject<UMovieSceneStringSection>(this, NAME_None, RF_Transactional);
}


const TArray<UMovieSceneSection*>& UMovieSceneStringTrack::GetAllSections() const
{
	return Sections;
}


bool UMovieSceneStringTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}


bool UMovieSceneStringTrack::IsEmpty() const
{
	return (Sections.Num() == 0);
}


void UMovieSceneStringTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}


void UMovieSceneStringTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}


void UMovieSceneStringTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}

#undef LOCTEXT_NAMESPACE

