// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneTextTrack.h"
#include "Sections/MovieSceneTextSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTextTrack)

UMovieSceneTextTrack::UMovieSceneTextTrack()
{
}

bool UMovieSceneTextTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneTextSection::StaticClass();
}

UMovieSceneSection* UMovieSceneTextTrack::CreateNewSection()
{
	return NewObject<UMovieSceneTextSection>(this, NAME_None, RF_Transactional);
}

void UMovieSceneTextTrack::AddSection(UMovieSceneSection& Section)
{
	Sections.Add(&Section);
}

const TArray<UMovieSceneSection*>& UMovieSceneTextTrack::GetAllSections() const
{
	return Sections;
}

bool UMovieSceneTextTrack::HasSection(const UMovieSceneSection& Section) const
{
	return Sections.Contains(&Section);
}

bool UMovieSceneTextTrack::IsEmpty() const
{
	return Sections.IsEmpty();
}

void UMovieSceneTextTrack::RemoveAllAnimationData()
{
	Sections.Empty();
}

void UMovieSceneTextTrack::RemoveSection(UMovieSceneSection& Section)
{
	Sections.Remove(&Section);
}

void UMovieSceneTextTrack::RemoveSectionAt(int32 SectionIndex)
{
	Sections.RemoveAt(SectionIndex);
}
