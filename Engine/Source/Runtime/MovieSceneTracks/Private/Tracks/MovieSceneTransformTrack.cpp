// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneTransformTrack.h"
#include "Sections/MovieScene3DTransformSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneTransformTrack)


UMovieSceneTransformTrack::UMovieSceneTransformTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
#if WITH_EDITORONLY_DATA
	TrackTint = FColor(65, 173, 164, 65);
#endif

	SupportedBlendTypes = FMovieSceneBlendTypeField::All();

	EvalOptions.bEvaluateNearestSection_DEPRECATED = EvalOptions.bCanEvaluateNearestSection = true;
}

bool UMovieSceneTransformTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieScene3DTransformSection::StaticClass();
}

UMovieSceneSection* UMovieSceneTransformTrack::CreateNewSection()
{
	return NewObject<UMovieScene3DTransformSection>(this, NAME_None, RF_Transactional);
}

