// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneVectorTrack.h"
#include "Sections/MovieSceneVectorSection.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneVectorTrack)


UMovieSceneFloatVectorTrack::UMovieSceneFloatVectorTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	NumChannelsUsed = 0;
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}


bool UMovieSceneFloatVectorTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneFloatVectorSection::StaticClass();
}

UMovieSceneSection* UMovieSceneFloatVectorTrack::CreateNewSection()
{
	UMovieSceneFloatVectorSection* NewSection = NewObject<UMovieSceneFloatVectorSection>(this, NAME_None, RF_Transactional);
	NewSection->SetChannelsUsed(NumChannelsUsed);
	return NewSection;
}

void UMovieSceneFloatVectorTrack::InitializeFromProperty(const FProperty* Property, const UE::MovieScene::FPropertyDefinition* Definition)
{
	const FStructProperty* StructProp = CastField<const FStructProperty>(Property);

	FName StructName = StructProp ? StructProp->Struct->GetFName() : NAME_None;

	if (StructName == NAME_Vector2f)
	{
		SetNumChannelsUsed(2);
	}
	else if (StructName == NAME_Vector3f)
	{
		SetNumChannelsUsed(3);
	}
	else if (StructName == NAME_Vector4f)
	{
		SetNumChannelsUsed(4);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("Attempting to create a vector track with an unsupported property type"));
	}
}


UMovieSceneDoubleVectorTrack::UMovieSceneDoubleVectorTrack( const FObjectInitializer& ObjectInitializer )
	: Super( ObjectInitializer )
{
	NumChannelsUsed = 0;
	SupportedBlendTypes = FMovieSceneBlendTypeField::All();
}


bool UMovieSceneDoubleVectorTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneDoubleVectorSection::StaticClass();
}

UMovieSceneSection* UMovieSceneDoubleVectorTrack::CreateNewSection()
{
	UMovieSceneDoubleVectorSection* NewSection = NewObject<UMovieSceneDoubleVectorSection>(this, NAME_None, RF_Transactional);
	NewSection->SetChannelsUsed(NumChannelsUsed);
	return NewSection;
}

void UMovieSceneDoubleVectorTrack::InitializeFromProperty(const FProperty* Property, const UE::MovieScene::FPropertyDefinition* Definition)
{
	const FStructProperty* StructProp = CastField<const FStructProperty>(Property);

	FName StructName = StructProp ? StructProp->Struct->GetFName() : NAME_None;
	
	if (StructName == NAME_Vector2D)
	{
		SetNumChannelsUsed(2);
	}
	else if (StructName == NAME_Vector3d || StructName == NAME_Vector)
	{
		SetNumChannelsUsed(3);
	}	
	else if (StructName == NAME_Vector4d || StructName == NAME_Vector4)
	{
		SetNumChannelsUsed(4);
	}
	else
	{
		ensureAlwaysMsgf(false, TEXT("Attempting to create a vector track with an unsupported property type"));
	}
}