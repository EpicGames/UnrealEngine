// Copyright Epic Games, Inc. All Rights Reserved.

#include "Tracks/MovieSceneVisibilityTrack.h"

#include "EntitySystem/MovieSceneIntermediatePropertyValue.h"
#include "Sections/MovieSceneBoolSection.h"
#include "Sections/MovieSceneVisibilitySection.h"
#include "Serialization/ObjectReader.h"
#include "Serialization/ObjectWriter.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(MovieSceneVisibilityTrack)

#define LOCTEXT_NAMESPACE "MovieSceneVisibilityTrack"

UMovieSceneVisibilityTrack::UMovieSceneVisibilityTrack(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
}

void UMovieSceneVisibilityTrack::PostLoad()
{
	// Upgrade bool sections to visibility sections.
	TArray<uint8> Bytes;
	bool bUpgraded = false;

	for (int32 Index = 0; Index < Sections.Num(); ++Index)
	{
		UMovieSceneBoolSection* BoolSection = ExactCast<UMovieSceneBoolSection>(Sections[Index]);
		if (BoolSection)
		{
			BoolSection->ConditionalPostLoad();
			Bytes.Reset();

			FObjectWriter(BoolSection, Bytes);
			UMovieSceneVisibilitySection* NewSection = NewObject<UMovieSceneVisibilitySection>(this, NAME_None, RF_Transactional);
			// Bool sections start with DefaultValue=false and bHasDefaultValue=false, so we need
			// to match this in order for the delta-serialization to do the right thing.
			NewSection->GetChannel().SetDefault(false);
			NewSection->GetChannel().RemoveDefault();
			FObjectReader(NewSection, Bytes);

			Sections[Index] = NewSection;
			bUpgraded = true;
		}
	}

	if (bUpgraded)
	{
		ForceUpdateEvaluationTree();
	}

	Super::PostLoad();
}

void UMovieSceneVisibilityTrack::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	// Preload BoolSections for PostLoad upgrade if necessary
	if (Ar.IsLoading())
	{
		for (int32 Index = 0; Index < Sections.Num(); ++Index)
		{
			UMovieSceneBoolSection* BoolSection = ExactCast<UMovieSceneBoolSection>(Sections[Index]);
			if (BoolSection)
			{
				Ar.Preload(BoolSection);		
			}
		}
	}
}

bool UMovieSceneVisibilityTrack::SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const
{
	return SectionClass == UMovieSceneVisibilitySection::StaticClass();
}

UMovieSceneSection* UMovieSceneVisibilityTrack::CreateNewSection()
{
	return NewObject<UMovieSceneVisibilitySection>(this, NAME_None, RF_Transactional);
}

#if WITH_EDITOR

void UMovieSceneVisibilityTrack::ProcessSourceValueForKeying(const FProperty* Property, UE::MovieScene::FSourcePropertyValue& InOutValue) const
{
	// Bound bool properties (e.g. AActor::bHidden, USceneComponent::bHiddenInGame) store hidden semantics,
	// but the visibility section's bool channel stores visible semantics (see UMovieSceneVisibilitySystem).
	// Invert here so the keyed value matches the channel's semantics.
	if (Property && CastField<FBoolProperty>(Property))
	{
		if (const bool* CurrentValue = InOutValue.Cast<bool>())
		{
			InOutValue = UE::MovieScene::FSourcePropertyValue::FromValue(!*CurrentValue);
		}
	}
}

#endif

#if WITH_EDITORONLY_DATA

FText UMovieSceneVisibilityTrack::GetDisplayName() const
{
	return LOCTEXT("DisplayName", "Visibility");
}

#endif

#undef LOCTEXT_NAMESPACE

