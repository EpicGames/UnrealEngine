// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MovieSceneReplaceableBinding.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "MovieSceneReplaceableActorBinding.generated.h"

/*
* An implementation of UMovieSceneReplaceableBindingBase that uses UMovieSceneSpawnableActorBinding as the preview spawnable,
* and has no implementation of ResolveRuntimeBindingInternal, relying instead of Sequencer's built in BindingOverride mechanism for binding at runtime.
*/
UCLASS(MinimalAPI, BlueprintType, EditInlineNew, DefaultToInstanced, Meta=(DisplayName="Replaceable Actor"))
class UMovieSceneReplaceableActorBinding
	: public UMovieSceneReplaceableBindingBase
{
public:

	GENERATED_BODY()

public:

	/* MovieSceneCustomBinding overrides*/
	/* Note that we specifically don't implement CreateCustomBinding here- it's implemented in the base class and separately calls
	 *	CreateInnerSpawnable and InitReplaceableBinding which we implement here (though InitReplaceableBinding has an empty implementation in this class). 
	 */
#if WITH_EDITOR
	FText GetBindingTypePrettyName() const override;
#endif

protected:
	/* MovieSceneReplaceableBindingBase overrides*/

	// By default we return nullptr here, as we rely on Sequencer's BindingOverride mechanism to bind these actors during runtime.
	// This can be overridden if desired in subclasses to provide a different way to resolve to an actor at runtime while still using spawnable actor as the preview.
	virtual FMovieSceneBindingResolveResult ResolveRuntimeBindingInternal(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override { return FMovieSceneBindingResolveResult(); }
	
	// Empty implementation by default as we don't need to initialize any data members other than the spawnable,which is initialized by CreateInnerSpawnable above
	virtual void InitReplaceableBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene) override {}

	MOVIESCENETRACKS_API virtual TSubclassOf<UMovieSceneSpawnableBindingBase> GetInnerSpawnableClass() const override;

	virtual int32 GetCustomBindingPriority() const override { return BaseEnginePriority + 1; }

};

/* Base class for Custom Replaceable Binding classes implemented by Blueprints */

UCLASS(Abstract, Blueprintable, MinimalAPI, EditInlineNew, DefaultToInstanced, Meta=(ShowWorldContextPin, DisplayName="Replaceable Blueprint Base"))
class UMovieSceneReplaceableActorBinding_BPBase
	: public UMovieSceneReplaceableBindingBase
{
public:

	GENERATED_BODY()

public:

	/* MovieSceneCustomBinding overrides*/
#if WITH_EDITOR
	FText GetBindingTypePrettyName() const override final;
	virtual FText GetBindingTrackIconTooltip() const override final;

	void OnBindingAddedOrChanged(UMovieScene& OwnerMovieScene) override final;
#endif

#if WITH_EDITORONLY_DATA
	
	/* Name to show in Sequencer for the custom binding type.*/
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Sequencer")
	FText BindingTypePrettyName;

	/* Tooltip to show in Sequencer for the custom binding type.*/
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Sequencer")
	FText BindingTypeTooltip;
#endif
	
	/*
	Priority with which to consider this binding type over others when considering binding an object to Sequencer.
	As a guideline, a priority of BaseEnginePriority will ensure that engine types(such as Spawnable Actor, Replaceable Actor) will
	be higher priority than your custom binding, and so your binding type will not automatically be created(but may be converted to manually).
	A priority of BaseCustomPriority and higher will ensure that your binding type is considered more highly than engine types,
	so if your binding type's 'SupportsBindingCreationFromObject' returns true for an object, your binding type will be created by default
	rather than an engine type.
	*/
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category = "Sequencer")
	int32 CustomBindingPriority = BaseEnginePriority;

	// Preview Spawnable Type to use for this replaceable
	UPROPERTY(EditDefaultsOnly, BlueprintReadOnly, Category="Preview", meta=(AllowAbstract=false))
	TSubclassOf<UMovieSceneSpawnableBindingBase> PreviewSpawnableType;

	int32 GetCustomBindingPriority() const override final { return CustomBindingPriority; }	/* Blueprint Interface */

	/*
	* Must be implemented. Called during non-editor/runtime to resolve the binding dynamically. In editor worlds/Sequencer will instead use the PreviewSpawnable binding to spawn a preview object.
	* If no object is returned, Sequencer's BindingOverrides can still be used to dynamically bind the object.
	*/
	UFUNCTION(BlueprintImplementableEvent, CallInEditor, meta=(DisplayName = "Resolve Runtime Binding"))
	FMovieSceneBindingResolveResult BP_ResolveRuntimeBinding(const FMovieSceneBindingResolveContext& ResolveContext) const;

	/* Called after binding creation to allow the replaceable to initialize any data members from the source object. */
	UFUNCTION(BlueprintNativeEvent, CallInEditor, meta=(DisplayName = "Init Replaceable Binding"))
	void BP_InitReplaceableBinding(UObject* SourceObject, UMovieScene* OwnerMovieScene);

	/* Called on the binding to determine whether this binding type supports creating a binding from the passed in object. */
	UFUNCTION(BlueprintNativeEvent, CallInEditor, meta=(DisplayName = "Supports Binding Creation From Object"))
	bool BP_SupportsBindingCreationFromObject(const UObject* SourceObject) const;

protected:
	/* MovieSceneReplaceableBindingBase overrides*/

#if WITH_EDITOR
	UMovieSceneCustomBinding* CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene) override final;
#endif

	// By default we return nullptr here, as we rely on Sequencer's BindingOverride mechanism to bind these actors during runtime.
	// This can be overridden if desired in subclasses to provide a different way to resolve to an actor at runtime while still using spawnable actor as the preview.
	FMovieSceneBindingResolveResult ResolveRuntimeBindingInternal(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;
	
	void InitReplaceableBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene) override final;

	TSubclassOf<UMovieSceneSpawnableBindingBase> GetInnerSpawnableClass() const override final { return PreviewSpawnableType; }
	
	bool SupportsBindingCreationFromObject(const UObject* SourceObject) const override final;


};
