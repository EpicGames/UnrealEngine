// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MovieSceneReplaceableBinding.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "Bindings/MovieSceneSpawnableBinding.h"
#include "MovieSceneReplaceableDirectorBlueprintBinding.generated.h"

/**
 Custom binding type that uses a Director Blueprint endpoint to allow the user to define at runtime how to resolve this binding.
 User can use any desired custom spawnable type as the preview within Sequencer, such as a MovieSceneSpawnableDirectorBlueprintBinding for another endpoint
 for spawning, or a MovieSceneSpawnableActorBinding to spawn from an actor template.
 */

UCLASS(BlueprintType, MinimalAPI, EditInlineNew, DefaultToInstanced, Meta=(DisplayName="Replaceable from Director Blueprint"))
class UMovieSceneReplaceableDirectorBlueprintBinding
	: public UMovieSceneReplaceableBindingBase
{
public:

	GENERATED_BODY()

public:

	/* MovieSceneCustomBinding overrides*/
#if WITH_EDITOR
	FText GetBindingTypePrettyName() const override;
	void OnBindingAddedOrChanged(UMovieScene& OwnerMovieScene) override;
#endif	
	
	// Director Blueprint defined binding info
	UPROPERTY(EditAnywhere, Category="Sequencer")
	FMovieSceneDynamicBinding DynamicBinding;

	// Preview Spawnable Type to use for this replaceable
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Sequencer")
	TSubclassOf<UMovieSceneSpawnableBindingBase> PreviewSpawnableType;

protected:
	/* MovieSceneReplaceableBindingBase overrides*/

	virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

#if WITH_EDITOR
	MOVIESCENETRACKS_API virtual UMovieSceneCustomBinding* CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene) override;
#endif

	// By default we return nullptr here, as we rely on Sequencer's BindingOverride mechanism to bind these actors during runtime.
	// This can be overridden if desired in subclasses to provide a different way to resolve to an actor at runtime while still using spawnable actor as the preview.
	MOVIESCENETRACKS_API virtual FMovieSceneBindingResolveResult ResolveRuntimeBindingInternal(const FMovieSceneBindingResolveParams& ResolveParams, int32 BindingIndex, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;
	
	// Empty implementation by default as we don't need to initialize any data members other than the spawnable, which is initialized by CreateInnerSpawnable in the base class
	virtual void InitReplaceableBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene) override {}

	virtual TSubclassOf<UMovieSceneSpawnableBindingBase> GetInnerSpawnableClass() const override { return PreviewSpawnableType; }
	
	virtual bool SupportsBindingCreationFromObject(const UObject* SourceObject) const override { return true; }
};

