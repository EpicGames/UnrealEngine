// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "MovieSceneSpawnRegister.h"
#include "MovieSceneSpawnableActorBinding.generated.h"

/**
 * The base class for actor-specific spawnable bindings. Contains a default implementation that can handle spawning an Actor from provided Actor class and optional Actor template.
 * Can be overridden in C++ or blueprint to provide an Actor class and to add custom PostSpawnObject behavior such as mesh setup based on an asset.
 * The below UMovieSceneSpawnableActorBinding class implements this base class and replicates the old FMovieSceneSpawnable behavior by using a specified Actor template to spawn an Actor and can be used out of the box.
 */
UCLASS(Abstract, MinimalAPI)
class UMovieSceneSpawnableActorBindingBase
	: public UMovieSceneSpawnableBindingBase
{
public:

	GENERATED_BODY()

public:

	// Optional template support for spawnables. Subclasses can override this to provide support for saving an object template into a binding.
	
	/* Override and return true if the binding type supports object templates.*/
	virtual bool SupportsObjectTemplates() const { return false; }

	/* Override and return the object template if the binding type supports object templates*/
	virtual UObject* GetObjectTemplate() { return nullptr; }

	/**
	 * Sets the object template to the specified object directly.
	 * Used for Copy/Paste, typically you should use CopyObjectTemplate.
	 */
	virtual void SetObjectTemplate(UObject* InObjectTemplate) {}

	/**
	 * Copy the specified object into this spawnable's template
	 *
	 * @param InSourceObject The source object to use. This object will be duplicated into the spawnable.
	 * @param MovieSceneSequence The movie scene sequence to which this spawnable belongs
	 */
	virtual void CopyObjectTemplate(UObject* InSourceObject, UMovieSceneSequence& MovieSceneSequence) {}


	/**
	 * Automatically determine a value for bNetAddressableName based on the spawnable type
	 */
	MOVIESCENETRACKS_API void AutoSetNetAddressableName();

	/*
	* Returns the optional level name to spawn the actor in, otherwise the Persistent level is used.
	*/
	FName GetLevelName() const { return LevelName; }
	
	/*
	* Used to provide an optional level name to spawn the actor in, otherwise the Persistent level is used.
	*/
	void SetLevelName(FName InLevelName) { LevelName = InLevelName; }


public:
	/** When enabled, the actor will be spawned with a unique name so that it can be addressable between clients and servers. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Spawnable)
	bool bNetAddressableName = false;

	/** Name of level to spawn into */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = Spawnable)
	FName LevelName;

protected:

	/* Override to provide Actor class to be spawned*/
	// TODO: Make UFUNCTION
	virtual TSubclassOf<AActor> GetActorClass() const PURE_VIRTUAL(UMovieSceneSpawnableActorBindingBase::GetActorClass, return AActor::StaticClass(););

	/* Optionally override to provide an Actor template to use during Spawn */
	virtual AActor* GetActorTemplate() const { return nullptr; }

	/* Returns the transform to spawn the actor at*/
	MOVIESCENETRACKS_API  FTransform GetSpawnTransform() const;

	UClass* GetBoundObjectClass() const override { return GetActorClass(); }
#if WITH_EDITOR
	/* MovieSceneCustomBinding overrides*/
	virtual int32 GetCustomBindingPriority() const override { return BaseEnginePriority + 2; }
#endif

protected:
	// UMovieSceneSpawnableBindingBase overrides

	/* Overridden to handle Actor-specific spawning */
	MOVIESCENETRACKS_API virtual UObject* SpawnObjectInternal(UWorld* WorldContext, FName SpawnName, const FGuid& BindingId, int32 BindingIndex, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;

	/* Overridden to handle Actor-specific destruction*/
	MOVIESCENETRACKS_API virtual void DestroySpawnedObjectInternal(UObject* Object) override;

	MOVIESCENETRACKS_API FName GetSpawnName(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;


private:
	FName GetNetAddressableName(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, const FGuid& BindingId, FMovieSceneSequenceID SequenceID, const FString& BaseName) const;
};

/*
* An implementation of UMovieSceneSpawnableActorBindingBase that matches the old FMovieSceneSpawnable spawnable implementation, allowing the spawning 
* of Actors from a UObject template which is serialized inside the Sequence.
*/
UCLASS(BlueprintType, MinimalAPI, EditInlineNew, DefaultToInstanced, Meta=(DisplayName="Spawnable Actor"))
class UMovieSceneSpawnableActorBinding
	: public UMovieSceneSpawnableActorBindingBase
{
public:

	GENERATED_BODY()

public:

	/* MovieSceneSpawnableActorBindingBase overrides*/
	bool SupportsObjectTemplates() const override { return true; }

	/* Override and return the object template if the binding type supports object templates*/
	UObject* GetObjectTemplate() override { return ActorTemplate.Get(); }

	/* MovieSceneSpawnableBindingBase overrides*/
	MOVIESCENETRACKS_API void SetObjectTemplate(UObject* InObjectTemplate) override;
	MOVIESCENETRACKS_API void CopyObjectTemplate(UObject* InSourceObject, UMovieSceneSequence& MovieSceneSequence) override;

	/* MovieSceneCustomBinding overrides*/
	MOVIESCENETRACKS_API virtual bool SupportsBindingCreationFromObject(const UObject* SourceObject) const override;
	MOVIESCENETRACKS_API virtual UMovieSceneCustomBinding* CreateNewCustomBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene) override;

#if WITH_EDITOR
	MOVIESCENETRACKS_API virtual bool SupportsConversionFromBinding(const FMovieSceneBindingReference& BindingReference, const UObject* SourceObject) const override;
	MOVIESCENETRACKS_API virtual UMovieSceneCustomBinding* CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene) override;
	MOVIESCENETRACKS_API virtual FText GetBindingTypePrettyName() const override;
#endif

protected:
	/* UObject overrides */
	MOVIESCENETRACKS_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

	/* MovieSceneSpawnableBindingBase overrides*/
	MOVIESCENETRACKS_API UWorld* GetWorldContext(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;
	
	/* MovieSceneSpawnableActorBindingBase overrides*/
	MOVIESCENETRACKS_API TSubclassOf<AActor> GetActorClass() const override;
	AActor* GetActorTemplate() const override { return ActorTemplate; }

protected:
	UPROPERTY(VisibleAnywhere, Category="Spawnable")
	TObjectPtr<AActor> ActorTemplate;


};

