// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Bindings/MovieSceneSpawnableBinding.h"
#include "Templates/SubclassOf.h"
#include "GameFramework/Actor.h"
#include "MovieSceneSpawnRegister.h"
#include "MovieSceneDynamicBinding.h"
#include "MovieSceneSpawnableDirectorBlueprintBinding.generated.h"

/**
 Custom binding type that uses a Director Blueprint endpoint to allow the user to define how to spawn an actor for this binding.
 */
UCLASS(Blueprintable, MinimalAPI)
class UMovieSceneSpawnableDirectorBlueprintBinding
	: public UMovieSceneSpawnableBindingBase
{
public:

	GENERATED_BODY()

	/* MovieSceneCustomBinding overrides- todo?*/
	UClass* GetBoundObjectClass() const override { return UObject::StaticClass(); }

	// Director Blueprint defined binding info
	UPROPERTY(EditAnywhere, Category="Sequencer")
	FMovieSceneDynamicBinding DynamicBinding;


protected:
	// UMovieSceneSpawnableBindingBase overrides

	/* Overridden to handle spawning */
	MOVIESCENETRACKS_API virtual UObject* SpawnObjectInternal(UWorld* WorldContext, FName SpawnName, const FGuid& BindingId, int32 BindingIndex, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;

	/* Overridden to handle destruction*/
	MOVIESCENETRACKS_API virtual void DestroySpawnedObjectInternal(UObject* Object) override;

	MOVIESCENETRACKS_API FName GetSpawnName(const FGuid& BindingId, UMovieScene& MovieScene, FMovieSceneSequenceIDRef TemplateID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;

	/* MovieSceneCustomBinding overrides*/
	bool SupportsBindingCreationFromObject(const UObject* SourceObject) const override;
	UMovieSceneCustomBinding* CreateNewCustomBinding(UObject* SourceObject, UMovieScene& OwnerMovieScene) override;

#if WITH_EDITOR
	bool SupportsConversionFromBinding(const FMovieSceneBindingReference& BindingReference, const UObject* SourceObject) const override;
	UMovieSceneCustomBinding* CreateCustomBindingFromBinding(const FMovieSceneBindingReference& BindingReference, UObject* SourceObject, UMovieScene& OwnerMovieScene) override;
	FText GetBindingTypePrettyName() const override;	
	FText GetBindingTrackIconTooltip() const override;
#endif

	/** UObject overrides */
	MOVIESCENETRACKS_API virtual void PostDuplicate(EDuplicateMode::Type DuplicateMode) override;

private:
	/* MovieSceneSpawnableBindingBase overrides*/
	UWorld* GetWorldContext(TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;
};
