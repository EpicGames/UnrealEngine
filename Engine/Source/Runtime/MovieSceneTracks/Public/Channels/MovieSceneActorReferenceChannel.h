// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneObjectBindingID.h"

#include "MovieSceneActorReferenceChannel.generated.h"

/**
 * Keyframe value type for actor references.
 */
USTRUCT()
struct FMovieSceneActorReferenceKey
{
	GENERATED_BODY()

	FMovieSceneActorReferenceKey()
	{}

	FMovieSceneActorReferenceKey(const FMovieSceneObjectBindingID& InBindingID)
		: Object(InBindingID)
	{}

	friend bool operator==(const FMovieSceneActorReferenceKey& A, const FMovieSceneActorReferenceKey& B)
	{
		return A.Object == B.Object && A.ComponentName == B.ComponentName && A.SocketName == B.SocketName;
	}

	friend bool operator!=(const FMovieSceneActorReferenceKey& A, const FMovieSceneActorReferenceKey& B)
	{
		return A.Object != B.Object || A.ComponentName != B.ComponentName || A.SocketName != B.SocketName;
	}

	UPROPERTY(EditAnywhere, Category="Key")
	FMovieSceneObjectBindingID Object;

	UPROPERTY(EditAnywhere, Category="Key")
	FName ComponentName;

	UPROPERTY(EditAnywhere, Category="Key")
	FName SocketName;
};

/**
 * A channel containing keyframes referencing an actor elsewhere in a Sequence.
 */
USTRUCT()
struct FMovieSceneActorReferenceChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	typedef FMovieSceneActorReferenceKey CurveValueType;

	FMovieSceneActorReferenceChannel()
		: DefaultValue()
	{}

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	inline TMovieSceneChannelData<FMovieSceneActorReferenceKey> GetData()
	{
		return TMovieSceneChannelData<FMovieSceneActorReferenceKey>(&KeyTimes, &KeyValues, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	inline TMovieSceneChannelData<const FMovieSceneActorReferenceKey> GetData() const
	{
		return TMovieSceneChannelData<const FMovieSceneActorReferenceKey>(&KeyTimes, &KeyValues);
	}

	/**
	 * Evaluate this channel
	 *
	 * @param InTime     The time to evaluate at
	 * @param OutValue   A value to receive the result
	 * @return true if the channel was evaluated successfully, false otherwise
	 */
	MOVIESCENETRACKS_API bool Evaluate(FFrameTime InTime, FMovieSceneActorReferenceKey& OutValue) const;

public:

	// ~ FMovieSceneChannel Interface
	MOVIESCENETRACKS_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENETRACKS_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENETRACKS_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENETRACKS_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENETRACKS_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENETRACKS_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENETRACKS_API virtual int32 GetNumKeys() const override;
	MOVIESCENETRACKS_API virtual void Reset() override;
	MOVIESCENETRACKS_API virtual void Offset(FFrameNumber DeltaPosition) override;
	MOVIESCENETRACKS_API virtual void ClearDefault() override;
	MOVIESCENETRACKS_API virtual FKeyHandle GetHandle(int32 Index) override;
	MOVIESCENETRACKS_API virtual int32 GetIndex(FKeyHandle Handle) override;

public:

	/**
	 * Set this channel's default value that should be used when no keys are present
	 *
	 * @param InDefaultValue The desired default value
	 */
	inline void SetDefault(FMovieSceneActorReferenceKey InDefaultValue)
	{
		DefaultValue = InDefaultValue;
	}

	/**
	 * Get this channel's default value that will be used when no keys are present
	 *
	 * @return (Optional) The channel's default value
	 */
	inline const FMovieSceneActorReferenceKey& GetDefault() const
	{
		return DefaultValue;
	}

	/**
	 * Upgrade legacy data by appending to the end of the array. Assumes sorted data
	 */
	void UpgradeLegacyTime(UObject* Context, double Time, FMovieSceneActorReferenceKey Value)
	{
		FFrameRate LegacyFrameRate = GetLegacyConversionFrameRate();
		FFrameNumber KeyTime = UpgradeLegacyMovieSceneTime(nullptr, LegacyFrameRate, Time);

		check(KeyTimes.Num() == 0 || KeyTime >= KeyTimes.Last());

		KeyTimes.Add(KeyTime);
		KeyValues.Add(Value);
	}

private:

	/** Sorted array of key times */
	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> KeyTimes;

	/** Default value used when there are no keys */
	UPROPERTY()
	FMovieSceneActorReferenceKey DefaultValue;

	/** Array of values that correspond to each key time */
	UPROPERTY(meta=(KeyValues))
	TArray<FMovieSceneActorReferenceKey> KeyValues;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;
};

// Backwards compatibility for C++ code.
using FMovieSceneActorReferenceData UE_DEPRECATED(5.8, "Please use FMovieSceneActorReferenceChannel") = FMovieSceneActorReferenceChannel;

inline bool EvaluateChannel(const FMovieSceneActorReferenceChannel* InChannel, FFrameTime InTime, FMovieSceneActorReferenceKey& OutValue)
{
	return InChannel->Evaluate(InTime, OutValue);
}

inline bool GetChannelDefault(const FMovieSceneActorReferenceChannel* Channel, FMovieSceneActorReferenceKey& OutDefaultValue)
{
	OutDefaultValue = Channel->GetDefault();
	return true;
}

