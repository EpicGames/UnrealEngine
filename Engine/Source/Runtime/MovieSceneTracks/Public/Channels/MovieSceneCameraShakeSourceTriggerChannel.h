// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraShakeBase.h"
#include "Camera/CameraTypes.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "Curves/KeyHandle.h"
#include "Math/Range.h"
#include "Math/Rotator.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneCameraShakeSourceTriggerChannel.generated.h"

class UCameraShakeBase;
struct FFrameRate;
struct FKeyHandle;

USTRUCT(BlueprintType)
struct FMovieSceneCameraShakeSourceTrigger
{
	GENERATED_BODY()

	FMovieSceneCameraShakeSourceTrigger()
		: ShakeClass(nullptr)
		, PlayScale(1.f)
		, PlaySpace(ECameraShakePlaySpace::CameraLocal)
		, UserDefinedPlaySpace(ForceInitToZero)
	{}

	FMovieSceneCameraShakeSourceTrigger(TSubclassOf<UCameraShakeBase> InShakeClass)
		: ShakeClass(InShakeClass)
		, PlayScale(1.f)
		, PlaySpace(ECameraShakePlaySpace::CameraLocal)
		, UserDefinedPlaySpace(ForceInitToZero)
	{}

	/** Class of the camera shake to play */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Shake")
	TSubclassOf<UCameraShakeBase> ShakeClass;

	/** Scalar that affects shake intensity */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Shake")
	float PlayScale;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Shake")
	ECameraShakePlaySpace PlaySpace;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Camera Shake")
	FRotator UserDefinedPlaySpace;
};

USTRUCT()
struct FMovieSceneCameraShakeSourceTriggerChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	typedef FMovieSceneCameraShakeSourceTrigger CurveValueType;

	inline TMovieSceneChannelData<FMovieSceneCameraShakeSourceTrigger> GetData()
	{
		return TMovieSceneChannelData<FMovieSceneCameraShakeSourceTrigger>(&KeyTimes, &KeyValues, this, &KeyHandles);
	}

	inline TMovieSceneChannelData<const FMovieSceneCameraShakeSourceTrigger> GetData() const
	{
		return TMovieSceneChannelData<const FMovieSceneCameraShakeSourceTrigger>(&KeyTimes, &KeyValues);
	}

public:
	// ~ FMovieSceneChannel Interface
	MOVIESCENETRACKS_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENETRACKS_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENETRACKS_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENETRACKS_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENETRACKS_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENETRACKS_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENETRACKS_API virtual int32 GetNumKeys() const override;
	MOVIESCENETRACKS_API virtual void Reset() override;
	MOVIESCENETRACKS_API virtual void Offset(FFrameNumber DeltaPosition) override;

private:
	/** Array of times for each key */
	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> KeyTimes;

	/** Array of values that correspond to each key time */
	UPROPERTY(meta=(KeyValues))
	TArray<FMovieSceneCameraShakeSourceTrigger> KeyValues;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneCameraShakeSourceTriggerChannel> : TMovieSceneChannelTraitsBase<FMovieSceneCameraShakeSourceTriggerChannel>
{
	enum { SupportsDefaults = false };
};

inline bool EvaluateChannel(const FMovieSceneCameraShakeSourceTriggerChannel* InChannel, FFrameTime InTime, FMovieSceneCameraShakeSourceTrigger& OutValue)
{
	return false;
}

