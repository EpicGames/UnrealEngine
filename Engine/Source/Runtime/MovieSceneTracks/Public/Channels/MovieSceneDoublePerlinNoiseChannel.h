// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "MovieSceneTracksComponentTypes.h"
#include "MovieSceneDoublePerlinNoiseChannel.generated.h"

USTRUCT()
struct FMovieSceneDoublePerlinNoiseChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	using CurveValueType = double;

	MOVIESCENETRACKS_API FMovieSceneDoublePerlinNoiseChannel();

	/**
	 * Evaluate this channel at the given time
	 *
	 * @param InSeconds		The time, in seconds, to evaluate at
	 * @return				The evaluated noise value
	 */
	MOVIESCENETRACKS_API double Evaluate(double InSeconds) const;

	/**
	 * Evaluate this channel at the given time
	 *
	 * @param InSection		The section that contains this channel, used to lookup the sequence's tick resolution
	 * @param InTime		The time, in ticks, to evaluate at
	 * @param OutValue		The evaluated noise value
	 * @return				Whether the noise was successfully evaluated
	 */
	MOVIESCENETRACKS_API bool Evaluate(const UMovieSceneSection* InSection, FFrameTime InTime, double& OutValue) const;

	/** The noise parameters */
	UPROPERTY(EditAnywhere, Category = "Perlin Noise")
	FPerlinNoiseParams PerlinNoiseParams;

	/**
	 * Evaluate perlin noise
	 *
	 * @params InParams			Perlin noise parameters
	 * @params InInitialOffset	The initial offset for the noise
	 * @params InSeconds		The time at which to evaluate the noise
	 * @return					The evaluated noise value
	 */
	static MOVIESCENETRACKS_API double Evaluate(const FPerlinNoiseParams& InParams, double InSeconds);
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneDoublePerlinNoiseChannel> : TMovieSceneChannelTraitsBase<FMovieSceneDoublePerlinNoiseChannel>
{
#if WITH_EDITOR

	/** Perlin noise channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<double> ExtendedEditorDataType;

#endif
};

inline bool EvaluateChannel(const UMovieSceneSection* InSection, const FMovieSceneDoublePerlinNoiseChannel* InChannel, FFrameTime InTime, double& OutValue)
{
	return InChannel->Evaluate(InSection, InTime, OutValue);
}

