// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Engine/Blueprint.h"
#include "Misc/Guid.h"
#include "UObject/FieldPath.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/WeakObjectPtr.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "UObject/SoftObjectPtr.h"

#include "MovieSceneEvent.generated.h"

class FProperty;
class UBlueprint;
class UClass;
class UEdGraph;
class UEdGraphNode;
class UFunction;
class UK2Node;
class UK2Node_FunctionEntry;
class UMovieSceneEventSectionBase;
class UObject;

/** Value definition for any type-agnostic variable (exported as text) */
USTRUCT(BlueprintType)
struct FMovieSceneEventPayloadVariable
{
	GENERATED_BODY()

	UPROPERTY()
	/** If the value for this pin should be an object, we store a pointer to it */
	FSoftObjectPath ObjectValue;

	UPROPERTY(EditAnywhere, Category="Sequencer|Event")
	FString Value;
};

/** Compiled reflection pointers for the event function and parameters */
USTRUCT(BlueprintType)
struct FMovieSceneEventPtrs
{
	GENERATED_BODY()

	FMovieSceneEventPtrs()
		: Function(nullptr)
		, BoundObjectProperty(nullptr)
	{}

	UPROPERTY()
	TObjectPtr<UFunction> Function;

	UPROPERTY()
	TFieldPath<FProperty> BoundObjectProperty;
};

USTRUCT(BlueprintType)
struct FMovieSceneEvent
{
	GENERATED_BODY()

	/**
	 * The function that should be called to invoke this event.
	 * Functions must have either no parameters, or a single, pass-by-value object/interface parameter, with no return parameter.
	 */
	UPROPERTY()
	FMovieSceneEventPtrs Ptrs;

public:

	/** Return the class of the bound object property */
	MOVIESCENETRACKS_API UClass* GetBoundObjectPropertyClass() const;

#if WITH_EDITORONLY_DATA

	/** Array of payload variables to be added to the generated function */
	UPROPERTY(EditAnywhere, Category="Sequencer|Event")
	TMap<FName, FMovieSceneEventPayloadVariable> PayloadVariables;

	UPROPERTY(transient)
	FName CompiledFunctionName;

	UPROPERTY(EditAnywhere, Category="Sequencer|Event")
	FName BoundObjectPinName;

	/** Serialized weak pointer to the function entry (UK2Node_FunctionEntry) or custom event node (UK2Node_CustomEvent) within the blueprint graph for this event. Stored as an editor-only UObject so UHT can parse it when building for non-editor. */
	UPROPERTY(EditAnywhere, Category="Sequencer|Event")
	TWeakObjectPtr<UObject> WeakEndpoint;

	/** (deprecated) The UEdGraph::GraphGuid property that relates the graph within which our endpoint lives. */
	UPROPERTY()
	FGuid GraphGuid_DEPRECATED;

	/** (deprecated) When valid, relates to the The UEdGraphNode::NodeGuid for a custom event node that defines our event endpoint. When invalid, we must be bound to a UBlueprint::FunctionGraphs graph. */
	UPROPERTY()
	FGuid NodeGuid_DEPRECATED;

	/** Deprecated weak pointer to the function entry to call - no longer serialized but cached on load. Predates GraphGuid and NodeGuid */
	UPROPERTY()
	TWeakObjectPtr<UObject> FunctionEntry_DEPRECATED;

#endif // WITH_EDITORONLY_DATA
};





