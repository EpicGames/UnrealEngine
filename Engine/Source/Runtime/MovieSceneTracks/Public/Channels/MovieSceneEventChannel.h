// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Channels/MovieSceneEvent.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "CoreTypes.h"
#include "Curves/KeyHandle.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneEventChannel.generated.h"

struct FFrameRate;
struct FKeyHandle;


USTRUCT()
struct FMovieSceneEventChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	typedef FMovieSceneEvent CurveValueType;

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	inline TMovieSceneChannelData<FMovieSceneEvent> GetData()
	{
		return TMovieSceneChannelData<FMovieSceneEvent>(&KeyTimes, &KeyValues, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	inline TMovieSceneChannelData<const FMovieSceneEvent> GetData() const
	{
		return TMovieSceneChannelData<const FMovieSceneEvent>(&KeyTimes, &KeyValues);
	}

public:

	// ~ FMovieSceneChannel Interface
	MOVIESCENETRACKS_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENETRACKS_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENETRACKS_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENETRACKS_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENETRACKS_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENETRACKS_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENETRACKS_API virtual int32 GetNumKeys() const override;
	MOVIESCENETRACKS_API virtual void Reset() override;
	MOVIESCENETRACKS_API virtual void Offset(FFrameNumber DeltaPosition) override;
	MOVIESCENETRACKS_API virtual FKeyHandle GetHandle(int32 Index) override;
	MOVIESCENETRACKS_API virtual int32 GetIndex(FKeyHandle Handle) override;
private:

	/** Array of times for each key */
	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> KeyTimes;

	/** Array of values that correspond to each key time */
	UPROPERTY(meta=(KeyValues))
	TArray<FMovieSceneEvent> KeyValues;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneEventChannel> : TMovieSceneChannelTraitsBase<FMovieSceneEventChannel>
{
	enum { SupportsDefaults = false };
};

inline bool EvaluateChannel(const FMovieSceneEventChannel* InChannel, FFrameTime InTime, FMovieSceneEvent& OutValue)
{
	return false;
}
