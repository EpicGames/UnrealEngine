// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelOverrideContainer.h"
#include "Channels/MovieSceneFloatPerlinNoiseChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneFloatPerlinNoiseChannelContainer.generated.h"

/**
 * Float perlin noise channel overriden container
 */
UCLASS(meta=(DisplayName="Float Perlin Noise", ToolTip="Override a channel to use float perlin noise"), MinimalAPI)
class UMovieSceneFloatPerlinNoiseChannelContainer : public UMovieSceneChannelOverrideContainer
{
	GENERATED_BODY()

public:

	using ChannelType = FMovieSceneFloatPerlinNoiseChannel;

	MOVIESCENETRACKS_API void InitializeOverride(FMovieSceneChannel* InChannel) override;
	MOVIESCENETRACKS_API bool SupportsOverride(FName DefaultChannelTypeName) const override;
	MOVIESCENETRACKS_API void ImportEntityImpl(
			const UE::MovieScene::FChannelOverrideEntityImportParams& OverrideParams, 
			const UE::MovieScene::FEntityImportParams& ImportParams, UE::MovieScene::FImportedEntity* OutImportedEntity) override;

	const FMovieSceneChannel* GetChannel() const override { return &PerlinNoiseChannel; }
	FMovieSceneChannel* GetChannel() override { return &PerlinNoiseChannel; }

#if WITH_EDITOR
	MOVIESCENETRACKS_API FMovieSceneChannelHandle AddChannelProxy(FName ChannelName, FMovieSceneChannelProxyData& ProxyData, const FMovieSceneChannelMetaData& MetaData) override;
#else
	MOVIESCENETRACKS_API void AddChannelProxy(FName ChannelName, FMovieSceneChannelProxyData& ProxyData) override;
#endif

private:

	UPROPERTY(EditAnywhere, Category = "Perlin Noise")
	FMovieSceneFloatPerlinNoiseChannel PerlinNoiseChannel;

	friend class FMovieSceneFloatPerlinNoiseChannelDetailsCustomization;
};

