// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Containers/Array.h"
#include "Containers/ArrayView.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Math/Range.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameTime.h"
#include "Misc/Optional.h"
#include "Serialization/StructuredArchive.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneStringChannel.generated.h"

struct FFrameRate;
struct FKeyHandle;
struct FPropertyTag;

USTRUCT()
struct FMovieSceneStringChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	typedef FString CurveValueType;

	FMovieSceneStringChannel()
		: DefaultValue(), bHasDefaultValue(false)
	{}

	/**
	 * Serialize this type from another
	 */
	MOVIESCENETRACKS_API bool SerializeFromMismatchedTag(const FPropertyTag& Tag, FStructuredArchive::FSlot Slot);

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	inline TMovieSceneChannelData<FString> GetData()
	{
		return TMovieSceneChannelData<FString>(&Times, &Values, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	inline TMovieSceneChannelData<const FString> GetData() const
	{
		return TMovieSceneChannelData<const FString>(&Times, &Values);
	}

	/**
     * Const access to this channel's times
     */
	inline TArrayView<const FFrameNumber> GetTimes() const
	{
		return Times;
	}

	/**
	 * Returns whether this channel has any values
	 */
	inline bool HasAnyData() const
	{
		return Times.Num() != 0 || bHasDefaultValue == true;
	}

	/**
	 * Evaluate this channel
	 *
	 * @param InTime     The time to evaluate at
	 * @return A pointer to the string, or nullptr
	 */
	MOVIESCENETRACKS_API const FString* Evaluate(FFrameTime InTime) const;

	/**
	 * Add keys with these times to channel. The number of elements in both arrays much match or nothing is added.
	 * @param InTimes Times to add
	 * @param InValues Values to add
	 */
	inline void AddKeys(const TArray<FFrameNumber>& InTimes, const TArray<FString>& InValues)
	{
		check(InTimes.Num() == InValues.Num());
		Times.Append(InTimes);
		Values.Append(InValues);
	}

public:

	// ~ FMovieSceneChannel Interface
	MOVIESCENETRACKS_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENETRACKS_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENETRACKS_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENETRACKS_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENETRACKS_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENETRACKS_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENETRACKS_API virtual int32 GetNumKeys() const override;
	MOVIESCENETRACKS_API virtual void Reset() override;
	MOVIESCENETRACKS_API virtual void Offset(FFrameNumber DeltaPosition) override;
	MOVIESCENETRACKS_API virtual void Optimize(const FKeyDataOptimizationParams& InParameters) override;
	MOVIESCENETRACKS_API virtual void ClearDefault() override;
	MOVIESCENETRACKS_API virtual FKeyHandle GetHandle(int32 Index) override;
	MOVIESCENETRACKS_API virtual int32 GetIndex(FKeyHandle Handle) override;

public:

	/**
	 * Set this channel's default value that should be used when no keys are present
	 *
	 * @param InDefaultValue The desired default value
	 */
	inline void SetDefault(FString InDefaultValue)
	{
		bHasDefaultValue = true;
		DefaultValue = InDefaultValue;
	}

	/**
	 * Get this channel's default value that will be used when no keys are present
	 *
	 * @return (Optional) The channel's default value
	 */
	inline TOptional<FString> GetDefault() const
	{
		return bHasDefaultValue ? TOptional<FString>(DefaultValue) : TOptional<FString>();
	}

	/**
	 * Remove this channel's default value causing the channel to have no effect where no keys are present
	 */
	inline void RemoveDefault()
	{
		bHasDefaultValue = false;
	}

private:

	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> Times;

	/** Array of values that correspond to each key time */
	UPROPERTY(meta=(KeyValues))
	TArray<FString> Values;

	/** Default value used when there are no keys */
	UPROPERTY()
	FString DefaultValue;

	/** */
	UPROPERTY()
	bool bHasDefaultValue;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;
};

template<>
struct TStructOpsTypeTraits<FMovieSceneStringChannel> : public TStructOpsTypeTraitsBase2<FMovieSceneStringChannel>
{
	enum { WithStructuredSerializeFromMismatchedTag = true };
};


template<>
struct TMovieSceneChannelTraits<FMovieSceneStringChannel> : TMovieSceneChannelTraitsBase<FMovieSceneStringChannel>
{
#if WITH_EDITOR

	/** String channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<FString> ExtendedEditorDataType;

#endif
};

inline bool EvaluateChannel(const FMovieSceneStringChannel* InChannel, FFrameTime InTime, FString& OutValue)
{
	if (const FString* Result = InChannel->Evaluate(InTime))
	{
		OutValue = *Result;
		return true;
	}
	return false;
}

inline bool ValueExistsAtTime(const FMovieSceneStringChannel* Channel, FFrameNumber InFrameNumber, const FString& Value)
{
	const FFrameTime FrameTime(InFrameNumber);

	const FString* ExistingValue = Channel->Evaluate(FrameTime);
	return ExistingValue && Value == *ExistingValue;
}

