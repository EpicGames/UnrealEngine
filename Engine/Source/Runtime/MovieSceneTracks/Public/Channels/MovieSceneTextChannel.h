// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelTraits.h"

#if WITH_EDITOR
#include "UObject/WeakObjectPtr.h"
#endif

#include "MovieSceneTextChannel.generated.h"

#if WITH_EDITOR
class UPackage;
#endif

USTRUCT()
struct FMovieSceneTextChannel : public FMovieSceneChannel
{
	GENERATED_BODY()

	typedef FText CurveValueType;

#if WITH_EDITOR
	MOVIESCENETRACKS_API void SetPackage(UPackage* InPackage);

	MOVIESCENETRACKS_API UPackage* GetPackage() const;
#endif

	/**
	 * Access a mutable interface for this channel's data
	 * @return An object that is able to manipulate this channel's data
	 */
	inline TMovieSceneChannelData<FText> GetData()
	{
		return TMovieSceneChannelData<FText>(&Times, &Values, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 * @return An object that is able to interrogate this channel's data
	 */
	inline TMovieSceneChannelData<const FText> GetData() const
	{
		return TMovieSceneChannelData<const FText>(&Times, &Values);
	}

	inline TConstArrayView<FFrameNumber> GetTimes() const
	{
		return Times;
	}

	/** Returns whether this channel has any values */
	inline bool HasAnyData() const
	{
		return Times.Num() != 0 || bHasDefaultValue == true;
	}

	/**
	 * Evaluate this channel
	 * @param InTime     The time to evaluate at
	 * @return A pointer to the string, or nullptr
	 */
	MOVIESCENETRACKS_API const FText* Evaluate(FFrameTime InTime) const;

	/**
	 * Add keys with these times to channel. The number of elements in both arrays much match or nothing is added.
	 * @param InTimes Times to add
	 * @param InValues Values to add
	 */
	inline void AddKeys(const TArray<FFrameNumber>& InTimes, const TArray<FText>& InValues)
	{
		check(InTimes.Num() == InValues.Num());
		Times.Append(InTimes);
		Values.Append(InValues);
	}

	/**
	 * Set this channel's default value that should be used when no keys are present
	 * @param InDefaultValue The desired default value
	 */
	inline void SetDefault(FText InDefaultValue)
	{
		bHasDefaultValue = true;
		DefaultValue = InDefaultValue;
	}

	/**
	 * Get this channel's default value that will be used when no keys are present
	 * @return (Optional) The channel's default value
	 */
	inline TOptional<FText> GetDefault() const
	{
		return bHasDefaultValue ? TOptional<FText>(DefaultValue) : TOptional<FText>();
	}

	/** Remove this channel's default value causing the channel to have no effect where no keys are present */
	inline void RemoveDefault()
	{
		bHasDefaultValue = false;
	}

	//~ Begin FMovieSceneChannel
	MOVIESCENETRACKS_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENETRACKS_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENETRACKS_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENETRACKS_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENETRACKS_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENETRACKS_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENETRACKS_API virtual int32 GetNumKeys() const override;
	MOVIESCENETRACKS_API virtual void Reset() override;
	MOVIESCENETRACKS_API virtual void Offset(FFrameNumber DeltaPosition) override;
	MOVIESCENETRACKS_API virtual void Optimize(const FKeyDataOptimizationParams& InParameters) override;
	MOVIESCENETRACKS_API virtual void ClearDefault() override;
	//~ End FMovieSceneChannel

private:
	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> Times;

	/** Array of values that correspond to each key time */
	UPROPERTY(meta=(KeyValues))
	TArray<FText> Values;

	/** Default value used when there are no keys */
	UPROPERTY()
	FText DefaultValue;

	UPROPERTY()
	bool bHasDefaultValue = false;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it does not get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;

#if WITH_EDITOR
	/** Package to use for Text Localization. Always set to the Package of the Section holding this Channel. */
	TWeakObjectPtr<UPackage> Package;
#endif
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneTextChannel> : TMovieSceneChannelTraitsBase<FMovieSceneTextChannel>
{
#if WITH_EDITOR
	/** Text channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<FText> ExtendedEditorDataType;
#endif
};

inline bool EvaluateChannel(const FMovieSceneTextChannel* InChannel, FFrameTime InTime, FText& OutValue)
{
	if (const FText* Result = InChannel->Evaluate(InTime))
	{
		OutValue = *Result;
		return true;
	}
	return false;
}

inline bool ValueExistsAtTime(const FMovieSceneTextChannel* Channel, FFrameNumber InFrameNumber, const FText& Value)
{
	const FFrameTime FrameTime(InFrameNumber);

	const FText* ExistingValue = Channel->Evaluate(FrameTime);

	return ExistingValue && Value.EqualTo(*ExistingValue);
}
