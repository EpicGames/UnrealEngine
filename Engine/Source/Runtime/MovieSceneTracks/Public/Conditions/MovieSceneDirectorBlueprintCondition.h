// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Conditions/MovieSceneCondition.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Kismet/BlueprintFunctionLibrary.h"
#include "MovieSceneSequenceID.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "MovieSceneDirectorBlueprintCondition.generated.h"

#define UE_API MOVIESCENETRACKS_API

namespace UE
{
	namespace MovieScene
	{
		struct FSharedPlaybackState;
	}
}

/** Value definition for any type-agnostic variable (exported as text) */
USTRUCT(BlueprintType)
struct FMovieSceneDirectorBlueprintConditionPayloadVariable
{
	GENERATED_BODY()
		
	UPROPERTY()
	FSoftObjectPath ObjectValue;

	UPROPERTY(EditAnywhere, Category = "Sequencer|Condition")
	FString Value;
};

/**
 * Data for a director blueprint condition endpoint call.
 */
USTRUCT()
struct FMovieSceneDirectorBlueprintConditionData
{
	GENERATED_BODY()

	/** The function to call (normally a generated blueprint function on the sequence director) */
	UPROPERTY()
	TObjectPtr<UFunction> Function;

	/** Property pointer for the function parameter that should receive the condition context params */
	UPROPERTY()
	TFieldPath<FProperty> ConditionContextProperty;

#if WITH_EDITORONLY_DATA

	/** Array of payload variables to be added to the generated function */
	UPROPERTY(EditAnywhere, Category = "Sequencer|Condition")
	TMap<FName, FMovieSceneDirectorBlueprintConditionPayloadVariable> PayloadVariables;

	/** Name of the generated blueprint function */
	UPROPERTY(transient)
	FName CompiledFunctionName;

	/** Pin name for passing the condition context params */
	UPROPERTY(EditAnywhere, Category="Sequencer|Condition")
	FName ConditionContextPinName;

	/** Endpoint node in the sequence director */
	UPROPERTY(EditAnywhere, Category="Sequencer|Condition")
	TWeakObjectPtr<UObject> WeakEndpoint;

#endif
};

/**
 * Utility class for invoking director blueprint condition endpoints.
 */
struct FMovieSceneDirectorBlueprintConditionInvoker
{
	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	static bool EvaluateDirectorBlueprintCondition(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState, const FMovieSceneDirectorBlueprintConditionData& DirectorBlueprintCondition);

private:
	static bool InvokeDirectorBlueprintCondition(UObject* DirectorInstance, const FMovieSceneDirectorBlueprintConditionData& DirectorBlueprintCondition, const FMovieSceneConditionContext& ConditionContext);
};



/**
 * Condition class allowing users to create a director blueprint endpoint in the Sequence to evaluate a condition.
 */
UCLASS(MinimalAPI, DisplayName="Director Blueprint Condition")
class UMovieSceneDirectorBlueprintCondition
	: public UMovieSceneCondition
{
	GENERATED_BODY()

public: 

	UPROPERTY(EditAnywhere, Category="Sequencer|Condition")
	FMovieSceneDirectorBlueprintConditionData DirectorBlueprintConditionData;

protected:

	/*
	* UMovieSceneCondition overrides 
	*/
	UE_API virtual bool EvaluateConditionInternal(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;
	
	virtual EMovieSceneConditionScope GetScopeInternal() const override { return Scope; }
	
	virtual EMovieSceneConditionCheckFrequency GetCheckFrequencyInternal() const override { return CheckFrequency; }

protected:
	
	UPROPERTY(EditAnywhere, Category="Sequencer|Condition")
	EMovieSceneConditionScope Scope = EMovieSceneConditionScope::Global;

	UPROPERTY(EditAnywhere, Category="Sequencer|Condition")
	EMovieSceneConditionCheckFrequency CheckFrequency = EMovieSceneConditionCheckFrequency::Once;
};

#undef UE_API
