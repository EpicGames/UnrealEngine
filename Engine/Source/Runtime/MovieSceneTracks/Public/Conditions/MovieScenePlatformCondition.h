// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Conditions/MovieSceneCondition.h"
#include "MovieScenePlatformCondition.generated.h"

#define UE_API MOVIESCENETRACKS_API

namespace UE
{
	namespace MovieScene
	{
		struct FSharedPlaybackState;
	}
}

/**
 * Condition on whether the platform running the executable matches one of the given platforms.
 */
UCLASS(MinimalAPI, DisplayName="Platform Condition")
class UMovieScenePlatformCondition
	: public UMovieSceneCondition
{
	GENERATED_BODY()

public: 

	UPROPERTY(EditAnywhere, Category="Sequencer|Condition")
	TArray<FName> ValidPlatforms;

protected:

	/*
	* UMovieSceneCondition overrides 
	*/
	UE_API virtual bool EvaluateConditionInternal(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;
	
	virtual EMovieSceneConditionScope GetScopeInternal() const override { return EMovieSceneConditionScope::Global; }
	
	virtual EMovieSceneConditionCheckFrequency GetCheckFrequencyInternal() const override { return EMovieSceneConditionCheckFrequency::Once; }

};

#undef UE_API
