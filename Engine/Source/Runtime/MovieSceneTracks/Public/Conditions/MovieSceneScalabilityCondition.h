// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Conditions/MovieSceneCondition.h"
#include "MovieSceneScalabilityCondition.generated.h"

#define UE_API MOVIESCENETRACKS_API

namespace UE
{
	namespace MovieScene
	{
		struct FSharedPlaybackState;
	}
}


// The below is a bit hardcoded to try and match how scalability settings are set up in Scalability.h.
// This is because unfortunately scalability settings at their core are not very extensible or data-driven 
// and so it's difficult to do this in a data driven way. So I've made enums here to make the setup user friendly here,
// and then do the mapping in code. If scalability gets re-architected, this will need be to be updated to match.

UENUM()
enum class EMovieSceneScalabilityConditionGroup : uint8
{
	ViewDistance,
	AntiAliasing,
	Shadow,
	GlobalIllumination,
	Reflection,
	PostProcess,
	Texture,
	Effects,
	Foliage,
	Shading,
	Landscape
};


UENUM()
enum class EMovieSceneScalabilityConditionOperator : uint8
{
	LessThan,
	LessThanOrEqualTo,
	EqualTo,
	GreaterThanOrEqualTo,
	GreaterThan
};


UENUM()
enum class EMovieSceneScalabilityConditionLevel : uint8
{
	Low,
	Medium,
	High,
	Epic,
	Cinematic
};

/**
 * Condition on whether the current engine scalability settings fulfill a given constraint.
 */
UCLASS(MinimalAPI, DisplayName="Scalability Condition")
class UMovieSceneScalabilityCondition
	: public UMovieSceneCondition
{
	GENERATED_BODY()

public: 

	UPROPERTY(EditAnywhere, Category="Sequencer|Condition")
	EMovieSceneScalabilityConditionGroup Group;

	UPROPERTY(EditAnywhere, Category="Sequencer|Condition")
	EMovieSceneScalabilityConditionOperator Operator;

	UPROPERTY(EditAnywhere, Category="Sequencer|Condition")
	EMovieSceneScalabilityConditionLevel Level;

protected:

	/*
	* UMovieSceneCondition overrides 
	*/
	UE_API virtual bool EvaluateConditionInternal(FGuid BindingGuid, FMovieSceneSequenceID SequenceID, TSharedRef<const UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) const override;
	
	virtual EMovieSceneConditionScope GetScopeInternal() const override { return EMovieSceneConditionScope::Global; }
	
	virtual EMovieSceneConditionCheckFrequency GetCheckFrequencyInternal() const override { return EMovieSceneConditionCheckFrequency::Once; }

};

#undef UE_API
