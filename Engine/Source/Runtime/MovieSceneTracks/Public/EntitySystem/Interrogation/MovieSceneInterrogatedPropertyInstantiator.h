// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieScenePropertyComponentHandler.h"
#include "EntitySystem/MovieSceneOverlappingEntityTracker.h"

#include "MovieSceneInterrogatedPropertyInstantiator.generated.h"

class UMovieSceneBlenderSystem;

/** Class responsible for resolving all property types registered with FBuiltInComponentTypes::PropertyRegistry */
UCLASS(MinimalAPI)
class UMovieSceneInterrogatedPropertyInstantiatorSystem
	: public UMovieSceneEntityInstantiatorSystem
{
public:

	using FMovieSceneEntityID = UE::MovieScene::FMovieSceneEntityID;
	using FValueRecompositionResult = UE::MovieScene::TRecompositionResult<double>;

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneInterrogatedPropertyInstantiatorSystem(const FObjectInitializer& ObjInit);

	/**
	 * Recompose a value from the constituent parts specified in InQuery, taking into accounts the wieghtings of the specific channel defined by ChannelCompositeIndex.
	 *
	 * @param PropertyDefinition  The property that this float channel is bound to
	 * @param ChannelCompositeIndex  The index of the composite that this float channel represents, if it is part of a composite value (for instance when keying/recomposing Translation.Z)
	 * @param InQuery        The query defining the entities and object to recompose
	 * @param InCurrentValue The value of the property to recompose
	 * @return A result containing the recomposed value for each of the entities specified in InQuery
	 */
	MOVIESCENETRACKS_API FValueRecompositionResult RecomposeBlendChannel(const UE::MovieScene::FPropertyDefinition& PropertyDefinition, int32 ChannelCompositeIndex, const UE::MovieScene::FDecompositionQuery& InQuery, double InCurrentValue);

	/**
	 * Recompose a value from the constituent parts specified in InQuery, taking into account the weightings of each channel.
	 * For instance, if a property comprises 3 additive values (a:1, b:2, c:3), and we recompose 'a' with an InCurrentValue of 10, the result for a would be 5.
	 * @note: ValueType must be either copy assignable to/from the storage type of the property, or must have a ConvertOperationalProperty overload
	 *
	 * @param InComponents   The components that define the property to decompose
	 * @param InQuery        The query defining the entities and object to recompose
	 * @param InCurrentValue The value of the property to recompose
	 * @return A result matching the storage type of the components, containing recomposed values for each of the entities specified in InQuery
	 */
	template<typename PropertyTraits, typename ValueType>
	UE::MovieScene::TRecompositionResult<ValueType> RecomposeBlend(const UE::MovieScene::TPropertyComponents<PropertyTraits>& InComponents, const UE::MovieScene::FDecompositionQuery& InQuery, const ValueType& InCurrentValue)
	{
		typename PropertyTraits::StorageType WorkingValue{};
		ConvertOperationalProperty(InCurrentValue, WorkingValue);

		WorkingValue = RecomposeBlendOperational(InComponents, InQuery, WorkingValue);

		ValueType Result{};
		ConvertOperationalProperty(WorkingValue, Result);
		return Result;
	}

	/**
	 * Recompose a value from the constituent parts specified in InQuery, taking into account the weightings of each channel.
	 * For instance, if a property comprises 3 additive values (a:1, b:2, c:3), and we recompose 'a' with an InCurrentValue of 10, the result for a would be 5.
	 *
	 * @param InComponents   The components that define the property to decompose
	 * @param InQuery        The query defining the entities and object to recompose
	 * @param InCurrentValue The value of the property to recompose
	 * @return A result matching the storage type of the components, containing recomposed values for each of the entities specified in InQuery
	 */
	template<typename PropertyTraits>
	UE::MovieScene::TRecompositionResult<typename PropertyTraits::StorageType> RecomposeBlendOperational(const UE::MovieScene::TPropertyComponents<PropertyTraits>& InComponents, const UE::MovieScene::FDecompositionQuery& InQuery, const typename PropertyTraits::StorageType& InCurrentValue);

public:

	struct FPropertyInfo
	{
		FPropertyInfo()
			: BlendChannel(INVALID_BLEND_CHANNEL)
		{}
		/** POinter to the blender system to use for this property, if its blended */
		TWeakObjectPtr<UMovieSceneBlenderSystem> Blender;
		UE::MovieScene::FInterrogationChannel InterrogationChannel;
		/** The entity that contains the property component itself. For fast path properties this is the actual child entity produced from the bound object instantiators. */
		UE::MovieScene::FMovieSceneEntityID PropertyEntityID;
		/** Blend channel allocated from Blender, or INVALID_BLEND_CHANNEL if unblended. */
		uint16 BlendChannel;
	};

	// TOverlappingEntityTracker handler interface
	MOVIESCENETRACKS_API void InitializeOutput(UE::MovieScene::FInterrogationKey Key, TArrayView<const FMovieSceneEntityID> Inputs, FPropertyInfo* Output, UE::MovieScene::FEntityOutputAggregate Aggregate);
	MOVIESCENETRACKS_API void UpdateOutput(UE::MovieScene::FInterrogationKey Key, TArrayView<const FMovieSceneEntityID> Inputs, FPropertyInfo* Output, UE::MovieScene::FEntityOutputAggregate Aggregate);
	MOVIESCENETRACKS_API void DestroyOutput(UE::MovieScene::FInterrogationKey Key, FPropertyInfo* Output, UE::MovieScene::FEntityOutputAggregate Aggregate);

	const FPropertyInfo* FindPropertyInfo(UE::MovieScene::FInterrogationKey Key) const
	{
		return PropertyTracker.FindOutput(Key);
	}

	void FindEntityIDs(UE::MovieScene::FInterrogationKey Key, TArray<FMovieSceneEntityID>& OutEntityIDs) const
	{
		PropertyTracker.FindEntityIDs(Key, OutEntityIDs);
	}

private:

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	MOVIESCENETRACKS_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;

	MOVIESCENETRACKS_API bool PropertySupportsFastPath(TArrayView<const FMovieSceneEntityID> Inputs, FPropertyInfo* Output) const;
	MOVIESCENETRACKS_API UClass* ResolveBlenderClass(const UE::MovieScene::FPropertyDefinition& PropertyDefinition, TArrayView<const FMovieSceneEntityID> Inputs) const;
	MOVIESCENETRACKS_API UE::MovieScene::FPropertyRecomposerPropertyInfo FindPropertyFromSource(FMovieSceneEntityID EntityID, UObject* Object) const;

private:

	static constexpr uint16 INVALID_BLEND_CHANNEL = uint16(-1);

	UE::MovieScene::TOverlappingEntityTracker<FPropertyInfo, UE::MovieScene::FInterrogationKey> PropertyTracker;
	UE::MovieScene::FComponentMask CleanFastPathMask;
	UE::MovieScene::FBuiltInComponentTypes* BuiltInComponents;
	UE::MovieScene::FPropertyRecomposerImpl RecomposerImpl;
};

template<typename PropertyTraits>
UE::MovieScene::TRecompositionResult<typename PropertyTraits::StorageType> UMovieSceneInterrogatedPropertyInstantiatorSystem::RecomposeBlendOperational(const UE::MovieScene::TPropertyComponents<PropertyTraits>& Components, const UE::MovieScene::FDecompositionQuery& InQuery, const typename PropertyTraits::StorageType& InCurrentValue)
{
	return RecomposerImpl.RecomposeBlendOperational<PropertyTraits>(Components, InQuery, InCurrentValue);
}
