// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Containers/SparseArray.h"
#include "Containers/SortedMap.h"
#include "Evaluation/MovieSceneEvaluationField.h"
#include "MovieSceneTracksComponentTypes.h"
#include "UObject/GCObject.h"
#include "UObject/ObjectKey.h"

#include "EntitySystem/MovieSceneEntitySystemTypes.h"
#include "EntitySystem/MovieSceneEntitySystemLinkerExtension.h"

#include "EntitySystem/Interrogation/MovieSceneInterrogationExtension.h"
#include "Evaluation/MovieSceneSequenceHierarchy.h"

struct FGuid;
struct FMovieSceneBinding;

class UMovieSceneTrack;
class IMovieScenePlayer;
class UMovieSceneEntitySystemLinker;

namespace UE
{
namespace MovieScene
{

struct FInitialValueCache;
struct FSystemInterrogatorEntityTracker;
enum class EEntitySystemCategory : uint32;

struct FInterrogationParams
{
	FFrameTime Time;

	FInterrogationParams(FFrameTime InTime)
		: Time(InTime)
	{}
	FInterrogationParams(FFrameNumber InTime)
		: Time(InTime)
	{}
};

/**
 * A class specialized for interrogating Sequencer entity data without applying any state to objects.
 * Currently only tracks within the same time-base are supported.
 * Will not link systems that are in the custom "ExcludedFromInterrogation" category.
 * Multiple different outputs can be interrogated simultaneously by Importing tracks onto separate channels
 * allocated through AllocateChannel.
 * 
 * Systems may implement their own interrogation logic that can be run after updates to allow third-party interrogation
 * behavior for specific channels or time.
 *
 * Example usage:
 *    Interrogator.ImportTrack(MyTrack, FInterrogationChannel::Default());
 *
 *    for (int32 FrameNumber = 0; FrameNumber < 100; ++FrameNumber)
 *        Interrogator.InterrogateTime(FrameNumber);
 *
 *    Interrogator.Update();
 *
 *    UMyTrackSystem* MySystem = Interrogator.GetLinker()->FindSystem<UMyTrackSystem>();
 *    if (MySystem)
 *    {
 *        TArray<DataType> OutData;
 *        MySystem->Interrogate(FInterogationKey::Default(), OutData);
 *    }
 */
class FInterrogationChannels
{
public:

	MOVIESCENETRACKS_API FInterrogationChannels();
	MOVIESCENETRACKS_API ~FInterrogationChannels();

	MOVIESCENETRACKS_API void Reset();

	const FSparseInterrogationChannelInfo& GetSparseChannelInfo() const
	{
		return SparseChannelInfo;
	}

	/**
	 * Allocate a new interrogation channel that can be used to uniquely identify groups of tracks that animate the same property or output.
	 *
	 * @param ParentChannel     The channel that should be considered this channel's parent, or FInterrogationChannel::Invalid if there is none
	 * @return A new interrogation channel
	 */
	MOVIESCENETRACKS_API FInterrogationChannel AllocateChannel(FInterrogationChannel ParentChannel, const FMovieScenePropertyBinding& PropertyBinding);


	/**
	 * Allocate a new interrogation channel that relates to a specific object
	 */
	MOVIESCENETRACKS_API FInterrogationChannel AllocateChannel(UObject* Object, const FMovieScenePropertyBinding& PropertyBinding);


	/**
	 * Allocate a new interrogation channel that relates to a specific object
	 */
	MOVIESCENETRACKS_API FInterrogationChannel AllocateChannel(UObject* Object, FInterrogationChannel ParentChannel, const FMovieScenePropertyBinding& PropertyBinding);


	/**
	 * Allocate a new channel for a set of transform tracks that isn't bound to any particular object (but can still exist within a hierarchy)
	 *
	 * @param ParentChannel     The channel that should be considered this channel's parent, or FInterrogationChannel::Invalid if there is none
	 * @param CurrentValue      A value to use if this channel has now animated data after the interrogation
	 * @return A new interrogation channel that can be passed to ImportUnboundTrack
	 */
	MOVIESCENETRACKS_API FInterrogationChannel AllocateUnboundChannel(FInterrogationChannel ParentChannel, const FTransform& CurrentValueLocalSpace);


	/**
	 * Import the entire transform hierarchy for the specified component, including all attached parents and tracks relating to them.
	 *
	 * @param SceneComponent    The scene component to import. A new channel will be allocated for this if one does not already exist.
	 * @param InPlayer          The player interface to use for looking up object binding IDs pertaining to scene components or actors
	 * @param SequenceID        The current sequence ID for the interrogation
	 * @return The channel that was either pre-existing or allocated for SceneComponent
	 */
	MOVIESCENETRACKS_API FInterrogationChannel ImportTransformHierarchy(USceneComponent* SceneComponent);


	/**
	 * Find the existing interrogation channel for the specified object
	 *
	 * @param Object    The object to find a channel for
	 * @return The (potentially invalid) interrogation channel that the specified object is on
	 */
	MOVIESCENETRACKS_API FInterrogationChannel FindChannel(UObject* Object);


	/**
	 * Called to activate the specified channel. Inactive channels will only ever use their object's current value.
	 *
	 * @param InChannel The channel to activate
	 */
	MOVIESCENETRACKS_API void ActivateChannel(FInterrogationChannel InChannel);

	/**
	 * Called to deactivate the specified channel. Inactive channels will only ever use their object's current value.
	 *
	 * @param InChannel The channel to deactivate
	 */
	MOVIESCENETRACKS_API void DeactivateChannel(FInterrogationChannel InChannel);


	/**
	 * Add a new time to interrogate this linker at, in the time-base of the imported tracks.
	 *
	 * @param Params    The desired time to interrogate at
	 * @return A unique index identifier for the specified time, or INDEX_NONE if the maximum number have been reached
	 */
	MOVIESCENETRACKS_API int32 AddInterrogation(const FInterrogationParams& Params);

public:


	/**
	 * Retrieve the number of channels allocated
	 */
	int32 GetNumChannels() const
	{
		return ActiveChannelBits.Num();
	}


	/**
	 * Retrieve the current interrogations
	 */
	TArrayView<const FInterrogationParams> GetInterrogations() const
	{
		return Interrogations;
	}

public:

	/**
	 * Query local space transforms
	 *
	 * @param SceneComponent    The scene component to query
	 * @param OutTransforms     Array to output transforms into, one per Interrogation
	 */
	MOVIESCENETRACKS_API void QueryLocalSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, USceneComponent* SceneComponent, TArray<FIntermediate3DTransform>& OutTransforms) const;

	/**
	 * Query local space transforms
	 *
	 * @param InChannel         The channel to query
	 * @param OutTransforms     Array to output transforms into, one per Interrogation
	 */
	MOVIESCENETRACKS_API void QueryLocalSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, FInterrogationChannel InChannel, TArray<FIntermediate3DTransform>& OutTransforms) const;

	/**
	 * Query all local space transforms, even including channels that do not have any variable track data
	 *
	 * @param OutTransformsByChannel  Sparse array to receive transforms allocated by their Channel index
	 */
	MOVIESCENETRACKS_API void QueryLocalSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, TSparseArray<TArray<FIntermediate3DTransform>>& OutTransformsByChannel) const;

	/**
	 * Query a specific set of channels for their local space transforms as defined by set bits within ChannelsToQuery
	 *
	 * @param ChannelsToQuery         Bit array containing set bits for each channel to query
	 * @param OutTransformsByChannel  Sparse array to receive transforms allocated by their Channel index
	 */
	MOVIESCENETRACKS_API void QueryLocalSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, const TBitArray<>& ChannelsToQuery, TSparseArray<TArray<FIntermediate3DTransform>>& OutTransformsByChannel) const;

public:

	/**
	 * Query world space transforms for a component
	 *
	 * @param SceneComponent    The scene component to query
	 * @param OutTransforms     Array to output transforms into, one per Interrogation
	 */
	MOVIESCENETRACKS_API void QueryWorldSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, USceneComponent* SceneComponent, TArray<FTransform>& OutTransforms) const;

	/**
	 * Query world space transforms for a channel
	 *
	 * @param InChannel         The channel to query
	 * @param OutTransforms     Array to output transforms into, one per Interrogation
	 */
	MOVIESCENETRACKS_API void QueryWorldSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, FInterrogationChannel InChannel, TArray<FTransform>& OutTransforms) const;

	/**
	 * Query all world space transforms, even including channels that do not have any variable track data
	 *
	 * @param OutTransformsByChannel  Sparse array to receive transforms allocated by their Channel index
	 */
	MOVIESCENETRACKS_API void QueryWorldSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, TSparseArray<TArray<FTransform>>& OutTransformsByChannel) const;

	/**
	 * Query a specific set of channels for their world space transforms as defined by set bits within ChannelsToQuery
	 *
	 * @param ChannelsToQuery         Bit array containing set bits for each channel to query
	 * @param OutTransformsByChannel  Sparse array to receive transforms allocated by their Channel index
	 */
	MOVIESCENETRACKS_API void QueryWorldSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, const TBitArray<>& ChannelsToQuery, TSparseArray<TArray<FTransform>>& OutTransformsByChannel) const;

	MOVIESCENETRACKS_API void QueryTransformOrigins(UMovieSceneEntitySystemLinker* Linker, const FMovieSceneSequenceHierarchy* Hierarchy, TArray<FTransform>& OutTransformOrigins, TArray<FMovieSceneSequenceID>& SubsequenceHierarchy, const UObject* InstanceData) const;
	
private:

	/** Query implementation functions */
	template<typename GetOutputForChannelType>
	void QueryWorldSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, const TBitArray<>& ChannelsToQuery, GetOutputForChannelType&& OnGetOutputForChannel) const;

	template<typename GetOutputForChannelType>
	void QueryLocalSpaceTransforms(UMovieSceneEntitySystemLinker* Linker, const TBitArray<>& ChannelsToQuery, GetOutputForChannelType&& OnGetOutputForChannel) const;

protected:

	/** Map from an object to its interrogation channel */
	TMap<FObjectKey, FInterrogationChannel> ObjectToChannel;

	/** Array of information pertaining to a given channel */
	FSparseInterrogationChannelInfo SparseChannelInfo;

	/** BitArray containing set bits for any channel that has data associated with it. The number of bits (0 or 1) in this array defines how many channels are allocated */
	TBitArray<> ActiveChannelBits;

	/** An array of interrogation times */
	TArray<FInterrogationParams> Interrogations;
};



class FSystemInterrogator
	: FGCObject
	, IInterrogationExtension
{
public:
	MOVIESCENETRACKS_API FSystemInterrogator();
	MOVIESCENETRACKS_API ~FSystemInterrogator();


	/**
	 * Gets the custom system category for interrogation-specific systems
	 */
	static MOVIESCENETRACKS_API EEntitySystemCategory GetInterrogationCategory();


	/**
	 * Gets the custom system category for systems who should be excluded from interrogation linkers
	 */
	static MOVIESCENETRACKS_API EEntitySystemCategory GetExcludedFromInterrogationCategory();

	
	/**
	 * Allocate a new interrogation channel that relates to a specific object
	 */
	FInterrogationChannel AllocateChannel(UObject* Object, const FMovieScenePropertyBinding& PropertyBinding)
	{
		return Channels.AllocateChannel(Object, PropertyBinding);
	}


	/**
	 * Import a track into this linker. This will add the track to the linker's evaluation field and
	 * cause entities to be created for it at each interrogation channel (if it is relevant at such times)
	 * Must be called before InterrogateTime() and Update().
	 *
	 * @param Track      The track to import
	 * @param InChannel  The channel to import this track onto. FInterrogationChannel::Default() can be used if this interrogator is only being used for a single output.
	 * @param SequenceID The SequenceID of the subsequence if importing one.
	*/
	MOVIESCENETRACKS_API void ImportTrack(UMovieSceneTrack* Track, FInterrogationChannel InChannel, FMovieSceneSequenceID SequenceID = MovieSceneSequenceID::Invalid);


	/**
	 * Import a track into this linker. This will add the track to the linker's evaluation field and
	 * cause entities to be created for it at each interrogation channel (if it is relevant at such times)
	 * Must be called before InterrogateTime() and Update().
	 *
	 * @param Track            The track to import
	 * @param ObjectBindingID  The binding ID for the object binding that this track resides within
	 * @param InChannel        The channel to import this track onto. FInterrogationChannel::Default() can be used if this interrogator is only being used for a single output.
	 * @param SequenceID	   The SequenceID of the subsequence if importing one.
	 */
	MOVIESCENETRACKS_API void ImportTrack(UMovieSceneTrack* Track, const FGuid& ObjectBindingID, FInterrogationChannel InChannel, FMovieSceneSequenceID SequenceID = MovieSceneSequenceID::Invalid);


	/**
	 * Import the entire transform hierarchy for the specified component, including all attached parents and tracks relating to them.
	 *
	 * @param SceneComponent    The scene component to import. A new channel will be allocated for this if one does not already exist.
	 * @param InPlayer          The player interface to use for looking up object binding IDs pertaining to scene components or actors
	 * @param SequenceID        The current sequence ID for the interrogation
	 * @return The channel that was either pre-existing or allocated for SceneComponent
	 */
	MOVIESCENETRACKS_API FInterrogationChannel ImportTransformHierarchy(USceneComponent* SceneComponent, IMovieScenePlayer* InPlayer, FMovieSceneSequenceID SequenceID);


	/**
	 * Import any transform tracks that relate to the specified scene component, or it's AActor if it is the root
	 *
	 * @param SceneComponent    The scene component to import. A new channel will be allocated for this if one does not already exist.
	 * @param InPlayer          The player interface to use for looking up object binding IDs pertaining to scene components or actors
	 * @param SequenceID        The current sequence ID for the interrogation
	 * @return The channel that was either pre-existing or allocated for SceneComponent
	 */
	MOVIESCENETRACKS_API FInterrogationChannel ImportLocalTransforms(USceneComponent* SceneComponent, IMovieScenePlayer* InPlayer, FMovieSceneSequenceID SequenceID);


	/**
	 * Import multiple tracks into this linker. See ImporTrack above.
	 */
	void ImportTracks(TArrayView<UMovieSceneTrack* const> Tracks, FInterrogationChannel InChannel)
	{
		for (UMovieSceneTrack* Track : Tracks)
		{
			ImportTrack(Track, InChannel);
		}
	}


	/**
	 * Import multiple tracks into this linker. See ImporTrack above.
	 */
	void ImportTracks(TArrayView<UMovieSceneTrack* const> Tracks, const FGuid& ObjectBindingID, FInterrogationChannel InChannel)
	{
		for (UMovieSceneTrack* Track : Tracks)
		{
			ImportTrack(Track, ObjectBindingID, InChannel);
		}
	}


	/**
	 * Add a new time to interrogate this linker at, in the time-base of the imported tracks.
	 *
	 * @param Params    The desired time to interrogate at
	 * @return A unique index identifier for the specified time, or INDEX_NONE if the maximum number have been reached
	 */
	MOVIESCENETRACKS_API int32 AddInterrogation(const FInterrogationParams& Params);


	/**
	 * Flush this interrogator by running all the systems relevant to the current data and populating the interrogation outputs.
	 */
	MOVIESCENETRACKS_API void Update();


	/**
	 * Reset this linker back to its original state
	 */
	MOVIESCENETRACKS_API void Reset();

public:

	/**
	 * Indicate that consumers of this class require a reverse-lookup table for imported entities to be maintained such that the various FindEntity functions can be called.
	 * (Not enabled by default due to performance cost with high interrogation counts)
	 */
	MOVIESCENETRACKS_API void TrackImportedEntities(bool bInTrackImportedEntities);


	/**
	 * Find an entity given the entity's owner.
	 * @note: Must call TrackImportedEntities(true) prior to calling ImportTrack for this function to return the correct entity
	 */
	MOVIESCENETRACKS_API FMovieSceneEntityID FindEntityFromOwner(FInterrogationKey InterrogationKey, UObject* Owner, uint32 EntityID) const;


	/**
	 * Access the underlying linker used for interrogation.
	 */
	UMovieSceneEntitySystemLinker* GetLinker() const
	{
		return Linker;
	}


	/**
	 * Retrieve the number of channels allocated
	 */
	int32 GetNumChannels() const
	{
		return Channels.GetNumChannels();
	}


	/**
	 * Retrieve the current interrogations
	 */
	TArrayView<const FInterrogationParams> GetInterrogations() const
	{
		return Channels.GetInterrogations();
	}

private:

	MOVIESCENETRACKS_API void FindPropertyOutputEntityIDs(const FPropertyDefinition& PropertyDefinition, FInterrogationChannel Channel, TArray<FMovieSceneEntityID>& OutEntityIDs) const;

public:

	/**
	 * Query local space transforms
	 *
	 * @param SceneComponent    The scene component to query
	 * @param OutTransforms     Array to output transforms into, one per Interrogation
	 */
	void QueryLocalSpaceTransforms(USceneComponent* SceneComponent, TArray<FIntermediate3DTransform>& OutTransforms) const
	{
		Channels.QueryLocalSpaceTransforms(Linker, SceneComponent, OutTransforms);
	}

	/**
	 * Query local space transforms
	 *
	 * @param InChannel         The channel to query
	 * @param OutTransforms     Array to output transforms into, one per Interrogation
	 */
	void QueryLocalSpaceTransforms(FInterrogationChannel InChannel, TArray<FIntermediate3DTransform>& OutTransforms) const
	{
		Channels.QueryLocalSpaceTransforms(Linker, InChannel, OutTransforms);
	}

	/**
	 * Query all local space transforms, even including channels that do not have any variable track data
	 *
	 * @param OutTransformsByChannel  Sparse array to receive transforms allocated by their Channel index
	 */
	void QueryLocalSpaceTransforms(TSparseArray<TArray<FIntermediate3DTransform>>& OutTransformsByChannel) const
	{
		Channels.QueryLocalSpaceTransforms(Linker, OutTransformsByChannel);
	}

	/**
	 * Query a specific set of channels for their local space transforms as defined by set bits within ChannelsToQuery
	 *
	 * @param ChannelsToQuery         Bit array containing set bits for each channel to query
	 * @param OutTransformsByChannel  Sparse array to receive transforms allocated by their Channel index
	 */
	void QueryLocalSpaceTransforms(const TBitArray<>& ChannelsToQuery, TSparseArray<TArray<FIntermediate3DTransform>>& OutTransformsByChannel) const
	{
		Channels.QueryLocalSpaceTransforms(Linker, ChannelsToQuery, OutTransformsByChannel);
	}

public:

	/**
	 * Query world space transforms for a component
	 *
	 * @param SceneComponent    The scene component to query
	 * @param OutTransforms     Array to output transforms into, one per Interrogation
	 */
	void QueryWorldSpaceTransforms(USceneComponent* SceneComponent, TArray<FTransform>& OutTransforms) const
	{
		Channels.QueryWorldSpaceTransforms(Linker, SceneComponent, OutTransforms);
	}

	/**
	 * Query world space transforms for a channel
	 *
	 * @param InChannel         The channel to query
	 * @param OutTransforms     Array to output transforms into, one per Interrogation
	 */
	void QueryWorldSpaceTransforms(FInterrogationChannel InChannel, TArray<FTransform>& OutTransforms) const
	{
		Channels.QueryWorldSpaceTransforms(Linker, InChannel, OutTransforms);
	}

	/**
	 * Query all world space transforms, even including channels that do not have any variable track data
	 *
	 * @param OutTransformsByChannel  Sparse array to receive transforms allocated by their Channel index
	 */
	void QueryWorldSpaceTransforms(TSparseArray<TArray<FTransform>>& OutTransformsByChannel) const
	{
		Channels.QueryWorldSpaceTransforms(Linker, OutTransformsByChannel);
	}

	/**
	 * Query a specific set of channels for their world space transforms as defined by set bits within ChannelsToQuery
	 *
	 * @param ChannelsToQuery         Bit array containing set bits for each channel to query
	 * @param OutTransformsByChannel  Sparse array to receive transforms allocated by their Channel index
	 */
	void QueryWorldSpaceTransforms(const TBitArray<>& ChannelsToQuery, TSparseArray<TArray<FTransform>>& OutTransformsByChannel) const
	{
		Channels.QueryWorldSpaceTransforms(Linker, ChannelsToQuery, OutTransformsByChannel);
	}

	/**
	 * Query the computed value of an animated property.
	 *
	 * See the other QueryPropertyValues method description.
	 *
	 * @param InPropertyComponent		The type of property being animated on the default channel.
	 * @param OutValues					The animated values, one for each interrogation time.
	 */
	template<typename PropertyTraits>
	void QueryPropertyValues(const TPropertyComponents<PropertyTraits>& InPropertyComponents, TArray<typename PropertyTraits::StorageType>& OutValues) const
	{
		return QueryPropertyValues(InPropertyComponents, FInterrogationChannel::Default(), OutValues);
	}

	/**
	 * Query the computed value of an animated property.
	 *
	 * All the tracks imported on the given channel are expected to be animating a property of the type described
	 * by the InPropertyComponents parameter.
	 *
	 * @param InPropertyComponent		The type of property being animated on the given channel.
	 * @param InChannel					The channel on which the property is being animated.
	 * @param OutValues					The animated values, one for each interrogation time.
	 */
	template<typename PropertyTraits>
	void QueryPropertyValues(const TPropertyComponents<PropertyTraits>& InPropertyComponents, FInterrogationChannel InChannel, TArray<typename PropertyTraits::StorageType>& OutValues) const
	{
		FBuiltInComponentTypes* Components = FBuiltInComponentTypes::Get();
		const FPropertyDefinition& PropertyDefinition = Components->PropertyRegistry.GetDefinition(InPropertyComponents.CompositeID);
		TArrayView<const FPropertyCompositeDefinition> PropertyComposites = Components->PropertyRegistry.GetComposites(PropertyDefinition);

		TArray<FMovieSceneEntityID> ValueEntityIDs;
		FindPropertyOutputEntityIDs(PropertyDefinition, InChannel, ValueEntityIDs);

		const int32 NumPropertyValues = Channels.GetInterrogations().Num();
		check(ValueEntityIDs.Num() == NumPropertyValues);
		OutValues.SetNum(NumPropertyValues);

		PropertyDefinition.Handler->RebuildOperational(PropertyDefinition, PropertyComposites, ValueEntityIDs, Linker, OutValues);
	}


	/**
	 * Query the transform origin of a subsection based on the imported hiearchy
	 *
	 * @param OutTransformOrigins	The interrogated transforms for the currently focused sequence.
	 * @param SubsequenceHierarchy	The currently focused sequence hierarchy
	 * @param InstanceData			The object that contains the origin overrides for the world context object of the root sequence.
	 * 
	 */
	void QueryTransformOrigins(TArray<FTransform>& OutTransformOrigins, TArray<FMovieSceneSequenceID>& SubsequenceHierarchy, const UObject* InstanceData) const
	{
		Channels.QueryTransformOrigins(Linker, Hierarchy, OutTransformOrigins, SubsequenceHierarchy, InstanceData);
	}

public:

	const FSparseInterrogationChannelInfo& GetSparseChannelInfo() const override
	{
		return Channels.GetSparseChannelInfo();
	}

	virtual const FMovieSceneSequenceHierarchy* GetHierarchy() const override
	{
		return Hierarchy;
	}

	void SetHierarchy(FMovieSceneSequenceHierarchy* InHierarchy)
	{
		Hierarchy = InHierarchy;
	}

private:

	/**
	 * Import transform tracks from this binding
	 */
	MOVIESCENETRACKS_API void ImportTransformTracks(const FMovieSceneBinding& Binding, FInterrogationChannel Channel);

	/**
	 * Import an entity for the specified query and index
	 */
	MOVIESCENETRACKS_API void InterrogateEntity(int32 InterrogationIndex, const FMovieSceneEvaluationFieldEntityQuery& Query);


private:

	MOVIESCENETRACKS_API virtual void AddReferencedObjects(FReferenceCollector& Collector);
	MOVIESCENETRACKS_API virtual FString GetReferencerName() const;

protected:

	/** Object <-> Channel tracker */
	FInterrogationChannels Channels;

	/** Scratch buffer used for generating entities for interrogation times */
	FMovieSceneEvaluationFieldEntitySet EntitiesScratch;

	/** Entity component field containing all the entity owners relevant at specific times */
	FMovieSceneEntityComponentField EntityComponentField;

	/** Tracker class that is used for keeping track of imported entities */
	TUniquePtr<FSystemInterrogatorEntityTracker> EntityTracker;

	/** The linker we own */
	TObjectPtr<UMovieSceneEntitySystemLinker> Linker;

	/** Initial value cache */
	TSharedPtr<FInitialValueCache> InitialValueCache;

	/** SubSection Hiearchy, for interrogating subsequence data e.g. subsequence transform origins */
	FMovieSceneSequenceHierarchy* Hierarchy;

	/** Array of shared extra metadata on entities */
	struct FExtraMetaData
	{
		UE::MovieScene::FInterrogationInstance InterrogationInstance;
		UE::MovieScene::FInterrogationChannel InterrogationChannel;
		FMovieSceneSequenceID InterrogationSequenceID;
	};
	TSparseArray<FExtraMetaData> ExtraMetaData;
};


} // namespace MovieScene
} // namespace UE
