// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Engine/EngineTypes.h"
#include "Evaluation/MovieScenePlaybackCapabilities.h"

class UCameraComponent;

namespace UE::MovieScene
{
	/**
	 * Parameter struct for notifying a playback client that a camera cut has happened.
	 */
	struct FOnCameraCutUpdatedParams
	{
		AActor* ViewTarget = nullptr;
		UCameraComponent* ViewTargetCamera = nullptr;
		bool bIsJumpCut = true;
	};

	/**
	 * Playback capability for sequences that can run camera cuts.
	 */
	struct FCameraCutPlaybackCapability
	{
		UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENETRACKS_API, FCameraCutPlaybackCapability)

		/** Whether the associated sequence should execute camera cuts */
		virtual bool ShouldUpdateCameraCut() { return true; }

		/** The play rate at which to run camera blends */
		virtual float GetCameraBlendPlayRate() { return 1.f; }

		/** Whether the associated sequence requires a custom aspect ratio axis constraint */
		virtual TOptional<EAspectRatioAxisConstraint> GetAspectRatioAxisConstraintOverride() { return TOptional<EAspectRatioAxisConstraint>(); }

		/** Called when a camera cut has happened */
		virtual void OnCameraCutUpdated(const FOnCameraCutUpdatedParams& Params) {}

#if WITH_EDITOR
		/** Whether the editor should cache the pre-animated viewport position */
		virtual bool ShouldRestoreEditorViewports() { return true; }
#endif
	};

}  // namespace UE::MovieScene

