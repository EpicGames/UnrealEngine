// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Sections/MovieSceneBaseCacheSection.h"
#include "MovieSceneBaseCacheTemplate.generated.h"

/** Base cache parameters that will be used for all the base cache sections */
USTRUCT()
struct FMovieSceneBaseCacheSectionTemplateParameters
{
	GENERATED_BODY()
	FMovieSceneBaseCacheSectionTemplateParameters() {}
	FMovieSceneBaseCacheSectionTemplateParameters(FFrameNumber InSectionStartTime, FFrameNumber InSectionEndTime)
		: SectionStartTime(InSectionStartTime)
		, SectionEndTime(InSectionEndTime)
	{}

	/** Get Frame Time as Animation Time */
	MOVIESCENETRACKS_API float MapTimeToAnimation(const FMovieSceneBaseCacheParams& BaseParams, float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const;

	/** Section start time */
	UPROPERTY()
	FFrameNumber SectionStartTime;

	/** Section end time */
	UPROPERTY()
	FFrameNumber SectionEndTime;
};
