// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#if WITH_EDITOR

#include "Camera/CameraShakeBase.h"
#include "Camera/CameraTypes.h"
#include "CoreMinimal.h"
#include "UObject/GCObject.h"

class FLevelEditorViewportClient;
class UCameraShakeSourceComponent;
struct FActiveCameraShakeInfo;
struct FEditorViewportViewModifierParams;

struct FCameraShakePreviewerAddParams
{
	// The class of the shake.
	TSubclassOf<UCameraShakeBase> ShakeClass;

	// Optional shake source.
	TObjectPtr<const UCameraShakeSourceComponent> SourceComponent;

	// Start time of the shake, for scrubbing.
	float GlobalStartTime;

	// Parameters to be passed to the shake's start method.
	float Scale = 1.f;
	ECameraShakePlaySpace PlaySpace = ECameraShakePlaySpace::CameraLocal;
	FRotator UserPlaySpaceRot = FRotator::ZeroRotator;
	TOptional<float> DurationOverride;
};

/**
 * A class that owns a gameplay camera shake manager, so that we can us it to preview shakes in editor.
 */
class FCameraShakePreviewer : public FGCObject, public TSharedFromThis<FCameraShakePreviewer>
{
public:
	using FViewportFilter = TFunctionRef<bool(FLevelEditorViewportClient*)>;

	MOVIESCENETRACKS_API FCameraShakePreviewer(UWorld* InWorld);
	MOVIESCENETRACKS_API ~FCameraShakePreviewer();

	UWorld* GetWorld() const { return World; }

	MOVIESCENETRACKS_API void ModifyView(FEditorViewportViewModifierParams& Params);

	MOVIESCENETRACKS_API void RegisterViewModifiers(bool bIgnoreDuplicateRegistration = false);
	MOVIESCENETRACKS_API void RegisterViewModifiers(FViewportFilter InViewportFilter, bool bIgnoreDuplicateRegistration = false);
	MOVIESCENETRACKS_API void UnRegisterViewModifiers();

	MOVIESCENETRACKS_API void RegisterViewModifier(FLevelEditorViewportClient* ViewportClient, bool bIgnoreDuplicateRegistration = false);
	MOVIESCENETRACKS_API void UnRegisterViewModifier(FLevelEditorViewportClient* ViewportClient);

	MOVIESCENETRACKS_API void Update(float DeltaTime, bool bIsPlaying);
	MOVIESCENETRACKS_API void Scrub(float ScrubTime);

	MOVIESCENETRACKS_API UCameraShakeBase* AddCameraShake(const FCameraShakePreviewerAddParams& Params);
	MOVIESCENETRACKS_API void RemoveCameraShake(UCameraShakeBase* ShakeInstance);
	MOVIESCENETRACKS_API void RemoveAllCameraShakesFromSource(const UCameraShakeSourceComponent* SourceComponent);
	MOVIESCENETRACKS_API void RemoveAllCameraShakes();

	int32 NumActiveCameraShakes() const { return ActiveShakes.Num(); }
	MOVIESCENETRACKS_API void GetActiveCameraShakes(TArray<FActiveCameraShakeInfo>& ActiveCameraShakes) const;

	MOVIESCENETRACKS_API void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

private:
	// FGCObject interface
	MOVIESCENETRACKS_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
	virtual FString GetReferencerName() const override { return TEXT("FCameraShakePreviewer"); }

private:
	MOVIESCENETRACKS_API void OnModifyView(FEditorViewportViewModifierParams& Params);
	MOVIESCENETRACKS_API void OnLevelViewportClientListChanged();

	MOVIESCENETRACKS_API void ResetModifiers();

private:
	UWorld* World;

	TArray<FLevelEditorViewportClient*> RegisteredViewportClients;

	struct FPreviewCameraShakeInfo
	{
		FCameraShakeBaseStartParams StartParams;
		TObjectPtr<UCameraShakeBase> ShakeInstance;
		TWeakObjectPtr<const UCameraShakeSourceComponent> SourceComponent;
		float StartTime;
	};
	TArray<FPreviewCameraShakeInfo> ActiveShakes;

	TOptional<float> LastDeltaTime;
	TOptional<float> LastScrubTime;

	FVector LastLocationModifier;
	FRotator LastRotationModifier;
	float LastFOVModifier;

	TArray<FPostProcessSettings> LastPostProcessSettings;
	TArray<float> LastPostProcessBlendWeights;
};

#endif

