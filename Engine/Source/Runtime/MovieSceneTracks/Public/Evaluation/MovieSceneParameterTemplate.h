// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Stats/Stats.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneFwd.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Sections/MovieSceneParameterSection.h"
#include "IMovieScenePlayer.h"

#include "MovieSceneParameterTemplate.generated.h"

DECLARE_CYCLE_STAT(TEXT("Parameter Track Token Execute"), MovieSceneEval_ParameterTrack_TokenExecute, STATGROUP_MovieSceneEval);

/** Evaluation structure that holds evaluated values */
struct FEvaluatedParameterSectionValues
{
	FEvaluatedParameterSectionValues() = default;

	FEvaluatedParameterSectionValues(FEvaluatedParameterSectionValues&&) = default;
	FEvaluatedParameterSectionValues& operator=(FEvaluatedParameterSectionValues&&) = default;

	// Non-copyable
	FEvaluatedParameterSectionValues(const FEvaluatedParameterSectionValues&) = delete;
	FEvaluatedParameterSectionValues& operator=(const FEvaluatedParameterSectionValues&) = delete;

	/** Array of evaluated scalar values */
	TArray<FScalarParameterNameAndValue, TInlineAllocator<2>> ScalarValues;
	/** Array of evaluated bool values */
	TArray<FBoolParameterNameAndValue, TInlineAllocator<2>> BoolValues;
	/** Array of evaluated vector2D values */
	TArray<FVector2DParameterNameAndValue, TInlineAllocator<2>> Vector2DValues;
	/** Array of evaluated vector values */
	TArray<FVectorParameterNameAndValue, TInlineAllocator<2>> VectorValues;
	/** Array of evaluated color values */
	TArray<FColorParameterNameAndValue, TInlineAllocator<2>> ColorValues;
	/** Array of evaluated transform values */
	TArray<FTransformParameterNameAndValue, TInlineAllocator<2>> TransformValues;
};

/** Template that performs evaluation of parameter sections */
USTRUCT()
struct FMovieSceneParameterSectionTemplate : public FMovieSceneEvalTemplate
{
	GENERATED_BODY()

	FMovieSceneParameterSectionTemplate() {}

protected:

	/** Protected constructor to initialize from a parameter section */
	MOVIESCENETRACKS_API FMovieSceneParameterSectionTemplate(const UMovieSceneParameterSection& Section);

	/** Evaluate our curves, outputting evaluated values into the specified container */
	MOVIESCENETRACKS_API void EvaluateCurves(const FMovieSceneContext& Context, FEvaluatedParameterSectionValues& OutValues) const;

protected:

	/** The scalar parameter names and their associated curves. */
	UPROPERTY()
	TArray<FScalarParameterNameAndCurve> Scalars;

	/** The bool parameter names and their associated curves. */
	UPROPERTY()
	TArray<FBoolParameterNameAndCurve> Bools;

	/** The vector parameter names and their associated curves. */
	UPROPERTY()
	TArray<FVector2DParameterNameAndCurves> Vector2Ds;

	/** The vector parameter names and their associated curves. */
	UPROPERTY()
	TArray<FVectorParameterNameAndCurves> Vectors;

	/** The color parameter names and their associated curves. */
	UPROPERTY()
	TArray<FColorParameterNameAndCurves> Colors;

	UPROPERTY()
	TArray<FTransformParameterNameAndCurves> Transforms;
};
