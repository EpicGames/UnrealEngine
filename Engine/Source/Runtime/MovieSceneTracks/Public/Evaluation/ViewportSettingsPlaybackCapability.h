// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "CoreTypes.h"
#include "Evaluation/MovieScenePlaybackCapabilities.h"
#include "Math/Vector.h"

class FViewportClient;

struct EMovieSceneViewportParams
{
	EMovieSceneViewportParams()
	{
		FadeAmount = 0.f;
		FadeColor = FLinearColor::Black;
		bEnableColorScaling = false;
	}

	enum SetViewportParam
	{
		SVP_FadeAmount   = 0x00000001,
		SVP_FadeColor    = 0x00000002,
		SVP_ColorScaling = 0x00000004,
		SVP_All          = SVP_FadeAmount | SVP_FadeColor | SVP_ColorScaling
	};

	SetViewportParam SetWhichViewportParam;

	float FadeAmount;
	FLinearColor FadeColor;
	FVector ColorScale; 
	bool bEnableColorScaling;
};

namespace UE::MovieScene
{

/**
 * Playback capability for controlling game and editor viewports.
 */
struct FViewportSettingsPlaybackCapability
{
	UE_DECLARE_MOVIESCENE_PLAYBACK_CAPABILITY_API(MOVIESCENETRACKS_API, FViewportSettingsPlaybackCapability)

	/*
	 * Set the perspective viewport settings
	 *
	 * @param ViewportParamMap A map from the viewport client to its settings
	 */
	virtual void SetViewportSettings(const TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) = 0;

	/*
	 * Get the current perspective viewport settings
	 *
	 * @param ViewportParamMap A map from the viewport client to its settings
	 */
	virtual void GetViewportSettings(TMap<FViewportClient*, EMovieSceneViewportParams>& ViewportParamsMap) const = 0;
};

}  // UE::MovieScene

