// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"


namespace EBuiltInEvaluationGroup
{
	enum Type
	{
		PreEvaluation,
		SpawnObjects,
		PostEvaluation,
	};
}

/**
 * The public interface of the MovieSceneTracks module
 */
class IMovieSceneTracksModule
	: public IModuleInterface
{
public:

	/**
	 * Singleton-like access to IMovieSceneTracksModule
	 *
	 * @return Returns MovieSceneTracksModule singleton instance, loading the module on demand if needed
	 */
	static inline IMovieSceneTracksModule& Get()
	{
		return FModuleManager::LoadModuleChecked<IMovieSceneTracksModule>("MovieSceneTracks");
	}

	/**
	 * Checks to see if this module is loaded and ready.  It is only valid to call Get() if IsAvailable() returns true.
	 *
	 * @return True if the module is loaded and ready to use
	 */
	static inline bool IsAvailable()
	{
		return FModuleManager::Get().IsModuleLoaded("MovieSceneTracks");
	}

	/**
	 * Retrieve the name of an evaluation group
	 */
	MOVIESCENETRACKS_API static FName GetEvaluationGroupName(EBuiltInEvaluationGroup::Type InEvalGroup);
};
