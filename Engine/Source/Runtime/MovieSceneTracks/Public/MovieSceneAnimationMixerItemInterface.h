// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/Interface.h"
#include "Styling/SlateTypes.h"
#include "Textures/SlateIcon.h"
#include "MovieSceneAnimationMixerItemInterface.generated.h"

/**
 * Interface for items (sections or tracks) that can be added to Animation Mixer tracks.
 * Provides metadata for menu display and visual customization.
 */
UINTERFACE(MinimalAPI)
class UMovieSceneAnimationMixerItemInterface : public UInterface
{
	GENERATED_BODY()
};

class IMovieSceneAnimationMixerItemInterface
{
	GENERATED_BODY()

public:
	/** Color tint when displayed in mixer track */
	virtual FColor GetMixerItemTint() const { return FColor::White; }

	/** Display name for add menu entry */
	virtual FText GetDisplayName() const { return FText::GetEmpty(); }

	/** Tooltip description for add menu entry */
	virtual FText GetDescription() const { return FText::GetEmpty(); }

	/** Icon for add menu entry (default: no icon) */
	virtual FSlateIcon GetIcon() const { return FSlateIcon(); }

	/** If this item supports vertical dragging within the mixer */
	virtual bool SupportsVerticalDragging() const { return true; }

	/* Some items such as decorations and transitions are not added through the section add menu */
	virtual bool IsVisibleInAddSectionMenu() const { return true; }
};
