// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/Quat.h"
#include "Math/Rotator.h"
#include "Math/Vector.h"

namespace UE::MovieScene
{

// Euler decomposition order for rotation matching. When matching specific
// rotation axes, the matched axis should be decomposed first to avoid
// gimbal lock artifacts on the axes we care about.
enum class ERotationOrder
{
	XYZ, XZY, YXZ, YZX, ZXY, ZYX
};

// Choose the decomposition order that puts the matched axis first.
// Yaw (Z) is the most common match axis, so YXZ is the default when matching yaw.
inline ERotationOrder FindBestRotationOrder(bool bMatchRoll, bool bMatchPitch, bool bMatchYaw)
{
	if (bMatchYaw)
	{
		return ERotationOrder::YXZ;
	}
	if (bMatchPitch)
	{
		return ERotationOrder::YZX;
	}
	return ERotationOrder::XYZ;
}

inline FQuat QuatFromEuler(const FVector& XYZAnglesInDegrees, ERotationOrder RotationOrder)
{
	double X = FMath::DegreesToRadians(XYZAnglesInDegrees.X);
	double Y = FMath::DegreesToRadians(XYZAnglesInDegrees.Y);
	double Z = FMath::DegreesToRadians(XYZAnglesInDegrees.Z);

	double CosX = FMath::Cos(X * 0.5);
	double CosY = FMath::Cos(Y * 0.5);
	double CosZ = FMath::Cos(Z * 0.5);

	double SinX = FMath::Sin(X * 0.5);
	double SinY = FMath::Sin(Y * 0.5);
	double SinZ = FMath::Sin(Z * 0.5);

	switch (RotationOrder)
	{
	case ERotationOrder::XYZ:
		return FQuat(SinX * CosY * CosZ - CosX * SinY * SinZ,
			CosX * SinY * CosZ + SinX * CosY * SinZ,
			CosX * CosY * SinZ - SinX * SinY * CosZ,
			CosX * CosY * CosZ + SinX * SinY * SinZ);

	case ERotationOrder::XZY:
		return FQuat(SinX * CosY * CosZ + CosX * SinY * SinZ,
			CosX * SinY * CosZ + SinX * CosY * SinZ,
			CosX * CosY * SinZ - SinX * SinY * CosZ,
			CosX * CosY * CosZ - SinX * SinY * SinZ);

	case ERotationOrder::YXZ:
		return FQuat(SinX * CosY * CosZ - CosX * SinY * SinZ,
			CosX * SinY * CosZ + SinX * CosY * SinZ,
			CosX * CosY * SinZ + SinX * SinY * CosZ,
			CosX * CosY * CosZ - SinX * SinY * SinZ);

	case ERotationOrder::YZX:
		return FQuat(SinX * CosY * CosZ - CosX * SinY * SinZ,
			CosX * SinY * CosZ - SinX * CosY * SinZ,
			CosX * CosY * SinZ + SinX * SinY * CosZ,
			CosX * CosY * CosZ + SinX * SinY * SinZ);

	case ERotationOrder::ZXY:
		return FQuat(SinX * CosY * CosZ + CosX * SinY * SinZ,
			CosX * SinY * CosZ - SinX * CosY * SinZ,
			CosX * CosY * SinZ - SinX * SinY * CosZ,
			CosX * CosY * CosZ + SinX * SinY * SinZ);

	case ERotationOrder::ZYX:
		return FQuat(SinX * CosY * CosZ + CosX * SinY * SinZ,
			CosX * SinY * CosZ - SinX * CosY * SinZ,
			CosX * CosY * SinZ + SinX * SinY * CosZ,
			CosX * CosY * CosZ - SinX * SinY * SinZ);
	}

	return FQuat::Identity;
}

inline FVector EulerFromQuat(const FQuat& Rotation, ERotationOrder RotationOrder)
{
	double X = Rotation.X;
	double Y = Rotation.Y;
	double Z = Rotation.Z;
	double W = Rotation.W;
	double X2 = X * 2.0;
	double Y2 = Y * 2.0;
	double Z2 = Z * 2.0;
	double XX2 = X * X2;
	double XY2 = X * Y2;
	double XZ2 = X * Z2;
	double YX2 = Y * X2;
	double YY2 = Y * Y2;
	double YZ2 = Y * Z2;
	double ZX2 = Z * X2;
	double ZY2 = Z * Y2;
	double ZZ2 = Z * Z2;
	double WX2 = W * X2;
	double WY2 = W * Y2;
	double WZ2 = W * Z2;

	FVector AxisX, AxisY, AxisZ;
	AxisX.X = (1.0 - (YY2 + ZZ2));
	AxisY.X = (XY2 + WZ2);
	AxisZ.X = (XZ2 - WY2);
	AxisX.Y = (XY2 - WZ2);
	AxisY.Y = (1.0 - (XX2 + ZZ2));
	AxisZ.Y = (YZ2 + WX2);
	AxisX.Z = (XZ2 + WY2);
	AxisY.Z = (YZ2 - WX2);
	AxisZ.Z = (1.0 - (XX2 + YY2));

	FVector Result = FVector::ZeroVector;

	if (RotationOrder == ERotationOrder::XYZ)
	{
		Result.Y = FMath::Asin(-FMath::Clamp<double>(AxisZ.X, -1.0, 1.0));

		if (FMath::Abs(AxisZ.X) < 1.0 - SMALL_NUMBER)
		{
			Result.X = FMath::Atan2(AxisZ.Y, AxisZ.Z);
			Result.Z = FMath::Atan2(AxisY.X, AxisX.X);
		}
		else
		{
			Result.X = 0.0;
			Result.Z = FMath::Atan2(-AxisX.Y, AxisY.Y);
		}
	}
	else if (RotationOrder == ERotationOrder::XZY)
	{

		Result.Z = FMath::Asin(FMath::Clamp<double>(AxisY.X, -1.0, 1.0));

		if (FMath::Abs(AxisY.X) < 1.0 - SMALL_NUMBER)
		{
			Result.X = FMath::Atan2(-AxisY.Z, AxisY.Y);
			Result.Y = FMath::Atan2(-AxisZ.X, AxisX.X);
		}
		else
		{
			Result.X = 0.0;
			Result.Y = FMath::Atan2(AxisX.Z, AxisZ.Z);
		}
	}
	else if (RotationOrder == ERotationOrder::YXZ)
	{
		Result.X = FMath::Asin(FMath::Clamp<double>(AxisZ.Y, -1.0, 1.0));

		if (FMath::Abs(AxisZ.Y) < 1.0 - SMALL_NUMBER)
		{
			Result.Y = FMath::Atan2(-AxisZ.X, AxisZ.Z);
			Result.Z = FMath::Atan2(-AxisX.Y, AxisY.Y);
		}
		else
		{
			Result.Y = 0.0;
			Result.Z = FMath::Atan2(AxisY.X, AxisX.X);
		}
	}
	else if (RotationOrder == ERotationOrder::YZX)
	{
		Result.Z = FMath::Asin(-FMath::Clamp<double>(AxisX.Y, -1.0, 1.0));

		if (FMath::Abs(AxisX.Y) < 1.0 - SMALL_NUMBER)
		{
			Result.X = FMath::Atan2(AxisZ.Y, AxisY.Y);
			Result.Y = FMath::Atan2(AxisX.Z, AxisX.X);
		}
		else
		{
			Result.X = FMath::Atan2(-AxisY.Z, AxisZ.Z);
			Result.Y = 0.0;
		}
	}
	else if (RotationOrder == ERotationOrder::ZXY)
	{
		Result.X = FMath::Asin(-FMath::Clamp<double>(AxisY.Z, -1.0, 1.0));

		if (FMath::Abs(AxisY.Z) < 1.0 - SMALL_NUMBER)
		{
			Result.Y = FMath::Atan2(AxisX.Z, AxisZ.Z);
			Result.Z = FMath::Atan2(AxisY.X, AxisY.Y);
		}
		else
		{
			Result.Y = FMath::Atan2(-AxisZ.X, AxisX.X);
			Result.Z = 0.0;
		}
	}
	else if (RotationOrder == ERotationOrder::ZYX)
	{
		Result.Y = FMath::Asin(FMath::Clamp<double>(AxisX.Z, -1.0, 1.0));

		if (FMath::Abs(AxisX.Z) < 1.0 - SMALL_NUMBER)
		{
			Result.X = FMath::Atan2(-AxisY.Z, AxisZ.Z);
			Result.Z = FMath::Atan2(-AxisX.Y, AxisX.X);
		}
		else
		{
			Result.X = FMath::Atan2(AxisZ.Y, AxisY.Y);
			Result.Z = 0.0;
		}
	}

	return Result * 180.0 / DOUBLE_PI;
}

} // namespace UE::MovieScene
