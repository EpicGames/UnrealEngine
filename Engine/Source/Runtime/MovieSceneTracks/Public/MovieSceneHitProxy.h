// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HitProxies.h"
#include "UObject/WeakObjectPtr.h"
#include "Tracks/MovieScene3DTransformTrack.h"

class UMovieSceneTrack;

struct HMovieSceneKeyProxy : public HHitProxy
{
	DECLARE_HIT_PROXY( MOVIESCENETRACKS_API );

	/** The track that contains the section */
	TWeakObjectPtr<UMovieSceneTrack> MovieSceneTrack;

	/** The trajectory key data */
	FTrajectoryKey Key;

	MOVIESCENETRACKS_API HMovieSceneKeyProxy(class UMovieSceneTrack* InTrack, const FTrajectoryKey& InKey);

	virtual EMouseCursor::Type GetMouseCursor() override;
};

