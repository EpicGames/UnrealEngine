// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "MovieSceneMixedAnimationTarget.generated.h"

struct FSlateBrush;

/**
 * Defines a Mixed Anim Target that animation pose-producing tracks can send pose-producing tasks to. 
 * Multiple pose producing tasks sent to the same target will be chained and mixed based on priority, pose weight, masks, and sequence hierarchy.
 * Inheriting from this struct allows the user to create new animation target types with their own metadata and custom component types.
 * Coupled with a custom component type, these mixer pose results can be read from a custom Movie Scene ECS system and passed as appropriate to an anim system.
 *  If no target struct is assigned, a default animation target will be assigned based on the makeup of the object being animated.
 */
USTRUCT(meta=(DisplayName="Automatic"))
struct FMovieSceneMixedAnimationTarget
{
	GENERATED_BODY()

	virtual ~FMovieSceneMixedAnimationTarget() = default;

	inline friend uint32 GetTypeHash(const FMovieSceneMixedAnimationTarget& Target)
	{
		return GetTypeHash(FMovieSceneMixedAnimationTarget::StaticStruct());
	}

	bool HasFiredWarningForTarget() const { return bHasFiredWarningForTarget; }

	void SetHasFiredWarningForTarget(bool bNewHasFiredState) { bHasFiredWarningForTarget = bNewHasFiredState; }

#if WITH_EDITOR
	/** Get the icon to display for this target type in the Sequencer outliner. Returns nullptr for no icon. */
	virtual const FSlateBrush* GetIcon() const { return nullptr; }

	/** Get a short display name for this target (e.g., "Automatic", specific target node name, etc.) */
	virtual FText GetShortDisplayName() const { return NSLOCTEXT("MovieSceneMixedAnimationTarget", "Automatic", "Automatic"); }
#endif

private:
	bool bHasFiredWarningForTarget = false;
};
