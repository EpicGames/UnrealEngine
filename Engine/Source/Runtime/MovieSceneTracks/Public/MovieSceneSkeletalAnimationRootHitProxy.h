// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "HitProxies.h"
#include "UObject/WeakObjectPtr.h"

class UMovieSceneSkeletalAnimationSection;
class  USkeletalMeshComponent;

struct HMovieSceneSkeletalAnimationRootHitProxy : public HHitProxy
{
	DECLARE_HIT_PROXY( MOVIESCENETRACKS_API );

	TWeakObjectPtr<UMovieSceneSkeletalAnimationSection> AnimSection;
	TWeakObjectPtr<USkeletalMeshComponent> SkelMeshComp;

	MOVIESCENETRACKS_API HMovieSceneSkeletalAnimationRootHitProxy(UMovieSceneSkeletalAnimationSection* InSection, USkeletalMeshComponent* InSkelMeshComp);

	virtual EMouseCursor::Type GetMouseCursor() override;
};

