// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneActorReferenceChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneObjectPathChannel.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieScenePropertyMetaData.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "EntitySystem/MovieScenePropertyTraits.h"
#include "EulerTransform.h"
#include "GameFramework/Actor.h"
#include "Materials/MaterialParameters.h"
#include "Math/Range.h"
#include "Math/Transform.h"
#include "Misc/Guid.h"
#include "MovieSceneTracksPropertyTraits.h"
#include "MovieSceneTracksPropertyTypes.h"
#include "Sections/MovieSceneCameraShakeSection.h"
#include "Styling/SlateColor.h"
#include "Tracks/MovieSceneMaterialTrack.h"

#include "MovieSceneTracksComponentTypes.generated.h"

class UMaterialParameterCollection;
class UMovieScene3DTransformSection;
class UMovieSceneAudioSection;
class UMovieSceneDataLayerSection;
class UMovieSceneLevelVisibilitySection;
class UMovieSceneSkeletalAnimationSection;
struct FCameraShakeBaseStartParams;
struct FCameraShakeSourceComponentStartParams;
struct FMovieSceneObjectBindingID;

namespace UE::MovieScene
{
	struct FIntermediate3DTransform;

	template<>
	struct TPropertyMatch<FIntermediate3DTransform>
	{
		static bool SupportsProperty(const FProperty& InProperty)
		{
			return false;
		}
	};
}

/** Component data for Perlin Noise channels */
USTRUCT()
struct FPerlinNoiseParams
{
	GENERATED_BODY()

	/** The frequency of the noise, i.e. how many times per second does the noise peak */
	UPROPERTY(EditAnywhere, Category="Perlin Noise")
	float Frequency;

	/** The amplitude of the noise, which will vary between [-Amplitude, +Amplitude] */
	UPROPERTY(EditAnywhere, Category = "Perlin Noise")
	double Amplitude;

	/** Starting offset, in seconds, into the noise pattern */
	UPROPERTY(EditAnywhere, Category = "Perlin Noise")
	float Offset;

	MOVIESCENETRACKS_API FPerlinNoiseParams();
	MOVIESCENETRACKS_API FPerlinNoiseParams(float InFrequency, double InAmplitude);

	/** Generates a new random offset between [0, InMaxOffset] and sets it */
	MOVIESCENETRACKS_API void RandomizeOffset(float InMaxOffset = 100.f);
};

/** Component data for the level visibility system */
USTRUCT()
struct FLevelVisibilityComponentData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<const UMovieSceneLevelVisibilitySection> Section = nullptr;
};

/** Component data for the data layer system */
USTRUCT()
struct FMovieSceneDataLayerComponentData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<const UMovieSceneDataLayerSection> Section = nullptr;
};

/** Component data for the constraint system */
USTRUCT()
struct FConstraintComponentData
{
	GENERATED_BODY()

	UPROPERTY()
	FGuid ConstraintID;
	UMovieScene3DTransformSection* Section;
};

/** Component data for a skeletal mesh animation */
USTRUCT()
struct FMovieSceneSkeletalAnimationComponentData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMovieSceneSkeletalAnimationSection> Section;
};

/** Component data for audio tracks */
USTRUCT()
struct FMovieSceneAudioComponentData
{
	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UMovieSceneAudioSection> Section = nullptr;
};

/** Component data for camera shakes */
USTRUCT()
struct FMovieSceneCameraShakeComponentData
{
	GENERATED_BODY()

	/** The shake data from the section that created this component */
	UPROPERTY()
	FMovieSceneCameraShakeSectionData SectionData;

	/** The range of the section that created this component */
	UPROPERTY()
	FFrameNumber SectionStartTime;
	UPROPERTY()
	FFrameNumber SectionEndTime;

	/** The signature of the source section at the time the shake instance was created */
	UPROPERTY()
	FGuid SectionSignature;

	FMovieSceneCameraShakeComponentData()
	{}
	FMovieSceneCameraShakeComponentData(const FMovieSceneCameraShakeSectionData& InSectionData, const UMovieSceneSection& InSection)
		: SectionData(InSectionData)
		, SectionSignature(InSection.GetSignature())
	{
		const TRange<FFrameNumber> SectionRange = InSection.GetRange();
		SectionStartTime = SectionRange.HasLowerBound() ? SectionRange.GetLowerBoundValue() : 0;
		SectionEndTime = SectionRange.HasUpperBound() ? SectionRange.GetUpperBoundValue() : FFrameNumber(TNumericLimits<int32>::Max());
	}
};

/**
 * Component data for camera shakes created by the shake system
 * This is separate from FMovieSceneCameraShakeComponentData because that
 * one it imported from source shake sections, and our component data here
 * will be preserved on reimported entities.
 */
USTRUCT()
struct FMovieSceneCameraShakeInstanceData
{
	GENERATED_BODY()

	/** Shake instance created by the shake evaluation system */
	UPROPERTY()
	TObjectPtr<UCameraShakeBase> ShakeInstance;

	/** The signature of the source section at the time the shake instance was created */
	UPROPERTY()
	FGuid SectionSignature;

	/** Whether this instance is managed by a shake previewer */
	UPROPERTY()
	bool bManagedByPreviewer = false;
};

/**
 * Component data for audio tracks inputs
 * This provides the names of the inputs whose values are stored on the
 * same entity using the DoubleResult, StringResult, BoolResult, and
 * IntegerResult components.
 */
USTRUCT()
struct FMovieSceneAudioInputData
{
	GENERATED_BODY()

	// Number of FloatInputs slots per audio entity, each slot maps 1:1 onto a generic float channel component on the entity.
	static_assert(sizeof(UE::MovieScene::FBuiltInComponentTypes::FloatChannel) >= sizeof(UE::MovieScene::TComponentTypeID<UE::MovieScene::FSourceFloatChannel>),
		"FBuiltInComponentTypes::FloatChannel must contain at least one element for the size division to yield a meaningful channel count.");
	static constexpr int32 NumFloatChannels = sizeof(UE::MovieScene::FBuiltInComponentTypes::FloatChannel) / sizeof(UE::MovieScene::TComponentTypeID<UE::MovieScene::FSourceFloatChannel>);

	UPROPERTY()
	FName FloatInputs[NumFloatChannels];

	UPROPERTY()
	FName StringInput;

	UPROPERTY()
	FName BoolInput;

	UPROPERTY()
	FName IntInput;
};

namespace UE
{
namespace MovieScene
{

struct FSourceActorReferenceChannel
{
	FSourceActorReferenceChannel()
		: Source(nullptr)
	{}

	FSourceActorReferenceChannel(const FMovieSceneActorReferenceChannel* InSource)
		: Source(InSource)
	{}

	const FMovieSceneActorReferenceChannel* Source;
};

enum class EActorReferenceType
{
	ObjectPointer,
	SoftObjectPointer
};

struct FActorReferenceComponent
{
	FMovieSceneObjectBindingID ObjectBindingID;
	TWeakObjectPtr<AActor> WeakActor;
	EActorReferenceType Type = EActorReferenceType::ObjectPointer;
};

struct FComponentAttachParamsDestination
{
	FName SocketName    = NAME_None;
	FName ComponentName = NAME_None;

	MOVIESCENETRACKS_API USceneComponent* ResolveAttachment(AActor* InParentActor) const;
};

struct FComponentAttachParams
{
	EAttachmentRule AttachmentLocationRule  = EAttachmentRule::KeepRelative;
	EAttachmentRule AttachmentRotationRule  = EAttachmentRule::KeepRelative;
	EAttachmentRule AttachmentScaleRule     = EAttachmentRule::KeepRelative;

	MOVIESCENETRACKS_API void ApplyAttach(USceneComponent* NewAttachParent, USceneComponent* ChildComponentToAttach, const FName& SocketName) const;
};

struct FComponentDetachParams
{
	EDetachmentRule DetachmentLocationRule  = EDetachmentRule::KeepRelative;
	EDetachmentRule DetachmentRotationRule  = EDetachmentRule::KeepRelative;
	EDetachmentRule DetachmentScaleRule     = EDetachmentRule::KeepRelative;

	MOVIESCENETRACKS_API void ApplyDetach(USceneComponent* NewAttachParent, USceneComponent* ChildComponentToAttach, const FName& SocketName) const;
};

struct FAttachmentComponent
{
	FComponentAttachParamsDestination Destination;

	FComponentAttachParams AttachParams;
	FComponentDetachParams DetachParams;
};

struct FFadeComponentData
{
	FLinearColor FadeColor;
	bool bFadeAudio;
};

struct FPropertyNotifyComponentData
{
	FName NotifyFunctionName;
	TWeakObjectPtr<UFunction> WeakNotifyFunction;
	bool bNotifyFunctionCached = false;
};

struct FFloatPropertyTraits : IPropertyTraits
{
	static constexpr bool bIsComposite = false;

	using StorageType  = double;
	using CustomAccessorStorageType = float;

	using FloatTraitsImpl = TDirectPropertyTraits<float>;

	FORCEINLINE static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FFloatProperty>();
	}
	FORCEINLINE static FIntermediatePropertyValue CoercePropertyValue(const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue)
	{
		check(InProperty.IsA<FFloatProperty>());
		return FIntermediatePropertyValue::FromValue(double(*InPropertyValue.Cast<float>()));
	}
	FORCEINLINE static void UnpackChannels(float InValue, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
	{
		OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneFloatChannel>(InValue, 0), NAME_None));
	}

	static void GetObjectPropertyValue(const UObject* InObject, const FCustomPropertyAccessor& BaseCustomAccessor, double& OutValue)
	{
		const TCustomPropertyAccessor<FFloatPropertyTraits>& CustomAccessor = static_cast<const TCustomPropertyAccessor<FFloatPropertyTraits>&>(BaseCustomAccessor);
		const float GetterValue = (*CustomAccessor.Functions.Getter)(InObject);
		OutValue = (double)GetterValue;
	}
	static void GetObjectPropertyValue(const UObject* InObject, uint16 PropertyOffset, double& OutValue)
	{
		float Value;
		FloatTraitsImpl::GetObjectPropertyValue(InObject, PropertyOffset, Value);
		OutValue = (double)Value;
	}
	static void GetObjectPropertyValue(const UObject* InObject, FTrackInstancePropertyBindings* PropertyBindings, double& OutValue)
	{
		float Value;
		FloatTraitsImpl::GetObjectPropertyValue(InObject, PropertyBindings, Value);
		OutValue = (double)Value;
	}
	static void GetObjectPropertyValue(const UObject* InObject, const FName& PropertyPath, double& OutValue)
	{
		float Value;
		FloatTraitsImpl::GetObjectPropertyValue(InObject, PropertyPath, Value);
		OutValue = (double)Value;
	}

	static void SetObjectPropertyValue(UObject* InObject, const FCustomPropertyAccessor& BaseCustomAccessor, double InValue)
	{
		const TCustomPropertyAccessor<FFloatPropertyTraits>& CustomAccessor = static_cast<const TCustomPropertyAccessor<FFloatPropertyTraits>&>(BaseCustomAccessor);
		const float SetterValue = (float)InValue;
		(*CustomAccessor.Functions.Setter)(InObject, SetterValue);
	}
	static void SetObjectPropertyValue(UObject* InObject, uint16 PropertyOffset, double InValue)
	{
		const float SetterValue = (float)InValue;
		FloatTraitsImpl::SetObjectPropertyValue(InObject, PropertyOffset, SetterValue);
	}
	static void SetObjectPropertyValue(UObject* InObject, FTrackInstancePropertyBindings* PropertyBindings, double InValue)
	{
		const float SetterValue = (float)InValue;
		FloatTraitsImpl::SetObjectPropertyValue(InObject, PropertyBindings, SetterValue);
	}
};

struct FBoolPropertyTraits : IPropertyTraits
{
	static constexpr bool bIsComposite = false;

	struct FBoolMetaData
	{
		/** Size of the bitfield/bool property in bytes. Must be one of 1(uint8), 2(uint16), 4(uint32), 8(uint64), or 0 signifying that this is not a bitfield, but a regular bool.. */
		uint8 BitFieldSize = 0;
		uint8 BitIndex = 0;
	};
	using StorageType         = bool;
	using MetaDataType        = TPropertyMetaData<FBoolMetaData>;

	FORCEINLINE static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FBoolProperty>();
	}
	FORCEINLINE static FIntermediatePropertyValue CoercePropertyValue(const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue)
	{
		return FIntermediatePropertyValue::FromValue(*InPropertyValue.Cast<bool>());
	}
	FORCEINLINE static void UnpackChannels(bool bInValue, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
	{
		OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneBoolChannel>(bInValue, 0), NAME_None));
	}

	/** Property Value Getters  */
	static void GetObjectPropertyValue(const UObject* InObject, FBoolMetaData MetaData, const FCustomPropertyAccessor& BaseCustomAccessor, bool& OutValue)
	{
		const TCustomPropertyAccessor<FBoolPropertyTraits>& CustomAccessor = static_cast<const TCustomPropertyAccessor<FBoolPropertyTraits>&>(BaseCustomAccessor);
		OutValue = (*CustomAccessor.Functions.Getter)(InObject, MetaData);
	}
	static void GetObjectPropertyValue(const UObject* InObject, FBoolMetaData MetaData, uint16 PropertyOffset, bool& OutValue)
	{
		const void* PropertyAddress = reinterpret_cast<const uint8*>(InObject) + PropertyOffset;
		switch(MetaData.BitFieldSize)
		{
		case 0: OutValue = (*reinterpret_cast<const bool  *>(PropertyAddress)); return; // 0 means no bitfield
		case 1: OutValue = (*reinterpret_cast<const uint8 *>(PropertyAddress) & (uint8 (1u) << MetaData.BitIndex)); return;
		case 2: OutValue = (*reinterpret_cast<const uint16*>(PropertyAddress) & (uint16(1u) << MetaData.BitIndex)); return;
		case 4: OutValue = (*reinterpret_cast<const uint32*>(PropertyAddress) & (uint32(1u) << MetaData.BitIndex)); return;
		case 8: OutValue = (*reinterpret_cast<const uint64*>(PropertyAddress) & (uint64(1u) << MetaData.BitIndex)); return;
		}
	}
	static void GetObjectPropertyValue(const UObject* InObject, FBoolMetaData MetaData, FTrackInstancePropertyBindings* PropertyBindings, bool& OutValue)
	{
		OutValue = PropertyBindings->GetCurrentValue<bool>(*InObject);
	}
	static void GetObjectPropertyValue(const UObject* InObject, FBoolMetaData MetaData, const FName& PropertyPath, bool& OutValue)
	{
		TOptional<bool> Property = FTrackInstancePropertyBindings::StaticValue<bool>(InObject, *PropertyPath.ToString());
		if (Property)
		{
			OutValue = MoveTemp(Property.GetValue());
		}
	}

	/** Property Value Setters  */
	static void SetObjectPropertyValue(UObject* InObject, FBoolMetaData MetaData, const FCustomPropertyAccessor& BaseCustomAccessor, bool InValue)
	{
		const TCustomPropertyAccessor<FBoolPropertyTraits>& CustomAccessor = static_cast<const TCustomPropertyAccessor<FBoolPropertyTraits>&>(BaseCustomAccessor);
		(*CustomAccessor.Functions.Setter)(InObject, MetaData, InValue);
	}
	static void SetObjectPropertyValue(UObject* InObject, FBoolMetaData MetaData, uint16 PropertyOffset, bool InValue)
	{
		void* PropertyAddress = reinterpret_cast<uint8*>(InObject) + PropertyOffset;

		// Perform a branchless set by getting the current value, removing the bit, then a bitwise or with InValue in the bit's position.
		// For example, bit index 4 of a uint8 field:
		//			f(true)  = (Value & 0b11101111) | 0b00010000
		//			f(false) = (Value & 0b11101111) | 0b00000000
		switch(MetaData.BitFieldSize)
		{
		case 0: *reinterpret_cast<bool  *>(PropertyAddress) = InValue; return; // 0 means no bitfield
		case 1: *reinterpret_cast<uint8 *>(PropertyAddress) = (*reinterpret_cast<uint8 *>(PropertyAddress) & ~(uint8 (1u)  << MetaData.BitIndex)) | (uint8 (InValue) << MetaData.BitIndex); return;
		case 2: *reinterpret_cast<uint16*>(PropertyAddress) = (*reinterpret_cast<uint16*>(PropertyAddress) & ~(uint16(1u)  << MetaData.BitIndex)) | (uint16(InValue) << MetaData.BitIndex); return;
		case 4: *reinterpret_cast<uint32*>(PropertyAddress) = (*reinterpret_cast<uint32*>(PropertyAddress) & ~(uint32(1u)  << MetaData.BitIndex)) | (uint32(InValue) << MetaData.BitIndex); return;
		case 8: *reinterpret_cast<uint64*>(PropertyAddress) = (*reinterpret_cast<uint64*>(PropertyAddress) & ~(uint64(1u)  << MetaData.BitIndex)) | (uint64(InValue) << MetaData.BitIndex); return;
		}
	}
	static void SetObjectPropertyValue(UObject* InObject, FBoolMetaData MetaData, FTrackInstancePropertyBindings* PropertyBindings, bool InValue)
	{
		PropertyBindings->CallFunction<bool>(*InObject, InValue);
	}
};

struct FObjectPropertyTraits : IPropertyTraits
{
	static constexpr bool bIsComposite = false;

	struct FObjectMetadata
	{
		TObjectPtr<UClass> ObjectClass = UObject::StaticClass();
		bool bAllowsClear = true;
	};

	using StorageType = FObjectComponent;
	using MetaDataType = TPropertyMetaData<FObjectMetadata>;

	using ObjectTraitsImpl = TPropertyTraits<UObject*, FObjectComponent>;

	FORCEINLINE static bool SupportsProperty(const FProperty& InProperty)
	{
		return InProperty.IsA<FObjectPropertyBase>();
	}
	FORCEINLINE static FIntermediatePropertyValue CoercePropertyValue(const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue)
	{
		return FIntermediatePropertyValue::FromValue(FObjectComponent::Weak(*InPropertyValue.Cast<UObject*>()));
	}
	FORCEINLINE static void UnpackChannels(const FObjectComponent& InValue, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
	{
		FMovieSceneObjectPathChannelKeyValue ChannelValue = InValue.GetObject();
		OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneObjectPathChannel>(ChannelValue, 0), NAME_None));
	}

	static void GetObjectPropertyValue(const UObject* InObject, FObjectMetadata ObjectMetadata, const FCustomPropertyAccessor& BaseCustomAccessor, FObjectComponent& OutValue)
	{
		const TCustomPropertyAccessor<FObjectPropertyTraits>& CustomAccessor = static_cast<const TCustomPropertyAccessor<FObjectPropertyTraits>&>(BaseCustomAccessor);
		OutValue = (*CustomAccessor.Functions.Getter)(InObject, ObjectMetadata);
	}
	static void GetObjectPropertyValue(const UObject* InObject, FObjectMetadata ObjectMetadata, uint16 PropertyOffset, FObjectComponent& OutValue)
	{
		ObjectTraitsImpl::GetObjectPropertyValue(InObject, PropertyOffset, OutValue);
	}
	static void GetObjectPropertyValue(const UObject* InObject, FObjectMetadata ObjectMetadata, FTrackInstancePropertyBindings* PropertyBindings, FObjectComponent& OutValue)
	{
		ObjectTraitsImpl::GetObjectPropertyValue(InObject, PropertyBindings, OutValue);
	}
	static void GetObjectPropertyValue(const UObject* InObject, FObjectMetadata ObjectMetadata, const FName& PropertyPath, StorageType& OutValue)
	{
		ObjectTraitsImpl::GetObjectPropertyValue(InObject, PropertyPath, OutValue);
	}

	static bool CanAssignValue(const FObjectMetadata& ObjectMetadata, UObject* DesiredValue)
	{
		if (!ObjectMetadata.ObjectClass)
		{
			return false;
		}
		else if (!DesiredValue)
		{
			return ObjectMetadata.bAllowsClear;
		}
		else if (DesiredValue->GetClass() != nullptr)
		{
			return DesiredValue->GetClass()->IsChildOf(ObjectMetadata.ObjectClass);
		}
		return false;
	}

	static void SetObjectPropertyValue(UObject* InObject, FObjectMetadata ObjectMetadata, const FCustomPropertyAccessor& BaseCustomAccessor, const FObjectComponent& InValue)
	{
		const TCustomPropertyAccessor<FObjectPropertyTraits>& CustomAccessor = static_cast<const TCustomPropertyAccessor<FObjectPropertyTraits>&>(BaseCustomAccessor);
		(*CustomAccessor.Functions.Setter)(InObject, ObjectMetadata, InValue);
	}
	static void SetObjectPropertyValue(UObject* InObject, FObjectMetadata ObjectMetadata, uint16 PropertyOffset, const FObjectComponent& InValue)
	{
		if (CanAssignValue(ObjectMetadata, InValue.GetObject()))
		{
			ObjectTraitsImpl::SetObjectPropertyValue(InObject, PropertyOffset, InValue);
		}
	}
	static void SetObjectPropertyValue(UObject* InObject, FObjectMetadata ObjectMetadata, FTrackInstancePropertyBindings* PropertyBindings, const FObjectComponent& InValue)
	{
		if (CanAssignValue(ObjectMetadata, InValue.GetObject()))
		{
			ObjectTraitsImpl::SetObjectPropertyValue(InObject, PropertyBindings, InValue);
		}
	}
};

struct FActorReferencePropertyTraits : IPropertyTraits
{
	static constexpr bool bIsComposite = false;

	struct FActorReferenceMetaData
	{
		EActorReferenceType Type = EActorReferenceType::ObjectPointer;
	};

	using StorageType = FActorReferenceComponent;
	using MetaDataType = TPropertyMetaData<FActorReferenceMetaData>;

	using ObjectPointerTraitsImpl = TPropertyTraits<TObjectPtr<AActor>, FActorReferenceComponent>;
	using SoftObjectPointerTraitsImpl = TPropertyTraits<TSoftObjectPtr<AActor>, FActorReferenceComponent>;

	FORCEINLINE static bool SupportsProperty(const FProperty& InProperty)
	{
		if (const FObjectProperty* ObjectProperty = CastField<FObjectProperty>(&InProperty))
		{
			return ObjectProperty->PropertyClass->IsChildOf<AActor>();
		}
		else if (const FSoftObjectProperty* SoftObjectProperty = CastField<FSoftObjectProperty>(&InProperty))
		{
			return SoftObjectProperty->PropertyClass->IsChildOf<AActor>();
		}
		return false;
	}
	FORCEINLINE static FIntermediatePropertyValue CoercePropertyValue(const FProperty& InProperty, const FSourcePropertyValue& InPropertyValue)
	{
		// This will return an FActorReferenceComponent that has (probably) a valid actor pointer, but no cached binding ID for it.
		// We'll have to handle that ourselves in the track editor, since going from a property value to a channel value means 
		// converting an AActor pointer to a FMovieSceneObjectBindingID, and we don't have the context to do that here.
		FActorReferenceComponent ComponentData;
		ComponentData.WeakActor = *InPropertyValue.Cast<AActor*>();
		return FIntermediatePropertyValue::FromValue(ComponentData);
	}
	FORCEINLINE static void UnpackChannels(const FActorReferenceComponent& InValue, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues)
	{
		// We expect the given value to have been computed by UActorReferenceChannelEvaluatorSystem, which caches the binding ID along
		// with the resolved actor pointer.
		FMovieSceneActorReferenceKey ChannelValue;
		ChannelValue.Object = InValue.ObjectBindingID;
		OutUnpackedValues.Add(FUnpackedChannelValue(TIndexedChannelValue<FMovieSceneActorReferenceChannel>(ChannelValue, 0), NAME_None));
	}

	static void GetObjectPropertyValue(const UObject* InObject, FActorReferenceMetaData MetaData, const FCustomPropertyAccessor& BaseCustomAccessor, FActorReferenceComponent& OutValue)
	{
		const TCustomPropertyAccessor<FActorReferencePropertyTraits>& CustomAccessor = static_cast<const TCustomPropertyAccessor<FActorReferencePropertyTraits>&>(BaseCustomAccessor);
		OutValue = (*CustomAccessor.Functions.Getter)(InObject, MetaData);
	}
	static void GetObjectPropertyValue(const UObject* InObject, FActorReferenceMetaData MetaData, uint16 PropertyOffset, FActorReferenceComponent& OutValue)
	{
		switch (MetaData.Type)
		{
			case EActorReferenceType::ObjectPointer:
				ObjectPointerTraitsImpl::GetObjectPropertyValue(InObject, PropertyOffset, OutValue);
				break;
			case EActorReferenceType::SoftObjectPointer:
				SoftObjectPointerTraitsImpl::GetObjectPropertyValue(InObject, PropertyOffset, OutValue);
				break;
		}
	}
	static void GetObjectPropertyValue(const UObject* InObject, FActorReferenceMetaData MetaData, FTrackInstancePropertyBindings* PropertyBindings, FActorReferenceComponent& OutValue)
	{
		switch (MetaData.Type)
		{
			case EActorReferenceType::ObjectPointer:
				ObjectPointerTraitsImpl::GetObjectPropertyValue(InObject, PropertyBindings, OutValue);
				break;
			case EActorReferenceType::SoftObjectPointer:
				SoftObjectPointerTraitsImpl::GetObjectPropertyValue(InObject, PropertyBindings, OutValue);
				break;
		}
	}
	static void GetObjectPropertyValue(const UObject* InObject, FActorReferenceMetaData MetaData, const FName& PropertyPath, StorageType& OutValue)
	{
		switch (MetaData.Type)
		{
			case EActorReferenceType::ObjectPointer:
				ObjectPointerTraitsImpl::GetObjectPropertyValue(InObject, PropertyPath, OutValue);
				break;
			case EActorReferenceType::SoftObjectPointer:
				SoftObjectPointerTraitsImpl::GetObjectPropertyValue(InObject, PropertyPath, OutValue);
				break;
		}
	}

	static void SetObjectPropertyValue(UObject* InObject, FActorReferenceMetaData MetaData, const FCustomPropertyAccessor& BaseCustomAccessor, const FActorReferenceComponent& InValue)
	{
		const TCustomPropertyAccessor<FActorReferencePropertyTraits>& CustomAccessor = static_cast<const TCustomPropertyAccessor<FActorReferencePropertyTraits>&>(BaseCustomAccessor);
		(*CustomAccessor.Functions.Setter)(InObject, MetaData, InValue);
	}
	static void SetObjectPropertyValue(UObject* InObject, FActorReferenceMetaData MetaData, uint16 PropertyOffset, const FActorReferenceComponent& InValue)
	{
		switch (MetaData.Type)
		{
			case EActorReferenceType::ObjectPointer:
				ObjectPointerTraitsImpl::SetObjectPropertyValue(InObject, PropertyOffset, InValue);
				break;
			case EActorReferenceType::SoftObjectPointer:
				SoftObjectPointerTraitsImpl::SetObjectPropertyValue(InObject, PropertyOffset, InValue);
				break;
		}
	}
	static void SetObjectPropertyValue(UObject* InObject, FActorReferenceMetaData MetaData, FTrackInstancePropertyBindings* PropertyBindings, const FActorReferenceComponent& InValue)
	{
		switch (MetaData.Type)
		{
			case EActorReferenceType::ObjectPointer:
				ObjectPointerTraitsImpl::SetObjectPropertyValue(InObject, PropertyBindings, InValue);
				break;
			case EActorReferenceType::SoftObjectPointer:
				SoftObjectPointerTraitsImpl::SetObjectPropertyValue(InObject, PropertyBindings, InValue);
				break;
		}
	}
};

using FBytePropertyTraits               = TPropertyTraits<uint8, uint8>;
using FEnumPropertyTraits               = TPropertyTraits<uint8, uint8>;
using FIntPropertyTraits                = TDynamicVariantPropertyTraits<int64, int64, int32, int16, int8>;
using FDoublePropertyTraits             = TDynamicVariantPropertyTraits<double, double>;
using FTransformPropertyTraits          = TDynamicVariantPropertyTraits<FIntermediate3DTransform, FTransform>;
using FEulerTransformPropertyTraits     = TPropertyTraits<FEulerTransform, FIntermediate3DTransform>;
using FComponentTransformPropertyTraits = TDirectPropertyTraits<FIntermediate3DTransform>;
using FRotatorPropertyTraits            = TDirectPropertyTraits<FRotator>;
using FStringPropertyTraits             = TDirectPropertyTraits<FString>;
using FTextPropertyTraits               = TDirectPropertyTraits<FText>;

using FBoolParameterTraits              = TDirectPropertyTraits<bool>;
using FByteParameterTraits              = TDirectPropertyTraits<uint8>;
using FIntegerParameterTraits           = TDirectPropertyTraits<int32>;
using FFloatParameterTraits             = TPropertyTraits<float, double>;
using FColorParameterTraits             = TPropertyTraits<FLinearColor, FIntermediateColor>;
using FVector2ParameterTraits           = TPropertyTraits<FVector2f, FFloatIntermediateVector>;
using FVector3ParameterTraits           = TPropertyTraits<FVector3f, FFloatIntermediateVector>;
using FTransformParameterTraits         = TPropertyTraits<FEulerTransform, FIntermediate3DTransform>;

using FColorPropertyTraits              = TDynamicVariantPropertyTraits<FIntermediateColor, FLinearColor, FColor, FSlateColor>;
using FDoubleVectorPropertyTraits       = TDynamicVariantPropertyTraits<FDoubleIntermediateVector, FVector2D, FVector, FVector3d, FVector4d, FVector4>;
using FFloatVectorPropertyTraits        = TDynamicVariantPropertyTraits<FFloatIntermediateVector, FVector2f, FVector3f, FVector4f>;

struct FMovieSceneTracksComponentTypes
{
	MOVIESCENETRACKS_API ~FMovieSceneTracksComponentTypes();

	TPropertyComponents<FBoolPropertyTraits> Bool;
	TPropertyComponents<FBytePropertyTraits> Byte;
	TPropertyComponents<FEnumPropertyTraits> Enum;
	TPropertyComponents<FIntPropertyTraits> Integer;
	TPropertyComponents<FFloatPropertyTraits> Float;
	TPropertyComponents<FDoublePropertyTraits> Double;
	TPropertyComponents<FColorPropertyTraits> Color;
	TPropertyComponents<FFloatVectorPropertyTraits> FloatVector;
	TPropertyComponents<FDoubleVectorPropertyTraits> DoubleVector;
	TPropertyComponents<FTransformPropertyTraits> Transform;
	TPropertyComponents<FEulerTransformPropertyTraits> EulerTransform;
	TPropertyComponents<FComponentTransformPropertyTraits> ComponentTransform;
	TPropertyComponents<FRotatorPropertyTraits> Rotator;
	TPropertyComponents<FStringPropertyTraits> String;
	TPropertyComponents<FTextPropertyTraits> Text;
	TPropertyComponents<FObjectPropertyTraits> Object;
	TPropertyComponents<FActorReferencePropertyTraits> ActorReference;

	TPropertyComponents<FFloatParameterTraits> FloatParameter;
	TPropertyComponents<FColorParameterTraits> ColorParameter;

	TComponentTypeID<FSourceDoubleChannel> QuaternionRotationChannel[3];
	TComponentTypeID<FSourceDoubleChannel> RotatorChannel[3];

	TComponentTypeID<FConstraintComponentData> ConstraintChannel;

	TComponentTypeID<FSourceActorReferenceChannel> ActorReferenceChannel;
	TComponentTypeID<FActorReferenceComponent> ActorReferenceResult;

	TComponentTypeID<TWeakObjectPtr<USceneComponent>> AttachParent;
	TComponentTypeID<FAttachmentComponent> AttachComponent;
	TComponentTypeID<FMovieSceneObjectBindingID> AttachParentBinding;
	TComponentTypeID<FPerlinNoiseParams> FloatPerlinNoiseChannel;
	TComponentTypeID<FPerlinNoiseParams> DoublePerlinNoiseChannel;

	TComponentTypeID<FMovieSceneSkeletalAnimationComponentData> SkeletalAnimation;

	TComponentTypeID<FComponentMaterialInfo> ComponentMaterialInfo;

	TComponentTypeID<FName> BoolParameterName;
	TComponentTypeID<FName> ScalarParameterName;
	TComponentTypeID<FName> Vector2DParameterName;
	TComponentTypeID<FName> VectorParameterName;
	TComponentTypeID<FName> ColorParameterName;
	TComponentTypeID<FName> TransformParameterName;

	TComponentTypeID<FName> GenericParameterName;

	struct
	{
		TPropertyComponents<FBoolParameterTraits>      Bool;
		TPropertyComponents<FByteParameterTraits>      Byte;
		TPropertyComponents<FIntegerParameterTraits>   Integer;
		TPropertyComponents<FFloatParameterTraits>     Scalar;
		TPropertyComponents<FVector2ParameterTraits>   Vector2;
		TPropertyComponents<FVector3ParameterTraits>   Vector3;
		TPropertyComponents<FColorParameterTraits>     Color;
		TPropertyComponents<FTransformParameterTraits> Transform;
	} Parameters;

	TComponentTypeID<FMaterialParameterInfo> ScalarMaterialParameterInfo;
	TComponentTypeID<FMaterialParameterInfo> ColorMaterialParameterInfo;
	TComponentTypeID<FMaterialParameterInfo> VectorMaterialParameterInfo;

	TComponentTypeID<FObjectComponent> BoundMaterial;
	TComponentTypeID<TWeakObjectPtr<UMaterialParameterCollection>> MPC;

	TComponentTypeID<FFadeComponentData> Fade;

	TComponentTypeID<FPropertyNotifyComponentData> PropertyNotify;

	TComponentTypeID<FMovieSceneAudioComponentData> Audio;
	TComponentTypeID<FMovieSceneAudioInputData> AudioInputs;
	TComponentTypeID<FName> AudioTriggerName;

	TComponentTypeID<FMovieSceneCameraShakeComponentData> CameraShake;
	TComponentTypeID<FMovieSceneCameraShakeInstanceData> CameraShakeInstance;

	struct FObjectPropertyRegistration : TCustomPropertyRegistration<FObjectPropertyTraits>
	{
		struct FMetaData
		{
			TWeakObjectPtr<UClass> AllowedClass;
			bool bAllowsClear = true;
		};

		void Add(UClass* ClassType, FName PropertyName, GetterFunc Getter, SetterFunc Setter)
		{
			TCustomPropertyRegistration<FObjectPropertyTraits>::Add(ClassType, PropertyName, Getter, Setter);
		}

		void Add(UClass* ClassType, FName PropertyName, GetterFunc Getter, SetterFunc Setter, FMetaData InMetaData)
		{
			int32 CustomIndex = CustomAccessors.Num();
			TCustomPropertyRegistration<FObjectPropertyTraits>::Add(ClassType, PropertyName, Getter, Setter);
			MetaData.Add(CustomIndex, InMetaData);
		}

		TMap<int32, FMovieSceneTracksComponentTypes::FObjectPropertyRegistration::FMetaData> MetaData;
	};

	struct FActorReferencePropertyRegistration : TCustomPropertyRegistration<FActorReferencePropertyTraits>
	{
		struct FMetaData
		{
			EActorReferenceType Type = EActorReferenceType::ObjectPointer;
		};

		void Add(UClass* ClassType, FName PropertyName, GetterFunc Getter, SetterFunc Setter)
		{
			TCustomPropertyRegistration<FActorReferencePropertyTraits>::Add(ClassType, PropertyName, Getter, Setter);
		}

		void Add(UClass* ClassType, FName PropertyName, GetterFunc Getter, SetterFunc Setter, FMetaData InMetaData)
		{
			int32 CustomIndex = CustomAccessors.Num();
			TCustomPropertyRegistration<FActorReferencePropertyTraits>::Add(ClassType, PropertyName, Getter, Setter);
			MetaData.Add(CustomIndex, InMetaData);
		}

		TMap<int32, FMovieSceneTracksComponentTypes::FActorReferencePropertyRegistration::FMetaData> MetaData;
	};

	struct FAccessors
	{
		TCustomPropertyRegistration<FBoolPropertyTraits> Bool;
		TCustomPropertyRegistration<FBytePropertyTraits> Byte;
		TCustomPropertyRegistration<FEnumPropertyTraits> Enum;
		TCustomPropertyRegistration<FIntPropertyTraits> Integer;
		TCustomPropertyRegistration<FFloatPropertyTraits> Float;
		TCustomPropertyRegistration<FDoublePropertyTraits> Double;
		TCustomPropertyRegistration<FColorPropertyTraits> Color;
		TCustomPropertyRegistration<FFloatVectorPropertyTraits> FloatVector;
		TCustomPropertyRegistration<FDoubleVectorPropertyTraits> DoubleVector;
		TCustomPropertyRegistration<FTransformPropertyTraits> Transform;
		TCustomPropertyRegistration<FComponentTransformPropertyTraits, 1> ComponentTransform;
		FObjectPropertyRegistration Object;
		FActorReferencePropertyRegistration ActorReference;

	} Accessors;

	struct
	{
		FComponentTypeID BoundMaterialChanged;
		FComponentTypeID CustomPrimitiveData;
		FComponentTypeID Slomo;
		FComponentTypeID Visibility;
		// Tag specifying that this entity produces skeleton poses that are evaluated through the anim mixer. 
		FComponentTypeID AnimMixerPoseProducer;
	} Tags;

	TComponentTypeID<FLevelVisibilityComponentData> LevelVisibility;
	TComponentTypeID<FMovieSceneDataLayerComponentData> DataLayer;

	static MOVIESCENETRACKS_API void Destroy();

	static MOVIESCENETRACKS_API FMovieSceneTracksComponentTypes* Get();

private:
	FMovieSceneTracksComponentTypes();
};

} // namespace MovieScene
} // namespace UE
