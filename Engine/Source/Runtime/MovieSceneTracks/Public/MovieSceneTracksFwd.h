// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

namespace UE
{
namespace MovieScene
{

struct FIntermediate3DTransform;

} // namespace MovieScene
} // namespace UE
