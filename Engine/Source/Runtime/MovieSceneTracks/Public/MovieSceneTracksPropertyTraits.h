// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Math/MathFwd.h"

class FText;
class FString;
class FProperty;

namespace UE::MovieScene
{

struct FUnpackedChannelValues;
struct FIntermediateColor;
struct FIntermediate3DTransform;
struct FFloatIntermediateVector;
struct FDoubleIntermediateVector;

MOVIESCENETRACKS_API void UnpackChannelsFromOperational(uint8 Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
MOVIESCENETRACKS_API void UnpackChannelsFromOperational(int64 Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
MOVIESCENETRACKS_API void UnpackChannelsFromOperational(double Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
MOVIESCENETRACKS_API void UnpackChannelsFromOperational(float Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
MOVIESCENETRACKS_API void UnpackChannelsFromOperational(const FString& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
MOVIESCENETRACKS_API void UnpackChannelsFromOperational(const FText& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
MOVIESCENETRACKS_API void UnpackChannelsFromOperational(const FRotator& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
MOVIESCENETRACKS_API void UnpackChannelsFromOperational(const FFloatIntermediateVector& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
MOVIESCENETRACKS_API void UnpackChannelsFromOperational(const FDoubleIntermediateVector& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
MOVIESCENETRACKS_API void UnpackChannelsFromOperational(const FIntermediateColor& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);
MOVIESCENETRACKS_API void UnpackChannelsFromOperational(const FIntermediate3DTransform& Value, const FProperty& Property, FUnpackedChannelValues& OutUnpackedValues);

} // namespace UE::MovieScene
