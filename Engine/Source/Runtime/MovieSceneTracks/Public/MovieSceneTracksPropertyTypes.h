// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Math/Vector2D.h"
#include "Math/Vector.h"
#include "Math/Vector4.h"
#include "Math/Rotator.h"
#include "Math/Color.h"
#include "EulerTransform.h"
#include "Styling/SlateColor.h"
#include "EntitySystem/MovieSceneComponentDebug.h"
#include "Misc/LargeWorldCoordinates.h"

class USceneComponent;

namespace UE
{
namespace MovieScene
{

struct FObjectComponent;
struct FGeneratedTrackKeys;
struct FActorReferenceComponent;

struct FVectorPropertyMetaData
{
	uint8 NumChannels = 0;
};

/** Intermediate type for the vector property system that lets us store how many dimensions the vector should have */
struct FFloatIntermediateVector
{
	double X, Y, Z, W;

	FFloatIntermediateVector()
		: X(0), Y(0), Z(0), W(0)
	{}

	FFloatIntermediateVector(double InX, double InY)
		: X(InX), Y(InY), Z(0), W(0)
	{}

	FFloatIntermediateVector(double InX, double InY, double InZ)
		: X(InX), Y(InY), Z(InZ), W(0)
	{}

	FFloatIntermediateVector(double InX, double InY, double InZ, double InW)
		: X(InX), Y(InY), Z(InZ), W(InW)
	{}

	FVector2f AsVector2f() const
	{
		return FVector2f(static_cast<float>(X), static_cast<float>(Y));
	}
	FVector2d AsVector2d() const
	{
		return FVector2d(X, Y);
	}

	FVector3f AsVector3f() const
	{
		return FVector3f(static_cast<float>(X), static_cast<float>(Y), static_cast<float>(Z));
	}
	FVector3d AsVector3d() const
	{
		return FVector3d(X, Y, Z);
	}

	FVector4f AsVector4f() const
	{
		return FVector4f(static_cast<float>(X), static_cast<float>(Y), static_cast<float>(Z), static_cast<float>(W));
	}
	FVector4d AsVector4d() const
	{
		return FVector4d(X, Y, Z, W);
	}
};

struct FDoubleIntermediateVector
{
	double X, Y, Z, W;

	FDoubleIntermediateVector()
		: X(0), Y(0), Z(0), W(0)
	{}

	FDoubleIntermediateVector(double InX, double InY)
		: X(InX), Y(InY), Z(0), W(0)
	{}

	FDoubleIntermediateVector(double InX, double InY, double InZ)
		: X(InX), Y(InY), Z(InZ), W(0)
	{}

	FDoubleIntermediateVector(double InX, double InY, double InZ, double InW)
		: X(InX), Y(InY), Z(InZ), W(InW)
	{}
};


/** Color type for the color property system */
enum class EColorPropertyType : uint8
{
	/** Undefined */
	Undefined,
	/** FSlateColor */
	Slate, 
	/** FLinearColor */
	Linear,
	/** FColor */
	Color,
};

/** Intermediate type for the color property system that lets us store what kind of color type we should use */
struct FIntermediateColor
{
	double R, G, B, A;

	FIntermediateColor()
		: R(0.f), G(0.f), B(0.f), A(0.f)
	{}

	explicit FIntermediateColor(double InR, double InG, double InB, double InA)
		: R(InR), G(InG), B(InB), A(InA)
	{}
	explicit FIntermediateColor(const FLinearColor& InColor)
		: R(InColor.R), G(InColor.G), B(InColor.B), A(InColor.A)
	{}
	explicit FIntermediateColor(const FColor& InColor)
	{
		FLinearColor NewColor = FLinearColor::FromSRGBColor(InColor);
		R = NewColor.R;
		G = NewColor.G;
		B = NewColor.B;
		A = NewColor.A;
	}

	explicit FIntermediateColor(const FSlateColor& InSlateColor)
	{
		FLinearColor SpecifiedColor = InSlateColor.GetSpecifiedColor();
		R = SpecifiedColor.R;
		G = SpecifiedColor.G;
		B = SpecifiedColor.B;
		A = SpecifiedColor.A;
	}

	double operator[](int32 Index) const
	{
		check(Index >= 0 && Index <= 3);
		return (&R)[Index];
	}

	FColor GetColor() const
	{
		const bool bConvertBackToSRgb = true;
		const FColor SRgbColor = GetLinearColor().ToFColor(bConvertBackToSRgb);
		return SRgbColor;
	}

	FLinearColor GetLinearColor() const
	{
		return FLinearColor(static_cast<float>(R), static_cast<float>(G), static_cast<float>(B), static_cast<float>(A));
	}

	FSlateColor GetSlateColor() const
	{
		return FSlateColor(GetLinearColor());
	}
};



/** Intermediate type used for applying partially animated transforms. Saves us from repteatedly recomposing quaternions from euler angles */
struct FIntermediate3DTransform
{
	// When LWC is enabled, translations are manipulated as doubles.
	double T_X, T_Y, T_Z, R_X, R_Y, R_Z, S_X, S_Y, S_Z;

	FIntermediate3DTransform()
		: T_X(0.), T_Y(0.), T_Z(0.), R_X(0.), R_Y(0.), R_Z(0.), S_X(0.), S_Y(0.), S_Z(0.)
	{}

	FIntermediate3DTransform(
			double InT_X, double InT_Y, double InT_Z, double InR_X, double InR_Y, double InR_Z, double InS_X, double InS_Y, double InS_Z)
		: T_X(InT_X), T_Y(InT_Y), T_Z(InT_Z), R_X(InR_X), R_Y(InR_Y), R_Z(InR_Z), S_X(InS_X), S_Y(InS_Y), S_Z(InS_Z)
	{}

	FIntermediate3DTransform(const FVector& InLocation, const FRotator& InRotation, const FVector& InScale)
		: T_X(InLocation.X), T_Y(InLocation.Y), T_Z(InLocation.Z)
		, R_X(InRotation.Roll), R_Y(InRotation.Pitch), R_Z(InRotation.Yaw)
		, S_X(InScale.X), S_Y(InScale.Y), S_Z(InScale.Z)
	{}

	double operator[](int32 Index) const
	{
		check(Index >= 0 && Index < 9);
		return (&T_X)[Index];
	}

	FVector GetTranslation() const
	{
		return FVector(T_X, T_Y, T_Z);
	}
	FRotator GetRotation() const
	{
		return FRotator(R_Y, R_Z, R_X);
	}
	FVector GetScale() const
	{
		return FVector(S_X, S_Y, S_Z);
	}
	FEulerTransform AsEuler() const
	{
		return FEulerTransform(GetTranslation(), GetRotation(), GetScale());
	}

	MOVIESCENETRACKS_API void ApplyTo(USceneComponent* SceneComponent) const;
	
	MOVIESCENETRACKS_API static void ApplyTransformTo(USceneComponent* SceneComponent, const FIntermediate3DTransform& Transform);
	MOVIESCENETRACKS_API static void ApplyTranslationAndRotationTo(USceneComponent* SceneComponent, const FIntermediate3DTransform& Transform);
};

MOVIESCENETRACKS_API FIntermediate3DTransform GetComponentTransform(const UObject* Object);
MOVIESCENETRACKS_API void SetComponentTransform(USceneComponent* SceneComponent, const FIntermediate3DTransform& InTransform);
MOVIESCENETRACKS_API void SetComponentTransformAndVelocity(UObject* Object, const FIntermediate3DTransform& InTransform);

MOVIESCENETRACKS_API void ConvertOperationalProperty(float In, double& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(double In, float& Out);

MOVIESCENETRACKS_API void ConvertOperationalProperty(int64 In, int8&  Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(int64 In, int16& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(int64 In, int32& Out);

MOVIESCENETRACKS_API void ConvertOperationalProperty(const FIntermediate3DTransform& In, FEulerTransform& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FEulerTransform& In, FIntermediate3DTransform& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FIntermediate3DTransform& In, FTransform& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FTransform& In, FIntermediate3DTransform& Out);

MOVIESCENETRACKS_API void ConvertOperationalProperty(const FIntermediateColor& InColor, FColor& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FIntermediateColor& InColor, FLinearColor& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FIntermediateColor& InColor, FSlateColor& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FColor& InColor, FIntermediateColor& OutIntermediate);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FLinearColor& InColor, FIntermediateColor& OutIntermediate);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FSlateColor& InColor, FIntermediateColor& OutIntermediate);

MOVIESCENETRACKS_API void ConvertOperationalProperty(const FFloatIntermediateVector& InVector, FVector2f& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FFloatIntermediateVector& InVector, FVector3f& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FFloatIntermediateVector& InVector, FVector4f& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FVector2f& In, FFloatIntermediateVector& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FVector3f& In, FFloatIntermediateVector& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FVector4f& In, FFloatIntermediateVector& Out);

MOVIESCENETRACKS_API void ConvertOperationalProperty(const FDoubleIntermediateVector& InVector, FVector2d& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FDoubleIntermediateVector& InVector, FVector3d& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FDoubleIntermediateVector& InVector, FVector4d& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FVector2d& In, FDoubleIntermediateVector& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FVector3d& In, FDoubleIntermediateVector& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FVector4d& In, FDoubleIntermediateVector& Out);

MOVIESCENETRACKS_API void ConvertOperationalProperty(const FObjectComponent& In, UObject*& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(UObject* In, FObjectComponent& Out);

MOVIESCENETRACKS_API void ConvertOperationalProperty(const FActorReferenceComponent& InComponentData, TObjectPtr<AActor>& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const FActorReferenceComponent& InComponentData, TSoftObjectPtr<AActor>& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const TSoftObjectPtr<AActor> InActor, FActorReferenceComponent& Out);
MOVIESCENETRACKS_API void ConvertOperationalProperty(const TObjectPtr<AActor> InActor, FActorReferenceComponent& Out);

} // namespace MovieScene
} // namespace UE
