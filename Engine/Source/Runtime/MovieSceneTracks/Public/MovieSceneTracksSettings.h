// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneTracksSettings.generated.h"

#define UE_API MOVIESCENETRACKS_API

/** Options for some of the Sequencer systems in this module. */
UCLASS(MinimalAPI, config=EditorPerProjectUserSettings, PerObjectConfig)
class UMovieSceneTracksSettings : public UObject
{
	GENERATED_BODY()

public:

	UE_API UMovieSceneTracksSettings(const FObjectInitializer& ObjectInitializer);

	/**
	 * Gets whether camera cut tracks should take control of the viewport in SIE, or PIE while ejected from the player controller.
	 */
	bool GetPreviewCameraCutsInSimulate() const { return bPreviewCameraCutsInSimulate; }

	/**
	 * Sets whether camera cut tracks should take control of the viewport in SIE, or PIE while ejected from the player controller.
	 */
	UE_API void SetPreviewCameraCutsInSimulate(bool bInPreviewCameraCutsInSimulate);

protected:

	/**
	 * Whether camera cut tracks should take control of the viewport in SIE (Simulate in Editor) or after ejecting
	 * from the player controller while in PIE.
	 */
	UPROPERTY(config, EditAnywhere, Category=General)
	bool bPreviewCameraCutsInSimulate;
};

#undef UE_API
