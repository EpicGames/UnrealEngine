// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieSceneTracksPropertyTypes.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedPropertyStorage.h"

#define UE_API MOVIESCENETRACKS_API

namespace UE
{
namespace MovieScene
{

struct FComponentTransformPreAnimatedTraits : FComponentTransformPropertyTraits
{
	using KeyType     = FObjectKey;
	using StorageType = FIntermediate3DTransform;

	/** These override the functions in FComponentTransformPropertyTraits in order to wrap them with a temporary mobility assignment */
	static UE_API void SetObjectPropertyValue(UObject* InObject, const FCustomPropertyAccessor& BaseCustomAccessor, const FIntermediate3DTransform& CachedTransform);
	static UE_API void SetObjectPropertyValue(UObject* InObject, uint16 PropertyOffset, const FIntermediate3DTransform& CachedTransform);
	static UE_API void SetObjectPropertyValue(UObject* InObject, FTrackInstancePropertyBindings* PropertyBindings, const FIntermediate3DTransform& CachedTransform);
};

struct FPreAnimatedComponentTransformStorage
	: TPreAnimatedPropertyStorage<FComponentTransformPreAnimatedTraits>
{
	MOVIESCENETRACKS_API FPreAnimatedComponentTransformStorage();

	static MOVIESCENETRACKS_API TAutoRegisterPreAnimatedStorageID<FPreAnimatedComponentTransformStorage> StorageID;

	MOVIESCENETRACKS_API void CachePreAnimatedTransform(const FCachePreAnimatedValueParams& Params, UObject* BoundObject);
	MOVIESCENETRACKS_API void CachePreAnimatedTransforms(const FCachePreAnimatedValueParams& Params, TArrayView<UObject* const> BoundObjects, TOptional<TFunctionRef<bool(int32)>> Predicate = TOptional<TFunctionRef<bool(int32)>>());
};


} // namespace MovieScene
} // namespace UE

#undef UE_API
