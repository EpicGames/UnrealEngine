// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Engine/EngineTypes.h"
#include "Sections/MovieScene3DConstraintSection.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieScene3DAttachSection.generated.h"

class AActor;
class USceneComponent;

/**
 * A 3D Attach section
 */
UCLASS(MinimalAPI)
class UMovieScene3DAttachSection
	: public UMovieScene3DConstraintSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	/** 
	 * Sets the object to attach to
	 *
	 * @param InAttachBindingId The object binding id to the path
	 */
	void SetAttachTargetID(const FMovieSceneObjectBindingID& InAttachBindingID);

private:

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

public:

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Attach")
	FName AttachSocketName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Attach")
	FName AttachComponentName;


#if WITH_EDITORONLY_DATA

	UPROPERTY(transient)
	bool bFullRevertOnDetach;

	UPROPERTY(transient)
	TSoftObjectPtr<USceneComponent> ReAttachOnDetach;

#endif

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Attach")
	EAttachmentRule AttachmentLocationRule;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Attach")
	EAttachmentRule AttachmentRotationRule;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Attach")
	EAttachmentRule AttachmentScaleRule;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Attach")
	EDetachmentRule DetachmentLocationRule;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Attach")
	EDetachmentRule DetachmentRotationRule;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Attach")
	EDetachmentRule DetachmentScaleRule;
};
