// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "CoreMinimal.h"
#include "Curves/KeyHandle.h"
#include "Misc/Guid.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieScene3DConstraintSection.generated.h"

class IMovieScenePlayer;
class UObject;
struct FFrame;
struct FMovieSceneSequenceHierarchy;
struct FMovieSceneSequenceID;


/**
 * Base class for 3D constraint section
 */
UCLASS(MinimalAPI)
class UMovieScene3DConstraintSection
	: public UMovieSceneSection
{
	GENERATED_UCLASS_BODY()

public:

	/** Gets the constraint binding for this Constraint section */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	const FMovieSceneObjectBindingID& GetConstraintBindingID() const
	{
		return ConstraintBindingID;
	}

	/** Sets the constraint binding for this Constraint section */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetConstraintBindingID(const FMovieSceneObjectBindingID& InConstraintBindingID)
	{
		ConstraintBindingID = InConstraintBindingID;
	}

public:

	//~ UMovieSceneSection interface

	virtual void OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;
	
	virtual void GetReferencedBindings(TArray<FGuid>& OutBindings) override;

	/** ~UObject interface */
	virtual void PostLoad() override;

protected:

	/** The possessable guid that this constraint uses */
	UPROPERTY()
	FGuid ConstraintId_DEPRECATED;

	/** The constraint binding that this movie Constraint uses */
	UPROPERTY(EditAnywhere, Category="Section")
	FMovieSceneObjectBindingID ConstraintBindingID;

};
