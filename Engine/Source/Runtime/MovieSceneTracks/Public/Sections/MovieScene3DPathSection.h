// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Sections/MovieScene3DConstraintSection.h"
#include "MovieScene3DPathSection.generated.h"

#define UE_API MOVIESCENETRACKS_API

class USceneComponent;
class USplineComponent;

UENUM(BlueprintType)
enum class MovieScene3DPathSection_Axis : uint8
{
	X UMETA(DisplayName = "X"),
	Y UMETA(DisplayName = "Y"),
	Z UMETA(DisplayName = "Z"),
	NEG_X UMETA(DisplayName = "-X"),
	NEG_Y UMETA(DisplayName = "-Y"),
	NEG_Z UMETA(DisplayName = "-Z")
};


/**
 * A 3D Path section
 */
UCLASS(MinimalAPI)
class UMovieScene3DPathSection
	: public UMovieScene3DConstraintSection
{
	GENERATED_UCLASS_BODY()

public:

	virtual void InitialPlacement(const TArray<UMovieSceneSection*>& Sections, FFrameNumber InStartTime, int32 Duration, bool bAllowMultipleRows) override;

	/**
	 * Evaluates the path track.
	 *
	 * @param Time The position in time within the movie scene.
	 */
	UE_API void Eval(USceneComponent* SceneComponent, FFrameTime Time, USplineComponent* SplineComponent, FVector& OutTranslation, FRotator& OutRotation) const;

	/** 
	 * Sets the path binding ID
	 *
	 * @param InPathBindingID The object binding id to the path.
	 */
	void SetPathBindingID(const FMovieSceneObjectBindingID& InPathBindingID);

	MovieScene3DPathSection_Axis GetFrontAxisEnum() const { return FrontAxisEnum; }
	MovieScene3DPathSection_Axis GetUpAxisEnum() const { return UpAxisEnum; }
	bool GetFollow() const { return bFollow; }
	bool GetReverse() const { return bReverse; }
	bool GetForceUpright() const { return bForceUpright; }

public:

	const FMovieSceneFloatChannel& GetTimingChannel() const { return TimingCurve; }
	FMovieSceneFloatChannel& GetTimingChannel() { return TimingCurve; }

	/** Timing Curve */
	UPROPERTY()
	FMovieSceneFloatChannel TimingCurve;

	/** Front Axis */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Path")
	MovieScene3DPathSection_Axis FrontAxisEnum;

	/** Up Axis */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Path")
	MovieScene3DPathSection_Axis UpAxisEnum;

	/** Follow Curve */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Path")
	uint32 bFollow:1;

	/** Reverse Timing */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Path")
	uint32 bReverse:1;

	/** Force Upright */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Path")
	uint32 bForceUpright:1;
};

#undef UE_API
