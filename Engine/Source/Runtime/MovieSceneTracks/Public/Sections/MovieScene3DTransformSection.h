// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Components/SceneComponent.h"
#include "Curves/KeyHandle.h"
#include "MovieSceneSection.h"
#include "MovieSceneConstrainedSection.h"
#include "MovieSceneKeyStruct.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneSectionChannelOverrideRegistry.h"
#include "Channels/IMovieSceneChannelOverrideProvider.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "TransformData.h"
#include "Misc/LargeWorldCoordinates.h"
#include "ConstraintsManager.h"
#include "ConstraintChannel.h"
#include "MovieScene3DTransformSection.generated.h"

#if WITH_EDITORONLY_DATA
/** Visibility options for 3d trajectory. */
UENUM()
enum class EShow3DTrajectory : uint8
{
	EST_OnlyWhenSelected UMETA(DisplayName="Only When Selected"),
	EST_Always UMETA(DisplayName="Always"),
	EST_Never UMETA(DisplayName="Never"),
};
#endif


/**
 * Proxy structure for translation keys in 3D transform sections.
 */
USTRUCT()
struct FMovieScene3DLocationKeyStruct
	: public FMovieSceneKeyStruct
{
	GENERATED_BODY()

	/** The key's translation value. */
	UPROPERTY(EditAnywhere, Category=Key, meta=(Spatial))
	FVector Location = FVector::ZeroVector;

	/** The key's time. */
	UPROPERTY(EditAnywhere, Category=Key)
	FFrameNumber Time;

	FMovieSceneKeyStructHelper KeyStructInterop;

	virtual void PropagateChanges(const FPropertyChangedEvent& ChangeEvent) override;
};
template<> struct TStructOpsTypeTraits<FMovieScene3DLocationKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieScene3DLocationKeyStruct> { enum { WithCopy = false }; };


/**
 * Proxy structure for translation keys in 3D transform sections.
 */
USTRUCT()
struct FMovieScene3DRotationKeyStruct
	: public FMovieSceneKeyStruct
{
	GENERATED_BODY()

	/** The key's rotation value. */
	UPROPERTY(EditAnywhere, Category=Key)
	FRotator Rotation = FRotator::ZeroRotator;

	/** The key's time. */
	UPROPERTY(EditAnywhere, Category=Key)
	FFrameNumber Time;

	FMovieSceneKeyStructHelper KeyStructInterop;

	virtual void PropagateChanges(const FPropertyChangedEvent& ChangeEvent) override;
};
template<> struct TStructOpsTypeTraits<FMovieScene3DRotationKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieScene3DRotationKeyStruct> { enum { WithCopy = false }; };

/**
 * Proxy structure for translation keys in 3D transform sections.
 */
USTRUCT()
struct FMovieScene3DScaleKeyStruct
	: public FMovieSceneKeyStruct
{
	GENERATED_BODY()

	/** The key's scale value. */
	UPROPERTY(EditAnywhere, Category=Key, meta=(Spatial))
	FVector Scale = FVector::OneVector;

	/** The key's time. */
	UPROPERTY(EditAnywhere, Category=Key)
	FFrameNumber Time;

	FMovieSceneKeyStructHelper KeyStructInterop;

	virtual void PropagateChanges(const FPropertyChangedEvent& ChangeEvent) override;
};
template<> struct TStructOpsTypeTraits<FMovieScene3DScaleKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieScene3DScaleKeyStruct> { enum { WithCopy = false }; };


/**
 * Proxy structure for 3D transform section key data.
 */
USTRUCT()
struct FMovieScene3DTransformKeyStruct
	: public FMovieSceneKeyStruct
{
	GENERATED_BODY()

	/** The key's translation value. */
	UPROPERTY(EditAnywhere, Category=Key, meta=(Spatial))
	FVector Location = FVector::ZeroVector;

	/** The key's rotation value. */
	UPROPERTY(EditAnywhere, Category=Key)
	FRotator Rotation = FRotator::ZeroRotator;

	/** The key's scale value. */
	UPROPERTY(EditAnywhere, Category=Key, meta=(Spatial))
	FVector Scale = FVector::OneVector;

	/** The key's time. */
	UPROPERTY(EditAnywhere, Category=Key)
	FFrameNumber Time;

	FMovieSceneKeyStructHelper KeyStructInterop;

	virtual void PropagateChanges(const FPropertyChangedEvent& ChangeEvent) override;
};
template<> struct TStructOpsTypeTraits<FMovieScene3DTransformKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieScene3DTransformKeyStruct> { enum { WithCopy = false }; };


ENUM_CLASS_FLAGS(EMovieSceneTransformChannel)

USTRUCT()
struct FMovieSceneTransformMask
{
	GENERATED_BODY()

	FMovieSceneTransformMask()
		: Mask(0)
	{}

	FMovieSceneTransformMask(EMovieSceneTransformChannel Channel)
		: Mask((__underlying_type(EMovieSceneTransformChannel))Channel)
	{}

	EMovieSceneTransformChannel GetChannels() const
	{
		return (EMovieSceneTransformChannel)Mask;
	}

	FVector GetTranslationFactor() const
	{
		EMovieSceneTransformChannel Channels = GetChannels();
		return FVector(
			EnumHasAllFlags(Channels, EMovieSceneTransformChannel::TranslationX) ? 1.f : 0.f,
			EnumHasAllFlags(Channels, EMovieSceneTransformChannel::TranslationY) ? 1.f : 0.f,
			EnumHasAllFlags(Channels, EMovieSceneTransformChannel::TranslationZ) ? 1.f : 0.f);
	}

	FVector GetRotationFactor() const
	{
		EMovieSceneTransformChannel Channels = GetChannels();
		return FVector(
			EnumHasAllFlags(Channels, EMovieSceneTransformChannel::RotationX) ? 1.f : 0.f,
			EnumHasAllFlags(Channels, EMovieSceneTransformChannel::RotationY) ? 1.f : 0.f,
			EnumHasAllFlags(Channels, EMovieSceneTransformChannel::RotationZ) ? 1.f : 0.f);
	}

	FVector GetScaleFactor() const
	{
		EMovieSceneTransformChannel Channels = GetChannels();
		return FVector(
			EnumHasAllFlags(Channels, EMovieSceneTransformChannel::ScaleX) ? 1.f : 0.f,
			EnumHasAllFlags(Channels, EMovieSceneTransformChannel::ScaleY) ? 1.f : 0.f,
			EnumHasAllFlags(Channels, EMovieSceneTransformChannel::ScaleZ) ? 1.f : 0.f);
	}

private:

	UPROPERTY()
	uint32 Mask;
};

/**
* This object contains information needed for constraint channels on the transform section
*/
UCLASS(MinimalAPI)
class UMovieScene3DTransformSectionConstraints : public UObject
{
	GENERATED_BODY()

public:

	/** Constraint Channels*/
	UPROPERTY()
	TArray<FConstraintAndActiveChannel> ConstraintsChannels;

	/** When undo/redoing we need to recreate channel proxies after we are done*/
#if WITH_EDITOR
	MOVIESCENETRACKS_API virtual void PostEditUndo() override;
#endif
};

/**
 * A 3D transform section
 */
UCLASS(MinimalAPI)
class UMovieScene3DTransformSection
	: public UMovieSceneSection
	, public IMovieSceneConstrainedSection
	, public IMovieSceneEntityProvider
	, public IMovieSceneChannelOverrideProvider
{
	GENERATED_UCLASS_BODY()

public:

	virtual void PostLoad() override;

#if WITH_EDITOR
	/* From UObject*/
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
#endif

	/* From UMovieSection*/

	MOVIESCENETRACKS_API virtual bool ShowCurveForChannel(const void* Channel) const override;
	MOVIESCENETRACKS_API virtual void SetBlendType(EMovieSceneBlendType InBlendType) override;
	MOVIESCENETRACKS_API virtual void OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;
	MOVIESCENETRACKS_API virtual void GetReferencedBindings(TArray<FGuid>& OutBindings) override;
	MOVIESCENETRACKS_API virtual void PreSave(FObjectPreSaveContext SaveContext) override;

#if WITH_EDITOR
	MOVIESCENETRACKS_API virtual void PostPaste() override;
#endif

public:

	/**
	 * Access the mask that defines which channels this track should animate
	 */
	MOVIESCENETRACKS_API FMovieSceneTransformMask GetMask() const;

	/**
	 * Set the mask that defines which channels this track should animate
	 */
	MOVIESCENETRACKS_API void SetMask(FMovieSceneTransformMask NewMask);

	/**
	 * Get the mask by name
	 */
	MOVIESCENETRACKS_API FMovieSceneTransformMask GetMaskByName(const FName& InName) const;

	/**
	* Get whether we should use quaternion interpolation for our rotations.
	*/
	MOVIESCENETRACKS_API bool GetUseQuaternionInterpolation() const;

	/**
	* Set whether we should use quaternion interpolation for our rotations.
	*/
	MOVIESCENETRACKS_API void SetUseQuaternionInterpolation(bool bInUseQuaternionInterpolation);

	/**
	* Get Weight Channel
	*/
	FMovieSceneFloatChannel* GetWeightChannel() { return &ManualWeight; }
protected:

	MOVIESCENETRACKS_API virtual TSharedPtr<FStructOnScope> GetKeyStruct(TArrayView<const FKeyHandle> KeyHandles) override;
	MOVIESCENETRACKS_API virtual EMovieSceneChannelProxyType CacheChannelProxy() override;

private:

	MOVIESCENETRACKS_API virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	MOVIESCENETRACKS_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	MOVIESCENETRACKS_API virtual void InterrogateEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

	template<typename BaseBuilderType>
	void BuildEntity(BaseBuilderType& InBaseBuilder, UMovieSceneEntitySystemLinker* Linker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity);
	void PopulateConstraintEntities(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder);
	void ImportConstraintEntity(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity);

private:

	static UE::MovieScene::FChannelOverrideNames ChannelOverrideNames;

	UMovieSceneSectionChannelOverrideRegistry* GetChannelOverrideRegistry(bool bCreateIfMissing) override;
	UE::MovieScene::FChannelOverrideProviderTraitsHandle GetChannelOverrideProviderTraits() const override;
	void OnChannelOverridesChanged() override;

private:

	UPROPERTY()
	FMovieSceneTransformMask TransformMask;

	/** Translation curves */
	UPROPERTY()
	FMovieSceneDoubleChannel Translation[3];
	
	/** Rotation curves */
	UPROPERTY()
	FMovieSceneDoubleChannel Rotation[3];

	/** Scale curves */
	UPROPERTY()
	FMovieSceneDoubleChannel Scale[3];

	/** Manual weight curve */
	UPROPERTY()
	FMovieSceneFloatChannel ManualWeight;

	/** Optional pointer to a "channels override" container object. This object would only be allocated if any channels are overridden with a non-standard channel 	*/
	UPROPERTY()
	TObjectPtr<UMovieSceneSectionChannelOverrideRegistry> OverrideRegistry;

	/** Optional pointer to constraint channels*/
	UPROPERTY()
	TObjectPtr<UMovieScene3DTransformSectionConstraints> Constraints;

	/** Whether to use a quaternion linear interpolation between keys. This finds the 'shortest' rotation between keyed orientations. */
	UPROPERTY(EditAnywhere, DisplayName = "Use Quaternion Interpolation", Category = "Rotation")
	bool bUseQuaternionInterpolation;

public:
	//IMovieSceneConstrainedSection overrides

	/*
	* If it has that constraint with that Name
	*/
	virtual  bool HasConstraintChannel(const FGuid& InGuid) const override;

	/*
	* Get constraint with that name
	*/
	virtual FConstraintAndActiveChannel* GetConstraintChannel(const FGuid& InConstraintGuid) override;

	/*
	*  Add Constraint channel
	*/
	virtual void AddConstraintChannel(UTickableConstraint* InConstraint) override;
	
	/*
	*  Remove Constraint channel
	*/
	virtual void RemoveConstraintChannel(const UTickableConstraint* InConstraint) override;

	/*
	*  Get a reference to the channels
	*/
	virtual TArray<FConstraintAndActiveChannel>& GetConstraintsChannels() override;

	/*
	*  Get a const reference to the channels
	*/
	virtual const TArray<FConstraintAndActiveChannel>& GetConstraintsChannels() const override;

	/*
	*  Replace the constraint with the specified name with the new one
	*/
	virtual void ReplaceConstraint(const FName InConstraintName, UTickableConstraint* InConstraint)  override;

	/*
	* Clear proxy if changed
	*/
	virtual void OnConstraintsChanged() override;

private:

#if WITH_EDITORONLY_DATA

public:

	/**
	 * Return the trajectory visibility
	 */
	EShow3DTrajectory GetShow3DTrajectory() const { return Show3DTrajectory; }

	/**
	 * Return the trajectory visibility
	 */
	void SetShow3DTrajectory(EShow3DTrajectory InShow3DTrajectory) { Show3DTrajectory = InShow3DTrajectory; }

private:

	/** Whether to show the 3d trajectory */
	UPROPERTY(EditAnywhere, DisplayName = "Show 3D Trajectory", Category = "Transform")
	EShow3DTrajectory Show3DTrajectory;

#endif // WITH_EDITORONLY_DATA
};
