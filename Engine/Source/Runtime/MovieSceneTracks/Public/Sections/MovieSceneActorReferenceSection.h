// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneActorReferenceChannel.h"
#include "Curves/IntegralCurve.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"

#include "MovieSceneActorReferenceSection.generated.h"

struct FMovieSceneSequenceID;

/**
 * A single actor reference point section
 */
UCLASS(MinimalAPI)
class UMovieSceneActorReferenceSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	//~ UObject interface
	virtual void PostLoad() override;

	//~ UMovieSceneSection interface
	virtual void OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;

	const FMovieSceneActorReferenceChannel& GetActorReferenceData() const { return ActorReferenceData; }

private:

	// IMovieSceneEntityProvider interface
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

private:

	UPROPERTY()
	FMovieSceneActorReferenceChannel ActorReferenceData;

private:

	/** Curve data */
	UPROPERTY()
	FIntegralCurve ActorGuidIndexCurve_DEPRECATED;

	UPROPERTY()
	TArray<FString> ActorGuidStrings_DEPRECATED;
};

