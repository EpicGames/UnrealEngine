// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneActorReferenceChannel.h"
#include "Channels/MovieSceneAudioTriggerChannel.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneIntegerChannel.h"
#include "Channels/MovieSceneStringChannel.h"
#include "Components/AudioComponent.h"
#include "CoreMinimal.h"
#include "Decorations/MovieSceneScalingAnchors.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "Sound/SoundAttenuation.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneAudioSection.generated.h"

class USoundBase;

/**
 * Audio section, for use in the audio track, or by attached audio objects
 */
UCLASS(MinimalAPI)
class UMovieSceneAudioSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
	, public IMovieSceneScalingDriver
{
	GENERATED_UCLASS_BODY()

public:

	/** Sets this section's sound */
	UFUNCTION(BlueprintCallable, Category="Sequencer|Section")
	MOVIESCENETRACKS_API void SetSound(class USoundBase* InSound);

	/** Gets the sound for this section */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	class USoundBase* GetSound() const {return Sound;}

	/** Gets the sound that this section should use for playback, taking into account localization concerns */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API USoundBase* GetPlaybackSound() const;

	/** Set the offset into the beginning of the audio clip */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetStartOffset(FFrameNumber InStartOffset) {StartFrameOffset = InStartOffset;}

	/** Get the offset into the beginning of the audio clip */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	FFrameNumber GetStartOffset() const {return StartFrameOffset;}

	/**
	 * Gets the sound volume curve
	 *
	 * @return The rich curve for this sound volume
	 */
	const FMovieSceneFloatChannel& GetSoundVolumeChannel() const { return SoundVolume; }

	/**
	 * Gets the sound pitch curve
	 *
	 * @return The rich curve for this sound pitch
	 */
	const FMovieSceneFloatChannel& GetPitchMultiplierChannel() const { return PitchMultiplier; }

	/**
	 * Return the sound volume
	 *
	 * @param InTime	The position in time within the movie scene
	 * @return The volume the sound will be played with.
	 */
	float GetSoundVolume(FFrameTime InTime) const
	{
		float OutValue = 0.f;
		SoundVolume.Evaluate(InTime, OutValue);
		return OutValue;
	}

	/**
	 * Return the pitch multiplier
	 *
	 * @param Position	The position in time within the movie scene
	 * @return The pitch multiplier the sound will be played with.
	 */
	float GetPitchMultiplier(FFrameTime InTime) const
	{
		float OutValue = 0.f;
		PitchMultiplier.Evaluate(InTime, OutValue);
		return OutValue;
	}

	/**
	 * @return Whether to allow looping if the section length is greater than the sound duration
	 * Note: Prefer IsRepeating(). This accessor will be deprecated once the
	 * Sequencer.Audio.UseRepeating CVar is removed and bRepeating becomes the sole property.
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	bool GetLooping() const { return bLooping; }

	/** Set whether the sound should be looped.
	 * Note: Prefer SetRepeating(). Will be deprecated once the Sequencer.Audio.UseRepeating CVar is removed.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void SetLooping(bool bInLooping);

	/** Set whether the sound should repeat */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void SetRepeating(bool bInRepeating);

	/** Returns true if this section should repeat: reads bRepeating when CVAR is on, bLooping when off */
	MOVIESCENETRACKS_API bool IsRepeating() const;

	/** True if current sound is a natively looping asset (not one-shot) */
	MOVIESCENETRACKS_API bool IsSoundLoopingAsset() const;

	/** True if playback should continue past the sound's natural duration (repeating one-shot or natively looping asset).
	 * Handles CVar legacy fallback: returns bLooping when Sequencer.Audio.UseRepeating is off. */
	MOVIESCENETRACKS_API bool IsLoopingOrRepeating() const;

	/** Returns true if the section should play through the sound's full duration past its visual end. */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	bool GetPlayUntilFinished() const { return bPlayUntilFinished; }

	/** True only when PlayUntilFinished is both set on the section and globally enabled by the Sequencer.Audio.UsePlayUntilFinished CVar. */
	MOVIESCENETRACKS_API bool IsPlayUntilFinishedActive() const;

	/** Sets whether the section should play through the sound's full duration past its visual end. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void SetPlayUntilFinished(bool bInPlayUntilFinished);

	/**
	 * Returns the section frame at which a deterministic-duration one-shot sound naturally finishes,
	 * accounting for StartFrameOffset. Returns unset for procedural, looping, or duration-unknown sounds,
	 * or when the section has no start frame / no outer MovieScene.
	 */
	MOVIESCENETRACKS_API TOptional<FFrameNumber> ComputeDeterministicSoundEndFrame() const;

	/**
	 * @return Whether subtitles should be suppressed
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	bool GetSuppressSubtitles() const { return bSuppressSubtitles; }

	/** Set whether subtitles should be suppressed */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetSuppressSubtitles(bool bInSuppressSubtitles) { bSuppressSubtitles = bInSuppressSubtitles; }

	/**
	 * @return Whether override settings on this section should be used
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	bool GetOverrideAttenuation() const { return bOverrideAttenuation; }

	/** Set whether the attentuation should be overriden */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetOverrideAttenuation(bool bInOverrideAttenuation) { bOverrideAttenuation = bInOverrideAttenuation; }

	/**
	 * @return The attenuation settings
	 */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	USoundAttenuation* GetAttenuationSettings() const { return AttenuationSettings; }

	/** Set the attenuation settings for this audio section */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetAttenuationSettings(USoundAttenuation* InAttenuationSettings) { AttenuationSettings = InAttenuationSettings; }

	/*
	 * @return The attach actor data
	 */
	const FMovieSceneActorReferenceChannel& GetAttachActorData() const { return AttachActorData; }

	MOVIESCENETRACKS_API void SetDefaultActorKey(const FMovieSceneObjectBindingID& NewBinding);

	/*
	 * @return The attach component given the bound actor and the actor attach key with the component and socket names
	 */
	MOVIESCENETRACKS_API USceneComponent* GetAttachComponent(const AActor* InParentActor, const FMovieSceneActorReferenceKey& Key) const;

	/** ~UObject interface */
	MOVIESCENETRACKS_API virtual void Serialize(FArchive& Ar) override;
	MOVIESCENETRACKS_API virtual void PostLoad() override;
#if WITH_EDITOR
	MOVIESCENETRACKS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif //WITH_EDITOR

	/** Called when subtitles are sent to the SubtitleManager.  Set this delegate if you want to hijack the subtitles for other purposes */
	void SetOnQueueSubtitles(const FOnQueueSubtitles& InOnQueueSubtitles)
	{
		OnQueueSubtitles = InOnQueueSubtitles;
	}

	/** Called when subtitles are sent to the SubtitleManager.  Set this delegate if you want to hijack the subtitles for other purposes */
	const FOnQueueSubtitles& GetOnQueueSubtitles() const
	{
		return OnQueueSubtitles;
	}

	/** called when we finish playing audio, either because it played to completion or because a Stop() call turned it off early */
	void SetOnAudioFinished(const FOnAudioFinished& InOnAudioFinished)
	{
		OnAudioFinished = InOnAudioFinished;
	}

	/** called when we finish playing audio, either because it played to completion or because a Stop() call turned it off early */
	const FOnAudioFinished& GetOnAudioFinished() const
	{
		return OnAudioFinished;
	}
	
	void SetOnAudioPlaybackPercent(const FOnAudioPlaybackPercent& InOnAudioPlaybackPercent)
	{
		OnAudioPlaybackPercent = InOnAudioPlaybackPercent;
	}

	const FOnAudioPlaybackPercent& GetOnAudioPlaybackPercent() const
	{
		return OnAudioPlaybackPercent;
	}
	
	/** Overloads for each input type, const */
	void ForEachInput(TFunction<void(FName, const FMovieSceneBoolChannel&)> InFunction) const { ForEachInternal(InFunction, Inputs_Bool); }
	void ForEachInput(TFunction<void(FName, const FMovieSceneStringChannel&)> InFunction) const { ForEachInternal(InFunction, Inputs_String); }
	void ForEachInput(TFunction<void(FName, const FMovieSceneIntegerChannel&)> InFunction) const  { ForEachInternal(InFunction, Inputs_Int); }
	void ForEachInput(TFunction<void(FName, const FMovieSceneFloatChannel&)> InFunction) const { ForEachInternal(InFunction, Inputs_Float); }
	void ForEachInput(TFunction<void(FName, const FMovieSceneAudioTriggerChannel&)> InFunction) const { ForEachInternal(InFunction, Inputs_Trigger); }

public:

	//~ UMovieSceneSection interface
	MOVIESCENETRACKS_API virtual TOptional<TRange<FFrameNumber> > GetAutoSizeRange() const override;
	MOVIESCENETRACKS_API virtual void TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys) override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys) override;
	MOVIESCENETRACKS_API virtual TOptional<FFrameTime> GetOffsetTime() const override;
	MOVIESCENETRACKS_API virtual void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate) override;
	MOVIESCENETRACKS_API virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	MOVIESCENETRACKS_API virtual UObject* GetSourceObject() const override;

	//~ IMovieSceneEntityProvider interface
	MOVIESCENETRACKS_API virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	MOVIESCENETRACKS_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

	//~ IMovieSceneScalingDriver interface
	MOVIESCENETRACKS_API virtual void PopulateInitialAnchors(TMap<FGuid, FMovieSceneScalingAnchor>& OutAnchors) override;
	MOVIESCENETRACKS_API virtual void PopulateAnchors(TMap<FGuid, FMovieSceneScalingAnchor>& OutAnchors) override;

	/** CVar change callback -- updates all existing audio sections when Sequencer.Audio.UseRepeating changes. */
	static void OnUseRepeatingCVarChanged(IConsoleVariable* Variable);

	/** CVar change callback -- hides/shows the When Finished UI on existing sections when Sequencer.Audio.UsePlayUntilFinished flips. */
	static void OnUsePlayUntilFinishedCVarChanged(IConsoleVariable* Variable);

private:

	template<typename ChannelType, typename ForEachFunction>
	inline static void ForEachInternal(ForEachFunction InFunction, const TMap<FName, ChannelType>& InMapToIterate)
	{
		for (auto& Item : InMapToIterate)
		{
			InFunction(Item.Key, Item.Value);
		}
	}

	MOVIESCENETRACKS_API void SetupSoundInputParameters(USoundBase* InSoundBase);

	/** Reflects Sequencer.Audio.UseRepeating for EditCondition visibility of bLooping/bRepeating. */
	UFUNCTION()
	static bool IsRepeatingCVarEnabled();

	/** 
	 * Reflects Sequencer.Audio.UsePlayUntilFinished AND the sound being one-shot (and not a branching SoundCue).
	 * Used as the EditCondition for the bPlayUntilFinished UPROPERTY so the checkbox greys out when PUF can't apply. 
	 */
	UFUNCTION()
	bool IsPlayUntilFinishedEditable() const;

	/** Updates completion mode in response to looping state changes. */
	void UpdateCompletionMode();

	/** Refreshes the cached Play Until Finished compatibility flag and migrates legacy KeepState sections to bPlayUntilFinished on first load. */
	void RefreshPlayUntilFinishedCompatibilityAndMigrate();

	/** Computes the (potentially extended) evaluation range for Play Until Finished. */
	TRange<FFrameNumber> ComputePlayUntilFinishedRange() const;

	void SetSoundInternal(class USoundBase* InSound, const bool bInvalidateChannelProxy, const bool bSoundChanged);

	// Cache the users intended CompletionMode.
	// Used to revert the CompletionMode back to the users intent when not looping (when looping, completion mode is always RestoreState)
	UPROPERTY()
	EMovieSceneCompletionMode CachedCompletionMode = EMovieSceneCompletionMode::ProjectDefault;

	// Cached PUF-compatibility of the current Sound asset. Refreshed only when Sound changes, since IsPlayUntilFinishedEditable() is hit
	// per-tick by the property grid and the underlying SoundCue node walk is non-trivial.
	bool bSoundIsPlayUntilFinishedCompatible = false;

	/** The sound cue or wave that this section plays */
	UPROPERTY(EditAnywhere, Category="Audio", BlueprintGetter=GetSound, BlueprintSetter=SetSound)
	TObjectPtr<USoundBase> Sound;

	/** The offset into the beginning of the audio clip */
	UPROPERTY(EditAnywhere, Category="Audio", meta=(UIFrameDisplayAs="NonMusical|Position"))
	FFrameNumber StartFrameOffset;

	/** The offset into the beginning of the audio clip */
	UPROPERTY()
	float StartOffset_DEPRECATED;

	/** The absolute time that the sound starts playing at */
	UPROPERTY( )
	float AudioStartTime_DEPRECATED;
	
	/** The amount which this audio is time dilated by */
	UPROPERTY( )
	float AudioDilationFactor_DEPRECATED;

	/** The volume the sound will be played with. */
	UPROPERTY( )
	float AudioVolume_DEPRECATED;

	/** The volume the sound will be played with. */
	UPROPERTY( )
	FMovieSceneFloatChannel SoundVolume;

	/** The pitch multiplier the sound will be played with. */
	UPROPERTY( )
	FMovieSceneFloatChannel PitchMultiplier;

	/** Generic inputs for the sound  */
	UPROPERTY()
	TMap<FName, FMovieSceneFloatChannel> Inputs_Float;
	UPROPERTY()
	TMap<FName, FMovieSceneStringChannel> Inputs_String;
	UPROPERTY()
	TMap<FName, FMovieSceneBoolChannel> Inputs_Bool;
	UPROPERTY()
	TMap<FName, FMovieSceneIntegerChannel> Inputs_Int;
	UPROPERTY()
	TMap<FName, FMovieSceneAudioTriggerChannel> Inputs_Trigger;

	UPROPERTY()
	FMovieSceneActorReferenceChannel AttachActorData;

	/* Allow looping if the section length is greater than the sound duration */
	UPROPERTY(EditAnywhere, Category = "Audio", meta = (EditCondition = "!IsRepeatingCVarEnabled()"))
	bool bLooping;

	/** Restart a one-shot sound at frame boundaries when section length exceeds sound duration.
	  * Active when Sequencer.Audio.UseRepeating CVAR is enabled. */
	UPROPERTY(EditAnywhere, Category = "Audio", meta = (EditCondition = "IsRepeatingCVarEnabled()"))
	bool bRepeating;

	/**
	 * When true, the section's evaluation range is extended past its visual end so the sound plays
	 * to completion while remaining under sequencer control (pause, volume/pitch keyframes still apply).
	 * Only used for one-shot sounds. Active when Sequencer.Audio.UsePlayUntilFinished CVAR is enabled. 
	 */
	UPROPERTY(EditAnywhere, Category = "Audio", DisplayName = "Play Until Finished", meta = (EditCondition = "IsPlayUntilFinishedEditable()"))
	bool bPlayUntilFinished = false;

	UPROPERTY(EditAnywhere, Category = "Audio")
	bool bSuppressSubtitles;

	/** Should the attenuation settings on this section be used. */
	UPROPERTY( EditAnywhere, Category="Attenuation" )
	bool bOverrideAttenuation;

	/** The attenuation settings to use. */
	UPROPERTY( EditAnywhere, Category="Attenuation", meta = (EditCondition = "bOverrideAttenuation") )
	TObjectPtr<class USoundAttenuation> AttenuationSettings;

	/** Called when subtitles are sent to the SubtitleManager.  Set this delegate if you want to hijack the subtitles for other purposes */
	UPROPERTY()
	FOnQueueSubtitles OnQueueSubtitles;

	/** called when we finish playing audio, either because it played to completion or because a Stop() call turned it off early */
	UPROPERTY()
	FOnAudioFinished OnAudioFinished;

	UPROPERTY()
	FOnAudioPlaybackPercent OnAudioPlaybackPercent;
};
