// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"
#include "MovieSceneBaseCacheSection.generated.h"

/**
 * Base class for the cache parameters that will be used in all the cache sections
 */
USTRUCT()
struct FMovieSceneBaseCacheParams
{
	GENERATED_BODY()

	/** Constructor/Destructor */
	MOVIESCENETRACKS_API FMovieSceneBaseCacheParams();
	virtual ~FMovieSceneBaseCacheParams() = default;

	/** Gets the animation sequence length, not modified by play rate */
	virtual float GetSequenceLength() const {return 0.0f;}

	/** The offset for the first loop of the animation clip */
	UPROPERTY(EditAnywhere, Category = "Cache")
	FFrameNumber FirstLoopStartFrameOffset;

	/** The offset into the beginning of the animation clip */
	UPROPERTY(EditAnywhere, Category = "Cache")
	FFrameNumber StartFrameOffset;

	/** The offset into the end of the animation clip */
	UPROPERTY(EditAnywhere, Category = "Cache")
	FFrameNumber EndFrameOffset;

	/** The playback rate of the animation clip */
	UPROPERTY(EditAnywhere, Category = "Cache")
	float PlayRate;

	/** Reverse the playback of the animation clip */
	UPROPERTY(EditAnywhere, Category = "Cache")
	uint32 bReverse : 1;
};

/**
 * Movie scene section that control base cache playback
 */
UCLASS(MinimalAPI)
class  UMovieSceneBaseCacheSection
	: public UMovieSceneSection
{
	GENERATED_UCLASS_BODY()

public:
	/** Pointer to the concrete uproperty that will be instanced in each BaseCacheSection children */  
	FMovieSceneBaseCacheParams* ParamsPtr = nullptr;
	
	/** Get Frame Time as Animation Time*/
	MOVIESCENETRACKS_API virtual float MapTimeToAnimation(float ComponentDuration, FFrameTime InPosition, FFrameRate InTickResolution) const;
	MOVIESCENETRACKS_API virtual TOptional<TRange<FFrameNumber> > GetAutoSizeRange() const override;
	
protected:
	//~ UMovieSceneSection interface
	MOVIESCENETRACKS_API virtual void TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys) override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys) override;
	MOVIESCENETRACKS_API virtual void GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const override;
	MOVIESCENETRACKS_API virtual TOptional<FFrameTime> GetOffsetTime() const override;

#if WITH_EDITOR
	MOVIESCENETRACKS_API void InitializePlayRate();

private:
	MOVIESCENETRACKS_API virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	MOVIESCENETRACKS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
public:
	float PreviousPlayRate;

#endif
};
