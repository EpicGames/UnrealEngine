// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneByteChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneByteSection.generated.h"

/**
 * A single byte section.
 */
UCLASS(MinimalAPI)
class UMovieSceneByteSection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	/** Ordered curve data */
	UPROPERTY()
	FMovieSceneByteChannel ByteCurve;

private:

	//~ IMovieSceneEntityProvider interface
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
};
