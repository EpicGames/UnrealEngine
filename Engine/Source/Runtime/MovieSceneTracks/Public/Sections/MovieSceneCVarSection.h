// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Containers/StringFwd.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCVarSection.generated.h"

class IMovieSceneConsoleVariableTrackInterface;
class UMovieSceneEntitySystemLinker;
class UObject;
struct FFrame;


USTRUCT()
struct FMovieSceneConsoleVariableCollection
{
	GENERATED_BODY()

	/** Array of console variable preset assets that this track should operate on */
	UPROPERTY(EditAnywhere, DisplayName="Collection Asset", Category="Collection")
	TScriptInterface<IMovieSceneConsoleVariableTrackInterface> Interface;

	/** Whether to include checked consolve variables or not */
	UPROPERTY(EditAnywhere, Category="Collection")
	bool bOnlyIncludeChecked = false;
};

USTRUCT()
struct FMovieSceneCVarOverrides
{
	GENERATED_BODY()

	MOVIESCENETRACKS_API void SetFromString(const FString& InString);

	MOVIESCENETRACKS_API void GetString(TStringBuilder<256>& OutBuilder) const;
	MOVIESCENETRACKS_API FString GetString() const;

	/** The name of the console variable and the value, separated by ' ' or '=', ie: "foo.bar=1" or "foo.bar 1". */
	UPROPERTY(EditAnywhere, Category="Console Variables")
	TMap<FString, FString> ValuesByCVar;
};

/**
 * A CVar section is responsible for applying a user-supplied value to the specified cvar, and then restoring the previous value after the section ends.
 */
UCLASS(MinimalAPI)
class UMovieSceneCVarSection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

public:
	/** Constructs a new console variable section */
	UMovieSceneCVarSection(const FObjectInitializer& Init);

	UFUNCTION(BlueprintCallable, Category="CVars")
	MOVIESCENETRACKS_API void SetFromString(const FString& InString);

	UFUNCTION(BlueprintCallable, Category="CVars")
	MOVIESCENETRACKS_API FString GetString() const;

private:

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

public:

	/** Array of console variable preset assets that this track should operate on */
	UPROPERTY(EditAnywhere, Category="Console Variable Collections")
	TArray<FMovieSceneConsoleVariableCollection> ConsoleVariableCollections;

	/** The name of the console variable and the value, separated by ' ' or '=', ie: "foo.bar=1" or "foo.bar 1". */
	UPROPERTY(EditAnywhere, Category="Console Variables")
	FMovieSceneCVarOverrides ConsoleVariables;
};
