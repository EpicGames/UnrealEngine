// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Curves/KeyHandle.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "Misc/Guid.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SequencerObjectVersion.h"
#include "MovieSceneCameraCutSection.generated.h"

struct FMovieSceneSequenceID;
class IMovieScenePlayer;
class UCameraComponent;

/**
 * Movie CameraCuts are sections on the CameraCuts track, that show what the viewer "sees"
 */
UCLASS(MinimalAPI)
class UMovieSceneCameraCutSection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

public:
	/** Constructs a new camera cut section */
	UMovieSceneCameraCutSection(const FObjectInitializer& Init);

	/** Sets the camera binding for this CameraCut section. Evaluates from the sequence binding ID */
	void SetCameraGuid(const FGuid& InGuid)
	{
		SetCameraBindingID(UE::MovieScene::FRelativeObjectBindingID(InGuid));
	}

	/** Gets the camera binding for this CameraCut section */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	const FMovieSceneObjectBindingID& GetCameraBindingID() const
	{
		return CameraBindingID;
	}

	/** Sets the camera binding for this CameraCut section */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetCameraBindingID(const FMovieSceneObjectBindingID& InCameraBindingID)
	{
		CameraBindingID = InCameraBindingID;
	}

	//~ UMovieSceneSection interface
	virtual void OnBindingIDsUpdated(const TMap<UE::MovieScene::FFixedObjectBindingID, UE::MovieScene::FFixedObjectBindingID>& OldFixedToNewFixedMap, FMovieSceneSequenceID LocalSequenceID, TSharedRef<UE::MovieScene::FSharedPlaybackState> SharedPlaybackState) override;
	virtual void GetReferencedBindings(TArray<FGuid>& OutBindings) override;

	/** ~UObject interface */
	virtual void PostLoad() override;

	/**
	 * Resolve a camera component for this cut section from the specified player and sequence ID
	 *
	 * @param Player     The sequence player to use to resolve the object binding for this camera
	 * @param SequenceID The sequence ID for the specific instance that this section exists within
	 *
	 * @return A camera component to be used for this cut section, or nullptr if one was not found.
	 */
	MOVIESCENETRACKS_API UCameraComponent* GetFirstCamera(IMovieScenePlayer& Player, FMovieSceneSequenceID SequenceID) const;

#if WITH_EDITOR
	MOVIESCENETRACKS_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	/**
	 * Computes the transform of the bound camera at the section's start time.
	 * This is for internal use by UMovieSceneCameraCutTrack during pre-compilation.
	 */
	void ComputeInitialCameraCutTransform();

private:
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

public:
	/** When blending, lock the previous camera (camera cut or gameplay camera). */
	UPROPERTY(EditAnywhere, Category="Section")
	bool bLockPreviousCamera = false;

private:
	/** The camera possessable or spawnable that this movie CameraCut uses */
	UPROPERTY()
	FGuid CameraGuid_DEPRECATED;

	/** The camera binding that this movie CameraCut uses */
	UPROPERTY(EditAnywhere, Category="Section")
	FMovieSceneObjectBindingID CameraBindingID;

	/** Camera transform at the start of the cut, computed at compile time */
	UPROPERTY()
	FTransform InitialCameraCutTransform;
	UPROPERTY()
	bool bHasInitialCameraCutTransform = false;

#if WITH_EDITORONLY_DATA
public:
	/** @return The thumbnail reference frame offset from the start of this section */
	float GetThumbnailReferenceOffset() const
	{
		return ThumbnailReferenceOffset;
	}

	/** Set the thumbnail reference offset */
	void SetThumbnailReferenceOffset(float InNewOffset)
	{
		Modify();
		ThumbnailReferenceOffset = InNewOffset;
	}

private:

	/** The reference frame offset for single thumbnail rendering */
	UPROPERTY()
	float ThumbnailReferenceOffset;
#endif

	friend class UMovieSceneCameraCutTrackInstance;
};
