// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraShakeBase.h"
#include "Camera/CameraTypes.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Math/Rotator.h"
#include "MovieSceneSection.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCameraShakeSection.generated.h"

class UCameraShakeBase;
class UObject;

USTRUCT()
struct FMovieSceneCameraShakeSectionData
{
	GENERATED_BODY()

	FMovieSceneCameraShakeSectionData()
		: ShakeClass(nullptr)
		, PlayScale(1.f)
		, PlaySpace(ECameraShakePlaySpace::CameraLocal)
		, UserDefinedPlaySpace(ForceInitToZero)
	{
	}

	/** Class of the camera shake to play */
	UPROPERTY(EditAnywhere, Category = "Camera Shake")
	TSubclassOf<UCameraShakeBase> ShakeClass;

	/** Scalar that affects shake intensity */
	UPROPERTY(EditAnywhere, Category = "Camera Shake")
	float PlayScale;

	UPROPERTY(EditAnywhere, Category = "Camera Shake")
	ECameraShakePlaySpace PlaySpace;

	UPROPERTY(EditAnywhere, Category = "Camera Shake")
	FRotator UserDefinedPlaySpace;
};

UCLASS(MinimalAPI)
class UMovieSceneCameraShakeSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

public:

	UMovieSceneCameraShakeSection(const FObjectInitializer& ObjectInitializer);

	virtual void PostLoad() override;
	virtual UObject* GetSourceObject() const override;

	/** IMovieSceneEntityProvider interface */
	void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

public:

	UPROPERTY(EditAnywhere, Category="Camera Shake", meta=(ShowOnlyInnerProperties))
	FMovieSceneCameraShakeSectionData ShakeData;

public:

	UPROPERTY()
	TSubclassOf<UCameraShakeBase> ShakeClass_DEPRECATED;
	
	UPROPERTY()
	float PlayScale_DEPRECATED;

	UPROPERTY()
	ECameraShakePlaySpace PlaySpace_DEPRECATED;

	UPROPERTY()
	FRotator UserDefinedPlaySpace_DEPRECATED;
};
