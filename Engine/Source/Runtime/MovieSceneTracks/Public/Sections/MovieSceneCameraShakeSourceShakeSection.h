// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneCameraShakeSection.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCameraShakeSourceShakeSection.generated.h"

class UObject;

UCLASS(MinimalAPI)
class UMovieSceneCameraShakeSourceShakeSection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

public:
	UMovieSceneCameraShakeSourceShakeSection(const FObjectInitializer& ObjectInitializer);

	virtual UObject* GetSourceObject() const override;

	/** IMovieSceneEntityProvider interface */
	void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	
public:
	UPROPERTY(EditAnywhere, Category="Camera Shake", meta=(ShowOnlyInnerProperties))
	FMovieSceneCameraShakeSectionData ShakeData;
};

