// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraShakeBase.h"
#include "Channels/MovieSceneCameraShakeSourceTriggerChannel.h"
#include "CoreMinimal.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCameraShakeSourceTriggerSection.generated.h"

class UObject;

UCLASS(MinimalAPI)
class UMovieSceneCameraShakeSourceTriggerSection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
public:
	GENERATED_BODY()

	UMovieSceneCameraShakeSourceTriggerSection(const FObjectInitializer& Init);

	const FMovieSceneCameraShakeSourceTriggerChannel& GetChannel() const { return Channel; }

	/** IMovieSceneEntityProvider interface */
	bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

private:
	UPROPERTY()
	FMovieSceneCameraShakeSourceTriggerChannel Channel;
};

