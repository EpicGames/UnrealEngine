// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Sections/MovieSceneSubSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCinematicShotSection.generated.h"

class UObject;
struct FFrame;

/**
 * Implements a cinematic shot section.
 */
UCLASS(BlueprintType, MinimalAPI)
class UMovieSceneCinematicShotSection
	: public UMovieSceneSubSection
{
	GENERATED_BODY()

public:

	/** Object constructor. */
	MOVIESCENETRACKS_API UMovieSceneCinematicShotSection(const FObjectInitializer& ObjInitializer);

private:

	/** ~UObject interface */
	MOVIESCENETRACKS_API virtual void PostLoad() override;

public:

	/** @return The shot display name. if empty, returns the sequence's name*/
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API FString GetShotDisplayName() const;

	/** Set the shot display name */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetShotDisplayName(const FString& InShotDisplayName)
	{
		if (TryModify())
		{
			ShotDisplayName = InShotDisplayName;
		}
	}

private:

	/** The Shot's display name */
	UPROPERTY()
	FString ShotDisplayName;

	/** The Shot's display name */
	UPROPERTY()
	FText DisplayName_DEPRECATED;

#if WITH_EDITORONLY_DATA
public:
	/** @return The shot thumbnail reference frame offset from the start of this section */
	float GetThumbnailReferenceOffset() const
	{
		return ThumbnailReferenceOffset;
	}

	/** Set the thumbnail reference offset */
	void SetThumbnailReferenceOffset(float InNewOffset)
	{
		Modify();
		ThumbnailReferenceOffset = InNewOffset;
	}

private:

	/** The shot's reference frame offset for single thumbnail rendering */
	UPROPERTY()
	float ThumbnailReferenceOffset;
#endif
};
