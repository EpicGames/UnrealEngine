// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "MovieSceneSection.h"
#include "MovieSceneKeyStruct.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneColorSection.generated.h"

struct FPropertyChangedEvent;

/**
 * Proxy structure for color section key data.
 */
USTRUCT()
struct FMovieSceneColorKeyStruct
	: public FMovieSceneKeyStruct
{
	GENERATED_BODY()

	/** The key's color value. */
	UPROPERTY(EditAnywhere, Category=Key, meta=(InlineColorPicker))
	FLinearColor Color = FLinearColor(0.f, 0.f, 0.f, 1.f);

	/** The key's time. */
	UPROPERTY(EditAnywhere, Category=Key)
	FFrameNumber Time;

	FMovieSceneKeyStructHelper KeyStructInterop;

	virtual void PropagateChanges(const FPropertyChangedEvent& ChangeEvent) override;
};
template<> struct TStructOpsTypeTraits<FMovieSceneColorKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieSceneColorKeyStruct> { enum { WithCopy = false }; };


/**
 * A single floating point section
 */
UCLASS(MinimalAPI)
class UMovieSceneColorSection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	/** Access the underlying generation function for the red component of this section */
	FMovieSceneFloatChannel& GetRedChannel() { return RedCurve; }
	const FMovieSceneFloatChannel& GetRedChannel() const { return RedCurve; }

	/** Access the underlying generation function for the green component of this section */
	FMovieSceneFloatChannel& GetGreenChannel() { return GreenCurve; }
	const FMovieSceneFloatChannel& GetGreenChannel() const { return GreenCurve; }

	/** Access the underlying generation function for the blue component of this section */
	FMovieSceneFloatChannel& GetBlueChannel() { return BlueCurve; }
	const FMovieSceneFloatChannel& GetBlueChannel() const { return BlueCurve; }

	/** Access the underlying generation function for the alpha component of this section */
	FMovieSceneFloatChannel& GetAlphaChannel() { return AlphaCurve; }
	const FMovieSceneFloatChannel& GetAlphaChannel() const { return AlphaCurve; }

protected:

	//~ UMovieSceneSection interface
	virtual TSharedPtr<FStructOnScope> GetKeyStruct(TArrayView<const FKeyHandle> KeyHandles) override;

private:

	//~ IMovieSceneEntityProvider interface
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

private:

	/** Red curve data */
	UPROPERTY()
	FMovieSceneFloatChannel RedCurve;

	/** Green curve data */
	UPROPERTY()
	FMovieSceneFloatChannel GreenCurve;

	/** Blue curve data */
	UPROPERTY()
	FMovieSceneFloatChannel BlueCurve;

	/** Alpha curve data */
	UPROPERTY()
	FMovieSceneFloatChannel AlphaCurve;
};
