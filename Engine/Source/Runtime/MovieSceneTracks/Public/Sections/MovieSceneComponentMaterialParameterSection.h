// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Materials/MaterialParameters.h"
#include "MovieSceneComponentMaterialParameterSection.generated.h"

/**
 * Structure representing an animated scalar parameter and it's associated animation curve.
 */
USTRUCT()
struct FScalarMaterialParameterInfoAndCurve
{
	GENERATED_USTRUCT_BODY()

		FScalarMaterialParameterInfoAndCurve()
	{}

	/** Creates a new FScalarMaterialParameterInfoAndCurve for a specific scalar parameter. */
	MOVIESCENETRACKS_API FScalarMaterialParameterInfoAndCurve(const FMaterialParameterInfo& InParameterInfo);

	/** The material parameter info of the scalar parameter which is being animated. */
	UPROPERTY()
	FMaterialParameterInfo ParameterInfo;

	/** The curve which contains the animation data for the scalar parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel ParameterCurve;

#if WITH_EDITORONLY_DATA
	/** Editor-only name of parameter layer, if applicable */
	UPROPERTY()
	FString ParameterLayerName;

	/** Editor-only name of parameter asset, if applicable */
	UPROPERTY()
	FString ParameterAssetName;
#endif
};

/**
* Structure representing an animated color parameter and it's associated animation curve.
*/
USTRUCT()
struct FColorMaterialParameterInfoAndCurves
{
	GENERATED_USTRUCT_BODY()

	FColorMaterialParameterInfoAndCurves()
	{}

	/** Creates a new FVectorParameterNameAndCurve for a specific color parameter. */
	MOVIESCENETRACKS_API FColorMaterialParameterInfoAndCurves(const FMaterialParameterInfo& InParameterInfo);

	/** The name of the color parameter which is being animated. */
	UPROPERTY()
	FMaterialParameterInfo ParameterInfo;

	/** The curve which contains the animation data for the red component of the color parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel RedCurve;

	/** The curve which contains the animation data for the green component of the color parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel GreenCurve;

	/** The curve which contains the animation data for the blue component of the color parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel BlueCurve;

	/** The curve which contains the animation data for the alpha component of the color parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel AlphaCurve;

#if WITH_EDITORONLY_DATA
	/** Name of parameter layer, if applicable */
	UPROPERTY()
	FString ParameterLayerName;

	/** Name of parameter asset, if applicable */
	UPROPERTY()
	FString ParameterAssetName;

	/* Optional UI names for parameter channels, if applicable */
	UPROPERTY()
	FParameterChannelNames ParameterChannelNames;
#endif
};

/**
 * A movie scene section containing data for material parameters.
 */
UCLASS(MinimalAPI)
class UMovieSceneComponentMaterialParameterSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:
	/** Adds a a key for a specific scalar parameter at the specified time with the specified value. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void AddScalarParameterKey(const FMaterialParameterInfo& InParameterInfo, FFrameNumber InTime, float InValue, const FString& InLayerName, const FString& InAssetName, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/** Adds a a key for a specific color parameter at the specified time with the specified value. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void AddColorParameterKey(const FMaterialParameterInfo& InParameterInfo, FFrameNumber InTime, FLinearColor InValue, const FString& InLayerName, const FString& InAssetName, const FParameterChannelNames& InChannelNames=FParameterChannelNames(), EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/** 
	 * Removes a scalar parameter from this section. 
	 * 
	 * @param InParameterInfo The material parameter info of the scalar parameter to remove.
	 * @returns True if a parameter with that name was found and removed, otherwise false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API bool RemoveScalarParameter(const FMaterialParameterInfo& InParameterInfo);
	
	/**
	 * Removes a color parameter from this section.
	 *
	 * @param InParameterInfo The material parameter info of the color parameter to remove.
	 * @returns True if a parameter with that name was found and removed, otherwise false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API bool RemoveColorParameter(const FMaterialParameterInfo& InParameterInfo);


#if WITH_EDITOR
	/**
	* Removes a scalar parameter from this section.
	*
	* @param InParameterPath The full path name of the scalar parameter to remove.
	* @returns True if a parameter with that name was found and removed, otherwise false.
	*/
	MOVIESCENETRACKS_API bool RemoveScalarParameter(FName InParameterPath);

	/**
	 * Removes a color parameter from this section.
	 *
	 * @param InParameterPath The full path name of the color parameter to remove.
	 * @returns True if a parameter with that name was found and removed, otherwise false.
	 */
	MOVIESCENETRACKS_API bool RemoveColorParameter(FName InParameterPath);
#endif

	MOVIESCENETRACKS_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	MOVIESCENETRACKS_API virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	MOVIESCENETRACKS_API void ExternalPopulateEvaluationField(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder);


public:
	
	/** The scalar parameter infos and their associated curves. */
	UPROPERTY()
	TArray<FScalarMaterialParameterInfoAndCurve> ScalarParameterInfosAndCurves;

	/** The color parameter infos and their associated curves. */
	UPROPERTY()
	TArray<FColorMaterialParameterInfoAndCurves> ColorParameterInfosAndCurves;

protected:

	//~ UMovieSceneSection interface
	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	MOVIESCENETRACKS_API virtual void PostEditImport() override;
#if WITH_EDITOR
	MOVIESCENETRACKS_API virtual void PostEditUndo() override;
#endif
};
