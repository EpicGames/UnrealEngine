// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/ContainersFwd.h"
#include "Templates/TupleFwd.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/Interface.h"

#include "MovieSceneConsoleVariableTrackInterface.generated.h"

UINTERFACE(MinimalAPI)
class UMovieSceneConsoleVariableTrackInterface : public UInterface
{
	GENERATED_BODY()
};

class IMovieSceneConsoleVariableTrackInterface
{
public:
	GENERATED_BODY()

	virtual void GetConsoleVariablesForTrack(bool bOnlyIncludeChecked, TArray<TTuple<FString, FString>>& OutVariables) = 0;
	virtual bool IsConsoleVariableEnabled(const FString& ConsoleVariableName) = 0;
};
