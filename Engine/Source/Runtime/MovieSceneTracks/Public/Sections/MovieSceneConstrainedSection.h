// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/Interface.h"
#include "MovieSceneSection.h"
#include "ConstraintsManager.h"
#include "ConstraintChannel.h"
#include "MovieSceneConstrainedSection.generated.h"

struct FGuid;
struct FMovieSceneChannel;

/**
 * Functionality to add to sections that contain constraints
 */

UINTERFACE(MinimalAPI)
class UMovieSceneConstrainedSection : public UInterface
{
public:
	GENERATED_BODY()
};

/**
 * Interface to be added to UMovieSceneSection types when they contain entity data
 */
class IMovieSceneConstrainedSection
{
public:

	GENERATED_BODY()
	MOVIESCENETRACKS_API IMovieSceneConstrainedSection();

	DECLARE_MULTICAST_DELEGATE_TwoParams(FConstraintChannelAddedEvent, IMovieSceneConstrainedSection*, FMovieSceneConstraintChannel*);


	/*
	* If it has that constraint with that Name
	*/
	virtual bool HasConstraintChannel(const FGuid& InGuid) const = 0;

	/*
	* Get constraint with that name
	*/
	virtual FConstraintAndActiveChannel* GetConstraintChannel(const FGuid& InConstraintGuid) = 0;

	/*
	*  Add Constraint channel
	*/
	virtual void AddConstraintChannel(UTickableConstraint* InConstraint) = 0;
	
	/*
	*  Remove Constraint channel
	*/
	virtual void RemoveConstraintChannel(const UTickableConstraint* InConstraint) = 0;

	/*
	*  Get a reference to the channels
	*/
	virtual TArray<FConstraintAndActiveChannel>& GetConstraintsChannels() = 0;

	/*
	*  Get a const reference to the channels
	*/
	virtual const TArray<FConstraintAndActiveChannel>& GetConstraintsChannels() const = 0;

	/*
	*  Replace the constraint with the specified name with the new one
	*/
	virtual void ReplaceConstraint(const FName InConstraintName, UTickableConstraint* InConstraint) = 0;

	/*
	* Added Delegate
	*/
	 FConstraintChannelAddedEvent& ConstraintChannelAdded() { return OnConstraintChannelAdded; }

	 /*
	 *  What to do if the constraint object has been changed, for example by an undo or redo. By default nothing to be overriden if needed.
	 */
	 virtual void OnConstraintsChanged() {};

	/*
	*  SetToNotRemoveChannel when we are told a constraint is removed, we need this sometimes sincet his will be destructive
	*/
	 MOVIESCENETRACKS_API void SetDoNoRemoveChannel(bool bInDoNotRemoveChannel);

	/*
	*  Removed delegate that get's added by the track editor
	*/
	FDelegateHandle OnConstraintRemovedHandle;

	/*
	* Find the constraint named ConstraintName
	*/
	MOVIESCENETRACKS_API UTickableConstraint* FindConstraint(const FName InConstraintName) const;

	/*
	* Find the constraint from the channel
	*/
	MOVIESCENETRACKS_API UTickableConstraint* FindConstraint(FMovieSceneChannel* InChannel) const;

	/*
	* Removes the constraint named ConstraintName
	*/
	MOVIESCENETRACKS_API bool RemoveConstraint(UWorld* InWorld, const FName InConstraintName, const bool bDoNotCompensate) const;
	
	/*
    * Removes all the constraints stored in this section
    */
	MOVIESCENETRACKS_API bool RemoveAllConstraints(UWorld* InWorld, const bool bDoNotCompensate) const;

protected:

	FConstraintChannelAddedEvent OnConstraintChannelAdded;
	bool bDoNotRemoveChannel = false;

};
