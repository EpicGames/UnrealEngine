// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Sections/MovieSceneParameterSection.h"

#include "MovieSceneCustomPrimitiveDataSection.generated.h"


/**
 * Movie scene section that animates custom primitive data scalars, vector2s, vector3s, and colors
 */
UCLASS(MinimalAPI)
class UMovieSceneCustomPrimitiveDataSection : public UMovieSceneParameterSection
{
	GENERATED_BODY()

public:

	MOVIESCENETRACKS_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

	MOVIESCENETRACKS_API void ExternalPopulateEvaluationField(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder);


	MOVIESCENETRACKS_API void ReconstructChannelProxy() override;

private:

#if WITH_EDITORONLY_DATA

public:
	uint64 GetChannelsUsed() const { return ChannelsUsedBitmap; }

protected:

	UPROPERTY()
	uint64 ChannelsUsedBitmap = 0;

#endif

};