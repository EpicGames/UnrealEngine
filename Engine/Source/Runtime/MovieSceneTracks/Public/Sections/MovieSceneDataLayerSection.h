// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectMacros.h"
#include "WorldPartition/DataLayer/DataLayerAsset.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "MovieSceneDataLayerSection.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneDataLayerSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
public:

	GENERATED_BODY()

	UMovieSceneDataLayerSection(const FObjectInitializer& ObjInit);

	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API EDataLayerRuntimeState GetDesiredState() const;

	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void SetDesiredState(EDataLayerRuntimeState InDesiredState);

	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API EDataLayerRuntimeState GetPrerollState() const;

	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void SetPrerollState(EDataLayerRuntimeState InPrerollState);

	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API bool GetFlushOnActivated() const;

	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void SetFlushOnActivated(bool bFlushOnActivated);

	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API bool GetFlushOnUnload() const;

	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void SetFlushOnUnload(bool bFlushOnUnload);
		
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	const TArray<UDataLayerAsset*>& GetDataLayerAssets() const { return DataLayerAssets; }

	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetDataLayerAssets(const TArray<UDataLayerAsset*>& InDataLayerAssets) { DataLayerAssets = InDataLayerAssets; }

private:

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

private:
	/** A list of data layers that should be loaded or unloaded by this section */
	UPROPERTY(EditAnywhere, Category = DataLayer)
	TArray<TObjectPtr<UDataLayerAsset>> DataLayerAssets;

	/** The desired state for the data layers on this section when the section is actively evaluating. */
	UPROPERTY(EditAnywhere, Category=DataLayer)
	EDataLayerRuntimeState DesiredState;

	/** The desired state for the data layers on this section when the section is pre or post-rolling. */
	UPROPERTY(EditAnywhere, Category=DataLayer, Meta=(EditCondition="HasPreRoll", EditConditionHides))
	EDataLayerRuntimeState PrerollState;

	/** Determine if we need to flush level streaming when the data layers are activated. May result in a hitch. */
	UPROPERTY(EditAnywhere, Category=DataLayer, Meta=(EditCondition="DesiredState == EDataLayerRuntimeState::Activated", EditConditionHides))
	bool bFlushOnActivated;

	/** Determine if we need to flush level streaming when the data layers unloads. */
	UPROPERTY(EditAnywhere, Category=DataLayer, Meta=(EditCondition="DesiredState == EDataLayerRuntimeState::Unloaded", EditConditionHides))
	bool bFlushOnUnload;

	UFUNCTION()
	bool HasPreRoll() const { return GetPreRollFrames() > 0.0f; }
};
