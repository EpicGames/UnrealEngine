// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneDoubleSection.generated.h"


/**
 * A double precision floating point section
 */
UCLASS( MinimalAPI )
class UMovieSceneDoubleSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneDoubleSection(const FObjectInitializer& ObjectInitializer);

public:

	/**
	 * Public access to this section's internal data function
	 */
	FMovieSceneDoubleChannel& GetChannel() { return DoubleCurve; }
	const FMovieSceneDoubleChannel& GetChannel() const { return DoubleCurve; }

protected:

	/** Double data */
	UPROPERTY()
	FMovieSceneDoubleChannel DoubleCurve;

private:

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
};
