// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Curves/IntegralCurve.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneByteChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneEnumSection.generated.h"


/**
 * A single enum section.
 */
UCLASS(MinimalAPI)
class UMovieSceneEnumSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	/** Ordered curve data */
	UPROPERTY()
	FMovieSceneByteChannel EnumCurve;

private:

	//~ IMovieSceneEntityProvider interface
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
};
