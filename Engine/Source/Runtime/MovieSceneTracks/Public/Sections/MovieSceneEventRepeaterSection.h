// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Sections/MovieSceneEventSectionBase.h"
#include "Channels/MovieSceneEvent.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneEventRepeaterSection.generated.h"


/**
 * Event section that will trigger its event exactly once, every time it is evaluated.
 */
UCLASS(MinimalAPI)
class UMovieSceneEventRepeaterSection
	: public UMovieSceneEventSectionBase
	, public IMovieSceneEntityProvider
{
public:
	GENERATED_BODY()

	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

#if WITH_EDITORONLY_DATA
	virtual TArrayView<FMovieSceneEvent> GetAllEntryPoints() override { return MakeArrayView(&Event, 1); }
#endif

	/** The event that should be triggered each time this section is evaluated */
	UPROPERTY(EditAnywhere, Category="Event")
	FMovieSceneEvent Event;
};


