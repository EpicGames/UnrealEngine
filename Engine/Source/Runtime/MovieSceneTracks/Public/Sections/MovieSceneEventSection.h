// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/Class.h"
#include "Curves/KeyHandle.h"
#include "MovieSceneSection.h"
#include "Curves/NameCurve.h"
#include "UObject/StructOnScope.h"
#include "Engine/Engine.h"
#include "UObject/SoftObjectPath.h"
#include "Channels/MovieSceneChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "MovieSceneEventSection.generated.h"

struct EventData;

USTRUCT(BlueprintType)
struct FMovieSceneEventParameters
{
	GENERATED_BODY()

	FMovieSceneEventParameters()
		: PackageFileVersion(GPackageFileUEVersion)
		, LicenseePackageFileVersion(GPackageFileLicenseeUEVersion )
	{}

	/** Construction from a struct type */
	FMovieSceneEventParameters(UScriptStruct& InStruct)
		: StructType(&InStruct)
		, PackageFileVersion(GPackageFileUEVersion)
		, LicenseePackageFileVersion(GPackageFileLicenseeUEVersion )
	{
	}

	MOVIESCENETRACKS_API friend bool operator==(const FMovieSceneEventParameters& A, const FMovieSceneEventParameters& B);

	MOVIESCENETRACKS_API friend bool operator!=(const FMovieSceneEventParameters& A, const FMovieSceneEventParameters& B);

	/**
	 * Access the struct type of this event parameter payload
	 * @return A valid UScriptStruct* or nullptr if the struct is not set, or no longer available
	 */
	UScriptStruct* GetStructType() const
	{
		return Cast<UScriptStruct>(StructType.TryLoad());
	}

	/**
	 * Change the type of this event parameter payload to be the specified struct
	 */
	void Reassign(UScriptStruct* NewStruct)
	{
		StructType = NewStruct;

		if (!NewStruct)
		{
			StructBytes.Reset();
		}
	}

	/**
	 * Retrieve an instance of this payload
	 *
	 * @param OutStruct Structure to receive the instance
	 */
	MOVIESCENETRACKS_API void GetInstance(FStructOnScope& OutStruct) const;

	/**
	 * Overwrite this payload with another instance of the same type.
	 *
	 * @param InstancePtr A valid pointer to an instance of the type represented by GetStructType
	 */
	MOVIESCENETRACKS_API void OverwriteWith(uint8* InstancePtr);

	/**
	 * Serialization implementation
	 */
	MOVIESCENETRACKS_API bool Serialize(FArchive& Ar);

	friend FArchive& operator<<(FArchive& Ar, FMovieSceneEventParameters& Payload)
	{
		Payload.Serialize(Ar);
		return Ar;
	}

private:

	/** Soft object path to the type of this parameter payload */
	FSoftObjectPath StructType;

	/** Used while loading to pass along to the FEventParameterReader */
	FPackageFileVersion PackageFileVersion;
	int32 LicenseePackageFileVersion;

	/** Serialized bytes that represent the payload. Serialized internally with FEventParameterArchive */
	TArray<uint8> StructBytes;
};

template<>
struct TStructOpsTypeTraits<FMovieSceneEventParameters> : public TStructOpsTypeTraitsBase2<FMovieSceneEventParameters>
{
	enum 
	{
		WithCopy = true,
		WithSerializer = true
	};
};

USTRUCT(BlueprintType)
struct FEventPayload
{
	GENERATED_BODY()

	FEventPayload() {}
	FEventPayload(FName InEventName) : EventName(InEventName) {}

	friend bool operator==(const FEventPayload& A, const FEventPayload& B)
	{
		return A.EventName == B.EventName && A.Parameters == B.Parameters;
	}

	friend bool operator!=(const FEventPayload& A, const FEventPayload& B)
	{
		return A.EventName != B.EventName || A.Parameters != B.Parameters;
	}
	/** The name of the event to trigger */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category=Event)
	FName EventName;

	/** The event parameters */
	UPROPERTY(EditAnywhere, BlueprintReadOnly, Category=Event, meta=(ShowOnlyInnerProperties))
	FMovieSceneEventParameters Parameters;
};

/** A curve of events */
USTRUCT()
struct FMovieSceneEventSectionData : public FMovieSceneChannel
{
	GENERATED_BODY()

#if WITH_EDITORONLY_DATA
		/**
	 * Called after this section data has been serialized to upgrade old data
	 */
	MOVIESCENETRACKS_API void PostSerialize(const FArchive& Ar);
#endif

	/**
	 * Access a mutable interface for this channel's data
	 *
	 * @return An object that is able to manipulate this channel's data
	 */
	inline TMovieSceneChannelData<FEventPayload> GetData()
	{
		return TMovieSceneChannelData<FEventPayload>(&Times, &KeyValues, this, &KeyHandles);
	}

	/**
	 * Access a constant interface for this channel's data
	 *
	 * @return An object that is able to interrogate this channel's data
	 */
	inline TMovieSceneChannelData<const FEventPayload> GetData() const
	{
		return TMovieSceneChannelData<const FEventPayload>(&Times, &KeyValues);
	}

	TArrayView<const FFrameNumber> GetKeyTimes() const
	{
		return Times;
	}

	TArrayView<const FEventPayload> GetKeyValues() const
	{
		return KeyValues;
	}

public:

	// ~ FMovieSceneChannel Interface
	MOVIESCENETRACKS_API virtual void GetKeys(const TRange<FFrameNumber>& WithinRange, TArray<FFrameNumber>* OutKeyTimes, TArray<FKeyHandle>* OutKeyHandles) override;
	MOVIESCENETRACKS_API virtual void GetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<FFrameNumber> OutKeyTimes) override;
	MOVIESCENETRACKS_API virtual void SetKeyTimes(TArrayView<const FKeyHandle> InHandles, TArrayView<const FFrameNumber> InKeyTimes) override;
	MOVIESCENETRACKS_API virtual void DuplicateKeys(TArrayView<const FKeyHandle> InHandles, TArrayView<FKeyHandle> OutNewHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeys(TArrayView<const FKeyHandle> InHandles) override;
	MOVIESCENETRACKS_API virtual void DeleteKeysFrom(FFrameNumber InTime, bool bDeleteKeysBefore) override;
	MOVIESCENETRACKS_API virtual void RemapTimes(const UE::MovieScene::IRetimingInterface& Retimer) override;
	MOVIESCENETRACKS_API virtual TRange<FFrameNumber> ComputeEffectiveRange() const override;
	MOVIESCENETRACKS_API virtual int32 GetNumKeys() const override;
	MOVIESCENETRACKS_API virtual void Reset() override;
	MOVIESCENETRACKS_API virtual void Offset(FFrameNumber DeltaPosition) override;
	MOVIESCENETRACKS_API virtual FKeyHandle GetHandle(int32 Index) override;
	MOVIESCENETRACKS_API virtual int32 GetIndex(FKeyHandle Handle) override;
	virtual void Optimize(const FKeyDataOptimizationParams& InParameters) override {}
	virtual void ClearDefault() override {}

private:

	UPROPERTY(meta=(KeyTimes))
	TArray<FFrameNumber> Times;

	/** Array of values that correspond to each key time */
	UPROPERTY(meta=(KeyValues))
	TArray<FEventPayload> KeyValues;

	/** This needs to be a UPROPERTY so it gets saved into editor transactions but transient so it doesn't get saved into assets. */
	UPROPERTY(Transient)
	FMovieSceneKeyHandleMap KeyHandles;


#if WITH_EDITORONLY_DATA

	UPROPERTY()
	TArray<float> KeyTimes_DEPRECATED;

#endif
};

#if WITH_EDITORONLY_DATA
template<>
struct TStructOpsTypeTraits<FMovieSceneEventSectionData> : public TStructOpsTypeTraitsBase2<FMovieSceneEventSectionData>
{
	enum { WithPostSerialize = true };
};
#endif

/**
 * Implements a section in movie scene event tracks.
 */
UCLASS(MinimalAPI)
class UMovieSceneEventSection
	: public UMovieSceneSection
{
	GENERATED_BODY()

	/** Default constructor. */
	UMovieSceneEventSection();

public:
	
	// ~UObject interface
	virtual void PostLoad() override;

	/**
	 * Get the section's event data.
	 *
	 * @return Event data.
	 */
	const FMovieSceneEventSectionData& GetEventData() const { return EventData; }

protected:

private:

	UPROPERTY()
	FNameCurve Events_DEPRECATED;

	UPROPERTY()
	FMovieSceneEventSectionData EventData;
};

template<>
struct TMovieSceneChannelTraits<FMovieSceneEventSectionData> : TMovieSceneChannelTraitsBase<FMovieSceneEventSectionData>
{
	enum { SupportsDefaults = false };
};

/** Stub out unnecessary functions */
inline bool EvaluateChannel(const FMovieSceneEventSectionData* InChannel, FFrameTime InTime, FEventPayload& OutValue)
{
	// Can't evaluate event section data in the typical sense
	return false;
}

inline bool ValueExistsAtTime(const FMovieSceneEventSectionData* InChannel, FFrameNumber Time, const FEventPayload& Value)
{
	// true if any value exists
	return InChannel->GetData().FindKey(Time) != INDEX_NONE;
}
