// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneEvent.h"
#include "Engine/Blueprint.h"
#include "MovieSceneEventSectionBase.generated.h"

/**
 * Base class for all event sections. Manages dirtying the section and track on recompilation of the director blueprint.
 */
UCLASS(MinimalAPI)
class UMovieSceneEventSectionBase
	: public UMovieSceneSection
{
public:
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneEventSectionBase(const FObjectInitializer& ObjInit);

	virtual TArrayView<FMovieSceneEvent> GetAllEntryPoints() { return TArrayView<FMovieSceneEvent>(); }

	virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR

	virtual void PostRename(UObject* OldOuter, const FName OldName) override;
	virtual void PostDuplicate(bool bDuplicateForPIE) override;
	virtual void RemoveForCook() override;

	MOVIESCENETRACKS_API void AttemptUpgrade();

	DECLARE_MULTICAST_DELEGATE_FourParams(FFixupPayloadParameterNameEvent, UMovieSceneEventSectionBase*, UK2Node*, FName, FName);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FUpgradeLegacyEventEndpoint, UMovieSceneEventSectionBase*);
	DECLARE_DELEGATE_RetVal_OneParam(void, FPostDuplicateEvent, UMovieSceneEventSectionBase*);
	DECLARE_DELEGATE_RetVal_OneParam(void, FRemoveForCookEvent, UMovieSceneEventSectionBase*);

	/**
	 * Handler should be invoked when an event endpoint that is referenced from this section has one of its pins renamed
	 */
	void OnUserDefinedPinRenamed(UK2Node* InNode, FName OldPinName, FName NewPinName)
	{
		FixupPayloadParameterNameEvent.Broadcast(this, InNode, OldPinName, NewPinName);
	}

	/**
	 * Post compilation handler that is invoked once generated function graphs have been compiled. Fixes up UFunction pointers for each event.
	 */
	MOVIESCENETRACKS_API void OnPostCompile(UBlueprint* Blueprint);


	/**
	 * Event that is broadcast when event payloads may need fixing up due to a pin rename
	 */
	MOVIESCENETRACKS_API static FFixupPayloadParameterNameEvent FixupPayloadParameterNameEvent;

	/**
	 * Delegate that is used to upgrade legacy event sections that need fixing up against a blueprint. Called on serialization and on compilation if necessary until successful upgrade occurs.
	 */
	MOVIESCENETRACKS_API static FUpgradeLegacyEventEndpoint UpgradeLegacyEventEndpoint;

	/**
	 * Delegate that is used to ensure that a blueprint compile hook exists for this event section after it has been duplicated.
	 */
	MOVIESCENETRACKS_API static FPostDuplicateEvent PostDuplicateSectionEvent;

	/**
	 * Delegate that is used to ensure that a hook exists for this event section before it has been removed for cook.
	 */
	MOVIESCENETRACKS_API static FRemoveForCookEvent RemoveForCookEvent;

private:

	/** Boolean that specifies whether AttemptUpgrade needs to do any work. This is necessary because all the data required for the upgrade may not be loaded at the time ::Serialize is called,
	 * so another attempt may be necessary as the blueprint is compiled */
	bool bDataUpgradeRequired;

#endif
};