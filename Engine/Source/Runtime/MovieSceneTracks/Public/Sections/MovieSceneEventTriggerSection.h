// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneEventSectionBase.h"
#include "Channels/MovieSceneEventChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneEventTriggerSection.generated.h"


/**
 * Event section that triggeres specific timed events.
 */
UCLASS(MinimalAPI)
class UMovieSceneEventTriggerSection
	: public UMovieSceneEventSectionBase
	, public IMovieSceneEntityProvider
{
public:
	GENERATED_BODY()

	UMovieSceneEventTriggerSection(const FObjectInitializer& ObjInit);

	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

#if WITH_EDITORONLY_DATA
	virtual TArrayView<FMovieSceneEvent> GetAllEntryPoints() override { return EventChannel.GetData().GetValues(); }
#endif

	/** The channel that defines this section's timed events */
	UPROPERTY()
	FMovieSceneEventChannel EventChannel;
};