// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneFloatChannel.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Math/Color.h"
#include "MovieSceneSection.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneFadeSection.generated.h"

class UObject;


/**
 * A single floating point section.
 */
UCLASS(MinimalAPI)
class UMovieSceneFadeSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

	/** Default constructor. */
	UMovieSceneFadeSection();

	/** IMovieSceneEntityProvider interface */
	void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

public:

	/** Float data */
	UPROPERTY()
	FMovieSceneFloatChannel FloatCurve;

	/** Fade color. */
	UPROPERTY(EditAnywhere, Category="Fade")
	FLinearColor FadeColor;

	/** Fade audio. */
	UPROPERTY(EditAnywhere, Category="Fade")
	uint32 bFadeAudio:1;
};
