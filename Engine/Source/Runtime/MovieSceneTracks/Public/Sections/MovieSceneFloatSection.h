// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/IMovieSceneChannelOverrideProvider.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneFloatSection.generated.h"


/**
 * A single floating point section
 */
UCLASS( MinimalAPI )
class UMovieSceneFloatSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
	, public IMovieSceneChannelOverrideProvider
{
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneFloatSection(const FObjectInitializer& ObjectInitializer);

public:

	/**
	 * Public access to this section's internal data function
	 */
	FMovieSceneFloatChannel& GetChannel() { return FloatCurve; }
	const FMovieSceneFloatChannel& GetChannel() const { return FloatCurve; }

protected:

	virtual UMovieSceneSectionChannelOverrideRegistry* GetChannelOverrideRegistry(bool bCreateIfMissing) override;
	virtual UE::MovieScene::FChannelOverrideProviderTraitsHandle GetChannelOverrideProviderTraits() const override;
	virtual void OnChannelOverridesChanged() override;

protected:

	/** Float data */
	UPROPERTY()
	FMovieSceneFloatChannel FloatCurve;

	/** Channel overrides */
	UPROPERTY()
	TObjectPtr<UMovieSceneSectionChannelOverrideRegistry> OverrideRegistry;

private:

	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;

#if WITH_EDITOR
	MOVIESCENETRACKS_API virtual void PostPaste() override;
#endif
};

