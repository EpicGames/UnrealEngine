// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneIntegerChannel.h"
#include "CoreMinimal.h"
#include "Curves/IntegralCurve.h"
#include "Curves/KeyHandle.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneIntegerSection.generated.h"


/**
 * A single integer section.
 */
UCLASS(MinimalAPI)
class UMovieSceneIntegerSection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:
	/**
	 * Access this section's underlying raw data
	 */
	const FMovieSceneIntegerChannel& GetChannel() const { return IntegerCurve; }

private:

	//~ IMovieSceneEntityProvider interface
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

private:

	/** Ordered curve data */
	UPROPERTY()
	FMovieSceneIntegerChannel IntegerCurve;
};
