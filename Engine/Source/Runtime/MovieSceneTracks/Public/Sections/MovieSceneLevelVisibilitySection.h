// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "MovieSceneLevelVisibilitySection.generated.h"


/**
 * Visibility options for the level visibility section.
 */
UENUM()
enum class ELevelVisibility : uint8
{
	/** The streamed levels should be visible. */
	Visible,

	/** The streamed levels should be hidden. */
	Hidden
};


/**
 * A section for use with the movie scene level visibility track, which controls streamed level visibility.
 */
UCLASS(MinimalAPI)
class UMovieSceneLevelVisibilitySection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API ELevelVisibility GetVisibility() const;

	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void SetVisibility(ELevelVisibility InVisibility);

	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	const TArray<FName>& GetLevelNames() const { return LevelNames; }

	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	void SetLevelNames(const TArray<FName>& InLevelNames) { LevelNames = InLevelNames; }

private:

	MOVIESCENETRACKS_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

private:

	/** Whether or not the levels in this section should be visible or hidden. */
	UPROPERTY(EditAnywhere, Category = LevelVisibility)
	ELevelVisibility Visibility;

	/** The short names of the levels who's visibility is controlled by this section. */
	UPROPERTY(EditAnywhere, Category = LevelVisibility)
	TArray<FName> LevelNames;
};
