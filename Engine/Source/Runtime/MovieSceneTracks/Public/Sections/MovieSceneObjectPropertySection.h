// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneObjectPathChannel.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"

#include "MovieSceneObjectPropertySection.generated.h"

class UObject;

UCLASS(MinimalAPI)
class UMovieSceneObjectPropertySection 
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
public:

	GENERATED_BODY()

	UMovieSceneObjectPropertySection(const FObjectInitializer& ObjInit);

	// ~UObject interface
	virtual void PostLoad() override;

public:

	UPROPERTY()
	FMovieSceneObjectPathChannel ObjectChannel;

private:
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

};
