// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "MovieSceneSection.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneBoolChannel.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneParameterSection.generated.h"

struct FMovieSceneParameterPropertyInterface;
struct FMovieSceneConstParameterPropertyInterface;


UINTERFACE(MinimalAPI)
class UMovieSceneParameterSectionExtender : public UInterface
{
public:
	GENERATED_BODY()
};

class IMovieSceneParameterSectionExtender
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API void ExtendEntity(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity);

private:

	virtual void ExtendEntityImpl(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity) = 0;
};



/**
 * Structure representing the animated value of a scalar parameter.
 */
struct FScalarParameterNameAndValue
{
	/** Creates a new FScalarParameterAndValue with a parameter name and a value. */
	FScalarParameterNameAndValue(FName InParameterName, float InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	/** The name of the scalar parameter. */
	FName ParameterName;

	/** The animated value of the scalar parameter. */
	float Value;
};

/**
 * Structure representing the value of a bool parameter.
 */
struct FBoolParameterNameAndValue
{
	/** Creates a new FScalarParameterAndValue with a parameter name and a value. */
	FBoolParameterNameAndValue(FName InParameterName, bool InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	/** The name of the bool parameter. */
	FName ParameterName;

	/** The value of the bool parameter. */
	bool Value;
};

/**
 * Structure representing the animated value of a vector2D parameter.
 */
struct FVector2DParameterNameAndValue
{
	/** Creates a new FVectorParameterAndValue with a parameter name and a value. */
	FVector2DParameterNameAndValue(FName InParameterName, FVector2D InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	/** The name of the vector parameter. */
	FName ParameterName;

	/** The animated value of the vector parameter. */
	FVector2D Value;
};

/**
 * Structure representing the animated value of a vector parameter.
 */
struct FVectorParameterNameAndValue
{
	/** Creates a new FVectorParameterAndValue with a parameter name and a value. */
	FVectorParameterNameAndValue(FName InParameterName, FVector InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	/** The name of the vector parameter. */
	FName ParameterName;

	/** The animated value of the vector parameter. */
	FVector Value;
};

/**
 * Structure representing the animated value of a color parameter.
 */
struct FColorParameterNameAndValue
{
	/** Creates a new FColorParameterAndValue with a parameter name and a value. */
	FColorParameterNameAndValue(FName InParameterName, FLinearColor InValue)
	{
		ParameterName = InParameterName;
		Value = InValue;
	}

	/** The name of the color parameter. */
	FName ParameterName;

	/** The animated value of the color parameter. */
	FLinearColor Value;
};

struct FTransformParameterNameAndValue
{

	/** The name of the transform  parameter. */
	FName ParameterName;
	/** Translation component */
	FVector Translation;
	/** Rotation component */
	FRotator Rotation;
	/** Scale component */
	FVector Scale;

	FTransformParameterNameAndValue(FName InParameterName,const FVector& InTranslation,
		const FRotator& InRotation, const FVector& InScale) : Translation(InTranslation),
		Rotation(InRotation), Scale(InScale)
	{
		ParameterName = InParameterName;	
	}
};

/**
 * Structure representing a parameter name without a value
 */
USTRUCT()
struct FBaseParameterNameAndValue
{
	GENERATED_USTRUCT_BODY()

	FBaseParameterNameAndValue()
	: ParameterName(NAME_None)
	{}

	/** Creates a new FScalarParameterNameAndCurve for a specific scalar parameter. */
	FBaseParameterNameAndValue(FName InParameterName)
	: ParameterName(InParameterName)
	{}

	/** The name of the parameter which is being animated. */
	UPROPERTY()
	FName ParameterName;
};

/**
 * Structure representing a bool parameter and it's associated animation curve.
 */
USTRUCT()
struct FBoolParameterNameAndCurve : public FBaseParameterNameAndValue
{
	GENERATED_USTRUCT_BODY()

	FBoolParameterNameAndCurve()
	{}

	FBoolParameterNameAndCurve(FName InParameterName)
	: FBaseParameterNameAndValue(InParameterName)
	{}

	/** The curve which contains the animation data for the scalar parameter. */
	UPROPERTY()
	FMovieSceneBoolChannel ParameterCurve;
};

/**
 * Structure representing an animated scalar parameter and it's associated animation curve.
 */
USTRUCT()
struct FScalarParameterNameAndCurve : public FBaseParameterNameAndValue
{
	GENERATED_USTRUCT_BODY()

	FScalarParameterNameAndCurve()
	{}

	FScalarParameterNameAndCurve(FName InParameterName)
	: FBaseParameterNameAndValue(InParameterName)
	{}

	/** The curve which contains the animation data for the scalar parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel ParameterCurve;
};

/**
 * Structure representing an animated vector2D parameter and it's associated animation curve.
 */
USTRUCT()
struct FVector2DParameterNameAndCurves : public FBaseParameterNameAndValue
{
	GENERATED_USTRUCT_BODY()

	FVector2DParameterNameAndCurves()
	{}

	FVector2DParameterNameAndCurves(FName InParameterName)
	: FBaseParameterNameAndValue(InParameterName)
	{}

	/** The curve which contains the animation data for the x component of the vector parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel XCurve;

	/** The curve which contains the animation data for the y component of the vector parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel YCurve;

};


/**
 * Structure representing an animated vector parameter and it's associated animation curve.
 */
USTRUCT()
struct FVectorParameterNameAndCurves : public FBaseParameterNameAndValue
{
	GENERATED_USTRUCT_BODY()

	FVectorParameterNameAndCurves()
	{}

	FVectorParameterNameAndCurves(FName InParameterName)
	: FBaseParameterNameAndValue(InParameterName)
	{}

	/** The curve which contains the animation data for the x component of the vector parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel XCurve;

	/** The curve which contains the animation data for the y component of the vector parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel YCurve;

	/** The curve which contains the animation data for the z component of the vector parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel ZCurve;
};


/**
* Structure representing an animated color parameter and it's associated animation curve.
*/
USTRUCT()
struct FColorParameterNameAndCurves : public FBaseParameterNameAndValue
{
	GENERATED_USTRUCT_BODY()

	FColorParameterNameAndCurves()
	{}

	FColorParameterNameAndCurves(FName InParameterName)
	: FBaseParameterNameAndValue(InParameterName)
	{}

	/** The curve which contains the animation data for the red component of the color parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel RedCurve;

	/** The curve which contains the animation data for the green component of the color parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel GreenCurve;

	/** The curve which contains the animation data for the blue component of the color parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel BlueCurve;

	/** The curve which contains the animation data for the alpha component of the color parameter. */
	UPROPERTY()
	FMovieSceneFloatChannel AlphaCurve;
};

/**
* Structure representing an animated transform parameter and it's associated animation curve.
*/
USTRUCT()
struct FTransformParameterNameAndCurves : public FBaseParameterNameAndValue
{
	GENERATED_USTRUCT_BODY()

	FTransformParameterNameAndCurves()
	{}

	FTransformParameterNameAndCurves(FName InParameterName)
	: FBaseParameterNameAndValue(InParameterName)
	{}

	/** Translation curves */
	UPROPERTY()
	FMovieSceneFloatChannel Translation[3];

	/** Rotation curves */
	UPROPERTY()
	FMovieSceneFloatChannel Rotation[3];

	/** Scale curves */
	UPROPERTY()
	FMovieSceneFloatChannel Scale[3];
};
/**
 * A single movie scene section which can contain data for multiple named parameters.
 */
UCLASS(MinimalAPI)
class UMovieSceneParameterSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:
	/** Adds a a key for a specific scalar parameter at the specified time with the specified value. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void AddScalarParameterKey(FName InParameterName, FFrameNumber InTime, float InValue, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/** Adds a a key for a specific bool parameter at the specified time with the specified value. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void AddBoolParameterKey(FName InParameterName, FFrameNumber InTime, bool InValue);

	/** Adds a a key for a specific vector2D parameter at the specified time with the specified value. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void AddVector2DParameterKey(FName InParameterName, FFrameNumber InTime, FVector2D InValue, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/** Adds a a key for a specific vector parameter at the specified time with the specified value. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void AddVectorParameterKey(FName InParameterName, FFrameNumber InTime, FVector InValue, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/** Adds a a key for a specific color parameter at the specified time with the specified value. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void AddColorParameterKey(FName InParameterName, FFrameNumber InTime, FLinearColor InValue, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/** Adds a a key for a specific color parameter at the specified time with the specified value. */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void AddTransformParameterKey(FName InParameterName, FFrameNumber InTime, const FTransform& InValue, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/** 
	 * Removes a scalar parameter from this section. 
	 * 
	 * @param InParameterName The name of the scalar parameter to remove.
	 * @returns True if a parameter with that name was found and removed, otherwise false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API bool RemoveScalarParameter(FName InParameterName);

	/**
	 * Removes a bool parameter from this section.
	 *
	 * @param InParameterName The name of the bool parameter to remove.
	 * @returns True if a parameter with that name was found and removed, otherwise false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API bool RemoveBoolParameter(FName InParameterName);

	/**
	 * Removes a vector2D parameter from this section.
	 *
	 * @param InParameterName The name of the vector2D parameter to remove.
	 * @returns True if a parameter with that name was found and removed, otherwise false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API bool RemoveVector2DParameter(FName InParameterName);

	/**
     * Removes a vector parameter from this section.
    *
    * @param InParameterName The name of the vector parameter to remove.
    * @returns True if a parameter with that name was found and removed, otherwise false.
    */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API bool RemoveVectorParameter(FName InParameterName);

	/**
	 * Removes a color parameter from this section.
	 *
	 * @param InParameterName The name of the color parameter to remove.
	 * @returns True if a parameter with that name was found and removed, otherwise false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API bool RemoveColorParameter(FName InParameterName);

	/**
	 * Removes a transform parameter from this section.
	 *
	 * @param InParameterName The name of the transform parameter to remove.
	 * @returns True if a parameter with that name was found and removed, otherwise false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API bool RemoveTransformParameter(FName InParameterName);

	/** Gets the animated scalar parameters and their associated curves. */
	MOVIESCENETRACKS_API TArray<FScalarParameterNameAndCurve>& GetScalarParameterNamesAndCurves();
	MOVIESCENETRACKS_API const TArray<FScalarParameterNameAndCurve>& GetScalarParameterNamesAndCurves() const;

	/** Gets the animated bool parameters and their associated curves. */
	MOVIESCENETRACKS_API TArray<FBoolParameterNameAndCurve>& GetBoolParameterNamesAndCurves();
	MOVIESCENETRACKS_API const TArray<FBoolParameterNameAndCurve>& GetBoolParameterNamesAndCurves() const;

	/** Gets the animated vector2D parameters and their associated curves. */
	MOVIESCENETRACKS_API TArray<FVector2DParameterNameAndCurves>& GetVector2DParameterNamesAndCurves();
	MOVIESCENETRACKS_API const TArray<FVector2DParameterNameAndCurves>& GetVector2DParameterNamesAndCurves() const;

	/** Gets the animated vector parameters and their associated curves. */
	MOVIESCENETRACKS_API TArray<FVectorParameterNameAndCurves>& GetVectorParameterNamesAndCurves();
	MOVIESCENETRACKS_API const TArray<FVectorParameterNameAndCurves>& GetVectorParameterNamesAndCurves() const;

	/** Gets the animated color parameters and their associated curves. */
	MOVIESCENETRACKS_API TArray<FColorParameterNameAndCurves>& GetColorParameterNamesAndCurves();
	MOVIESCENETRACKS_API const TArray<FColorParameterNameAndCurves>& GetColorParameterNamesAndCurves() const;

	/** Gets the animated transform parameters and their associated curves. */
	MOVIESCENETRACKS_API TArray<FTransformParameterNameAndCurves>& GetTransformParameterNamesAndCurves();
    MOVIESCENETRACKS_API const TArray<FTransformParameterNameAndCurves>& GetTransformParameterNamesAndCurves() const;

	/** Gets the set of all parameter names used by this section. */
	UFUNCTION(BlueprintPure, Category = "Sequencer|Section")
	MOVIESCENETRACKS_API void GetParameterNames(TSet<FName>& ParameterNames) const;

	MOVIESCENETRACKS_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	MOVIESCENETRACKS_API virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	MOVIESCENETRACKS_API void ExternalPopulateEvaluationField(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder);

protected:

	//~ UMovieSceneSection interface
	MOVIESCENETRACKS_API virtual void Serialize(FArchive& Ar) override;
	MOVIESCENETRACKS_API virtual void PostEditImport() override;
	MOVIESCENETRACKS_API virtual void ReconstructChannelProxy();

protected:
	/** The bool parameter names and their associated curves. */
	UPROPERTY()
	TArray<FBoolParameterNameAndCurve> BoolParameterNamesAndCurves;

	/** The scalar parameter names and their associated curves. */
	UPROPERTY()
	TArray<FScalarParameterNameAndCurve> ScalarParameterNamesAndCurves;

	/** The vector3D parameter names and their associated curves. */
	UPROPERTY()
	TArray<FVector2DParameterNameAndCurves> Vector2DParameterNamesAndCurves;

	/** The vector parameter names and their associated curves. */
	UPROPERTY()
	TArray<FVectorParameterNameAndCurves> VectorParameterNamesAndCurves;

	/** The color parameter names and their associated curves. */
	UPROPERTY()
	TArray<FColorParameterNameAndCurves> ColorParameterNamesAndCurves;

	/** The transform  parameter names and their associated curves. */
	UPROPERTY()
	TArray<FTransformParameterNameAndCurves> TransformParameterNamesAndCurves;
};
