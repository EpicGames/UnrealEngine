// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneByteChannel.h"
#include "Channels/MovieSceneChannelData.h"
#include "Channels/MovieSceneChannelEditorData.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Containers/ArrayView.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Curves/KeyHandle.h"
#include "Misc/FrameTime.h"
#include "MovieSceneSection.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/Class.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneParticleSection.generated.h"

class UObject;

/**
 * Defines the types of particle keys.
 */
UENUM(BlueprintType)
enum class EParticleKey : uint8
{
	Activate   = 0,
	Deactivate = 1,
	Trigger    = 2,
};

USTRUCT()
struct FMovieSceneParticleChannel : public FMovieSceneByteChannel
{
	GENERATED_BODY()

	MOVIESCENETRACKS_API FMovieSceneParticleChannel();
};

template<>
struct TStructOpsTypeTraits<FMovieSceneParticleChannel> : public TStructOpsTypeTraitsBase2<FMovieSceneParticleChannel>
{
	enum { WithStructuredSerializeFromMismatchedTag = true };
};


template<>
struct TMovieSceneChannelTraits<FMovieSceneParticleChannel> : TMovieSceneChannelTraitsBase<FMovieSceneParticleChannel>
{
	enum { SupportsDefaults = false };

#if WITH_EDITOR

	/** Byte channels can have external values (ie, they can get their values from external objects for UI purposes) */
	typedef TMovieSceneExternalValue<uint8> ExtendedEditorDataType;

#endif
};

/**
 * Particle section, for particle toggling and triggering.
 */
UCLASS(MinimalAPI)
class UMovieSceneParticleSection
	: public UMovieSceneSection
{
	GENERATED_UCLASS_BODY()

public:

	/** Curve containing the particle keys. */
	UPROPERTY()
	FMovieSceneParticleChannel ParticleKeys;
};


inline void AssignValue(FMovieSceneParticleChannel* InChannel, FKeyHandle InKeyHandle, EParticleKey InValue)
{
	TMovieSceneChannelData<uint8> ChannelData = InChannel->GetData();
	int32 ValueIndex = ChannelData.GetIndex(InKeyHandle);

	if (ValueIndex != INDEX_NONE)
	{
		ChannelData.GetValues()[ValueIndex] = (uint8)InValue;
	}
}

inline bool EvaluateChannel(const FMovieSceneParticleChannel* InChannel, FFrameTime InTime, EParticleKey& OutValue)
{
	uint8 RawValue = 0;
	if (InChannel->Evaluate(InTime, RawValue))
	{
		OutValue = (EParticleKey)RawValue;
		return true;
	}
	return false;
}