// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneSection.h"
#include "Channels/MovieSceneObjectPathChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieScenePrimitiveMaterialSection.generated.h"

UCLASS(MinimalAPI)
class UMovieScenePrimitiveMaterialSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
public:

	GENERATED_BODY()

	UMovieScenePrimitiveMaterialSection(const FObjectInitializer& ObjInit);

	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* Linker, const FEntityImportParams& ImportParams, FImportedEntity* OutImportedEntity) override;

	UPROPERTY()
	FMovieSceneObjectPathChannel MaterialChannel;
};
