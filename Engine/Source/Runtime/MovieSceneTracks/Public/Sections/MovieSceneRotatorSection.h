// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneDoubleChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "MovieSceneRotatorSection.generated.h"

/** Movie scene section that animates each component (X, Y, Z) of an FRotator property */
UCLASS(MinimalAPI)
class UMovieSceneRotatorSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

public:
	static constexpr int32 RollChannelIndex = 2;
	static constexpr int32 PitchChannelIndex = 0;
	static constexpr int32 YawChannelIndex = 1;

	UMovieSceneRotatorSection(const FObjectInitializer& InObjectInitializer);

	/** Get the Roll channel */
	const FMovieSceneDoubleChannel& GetChannelX() const
	{
		return Rotation[RollChannelIndex];
	}

	/** Get the Pitch channel */
	const FMovieSceneDoubleChannel& GetChannelY() const
	{
		return Rotation[PitchChannelIndex];
	}

	/** Get the Yaw channel */
	const FMovieSceneDoubleChannel& GetChannelZ() const
	{
		return Rotation[YawChannelIndex];
	}

	const FMovieSceneDoubleChannel& GetChannel(int32 InIndex) const
	{
		constexpr int32 Size = UE_ARRAY_COUNT(Rotation);
		check(InIndex >= 0 && InIndex < Size)
		return Rotation[InIndex];
	}

	//~ Begin IMovieSceneEntityProvider
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& InEffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* InEntityLinker, const FEntityImportParams& InParams, FImportedEntity* OutImportedEntity) override;
	//~ End IMovieSceneEntityProvider

private:
	UPROPERTY()
	FMovieSceneDoubleChannel Rotation[3];
};
