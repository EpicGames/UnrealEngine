// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneSection.h"
#include "Animation/AnimSequenceBase.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "StructUtils/InstancedStruct.h"
#include "MovieSceneSkeletalAnimationSection.generated.h"

struct FMovieSceneSkeletalAnimRootMotionTrackParams;
struct FAnimationPoseData;
class UMirrorDataTable;
enum class ESwapRootBone : uint8;

USTRUCT(BlueprintType)
struct FMovieSceneSkeletalAnimationParams
{
	GENERATED_BODY()

	FMovieSceneSkeletalAnimationParams();

	/** Gets the animation duration, modified by play rate */
	UE_DEPRECATED(5.5, "Animation lengh no longer has a single, consistent length if there is timewarp.")
	float GetDuration() const { return 0.f; }

	/** Gets the animation sequence length, not modified by play rate */
	float GetSequenceLength() const { return Animation != nullptr ? Animation->GetPlayLength() : 0.f; }

	/**
	 * Convert a sequence frame to a time in seconds inside the animation clip, taking into account start/end offsets,
	 * looping, etc.
	 */
	double MOVIESCENETRACKS_API MapTimeToAnimation(const UMovieSceneSection* InSection, FFrameTime InPosition, FFrameRate InTickResolution, UAnimSequenceBase* OverrideSequence = nullptr) const;

	/**
	 * As above, but with already computed section bounds.
	 */
	double MOVIESCENETRACKS_API MapTimeToAnimation(FFrameNumber InSectionStartTime, FFrameNumber InSectionEndTime, FFrameTime InPosition, FFrameRate InTickResolution, UAnimSequenceBase* OverrideSequence = nullptr) const;

	/**
	 * Make a transform structure from these animation parameters
	 */
	MOVIESCENETRACKS_API FMovieSceneSequenceTransform MakeTransform(const FFrameRate& OuterFrameRate, const TRange<FFrameNumber>& OuterRange, UAnimSequenceBase* OverrideSequence = nullptr, bool bClampToOuterRange = true, bool bForceLoop = false) const;

	/** The animation this section plays */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Animation", meta=(AllowedClasses = "/Script/Engine.AnimSequence,/Script/Engine.AnimComposite,/Script/Engine.AnimStreamable"))
	TObjectPtr<UAnimSequenceBase> Animation;

	/** The offset into the beginning of the animation clip for the first loop of play. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Animation|Offset")
	FFrameNumber FirstLoopStartFrameOffset;

	/** The offset into the beginning of the animation clip */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Animation|Offset")
	FFrameNumber StartFrameOffset;

	/** The offset into the end of the animation clip */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Animation|Offset")
	FFrameNumber EndFrameOffset;

	/** The playback rate of the animation clip */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Animation|Playback")
	FMovieSceneTimeWarpVariant PlayRate;

	/** Reverse the playback of the animation clip */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category="Animation|Playback")
	uint32 bReverse:1;

	/** The slot name to use for the animation */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Animation|Playback" )
	FName SlotName;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Animation|Playback")
	TObjectPtr<class UMirrorDataTable> MirrorDataTable;

	/** The weight curve for this animation section */
	UPROPERTY()
	FMovieSceneFloatChannel Weight;

	/** If on will skip sending animation notifies */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Animation|Playback")
	bool bSkipAnimNotifiers;

	/** If on animation sequence will always play when active even if the animation is controlled by a Blueprint or Anim Instance Class*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Animation|Playback")
	bool bForceCustomMode;

	/** If on the root bone transform will be swapped to the specified root*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Animation|Playback")
	ESwapRootBone SwapRootBone;

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Animation|Playback")
	bool bLinearPlaybackWhenScaled;

	UPROPERTY()
	float StartOffset_DEPRECATED;

	UPROPERTY()
	float EndOffset_DEPRECATED;

};

/**
 * Movie scene section that control skeletal animation
 */
UCLASS( MinimalAPI )
class UMovieSceneSkeletalAnimationSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Animation", meta=(ShowOnlyInnerProperties))
	FMovieSceneSkeletalAnimationParams Params;

	/** Get Frame Time as Animation Time*/
	MOVIESCENETRACKS_API double MapTimeToAnimation(FFrameTime InPosition, FFrameRate InTickResolution) const;

	MOVIESCENETRACKS_API UAnimSequenceBase* GetAnimation() const;
	MOVIESCENETRACKS_API UAnimSequenceBase* GetPlaybackAnimation() const;

	//~ UMovieSceneSection interface
	virtual EMovieSceneChannelProxyType CacheChannelProxy() override;
	virtual void SetRange(const TRange<FFrameNumber>& NewRange) override;
	virtual void SetStartFrame(TRangeBound<FFrameNumber> NewStartFrame) override;
	virtual void SetEndFrame(TRangeBound<FFrameNumber> NewEndFrame)override;
	virtual FMovieSceneTimeWarpVariant* GetTimeWarp() override;
	virtual UObject* GetSourceObject() const override;

	MOVIESCENETRACKS_API void DeleteChannels(TArrayView<const FName> ChannelNames);

	struct FRootMotionParams
	{
		bool bBlendFirstChildOfRoot;
		int32 ChildBoneIndex;
		TOptional<FTransform> Transform;
		TOptional<FTransform> PreviousTransform;
	};
protected:

	virtual TOptional<TRange<FFrameNumber> > GetAutoSizeRange() const override;
	virtual void TrimSection(FQualifiedFrameTime TrimTime, bool bTrimLeft, bool bDeleteKeys) override;
	virtual UMovieSceneSection* SplitSection(FQualifiedFrameTime SplitTime, bool bDeleteKeys) override;
	virtual void GetSnapTimes(TArray<FFrameNumber>& OutSnapTimes, bool bGetSectionBorders) const override;
	virtual TOptional<FFrameTime> GetOffsetTime() const override;
	virtual void MigrateFrameTimes(FFrameRate SourceRate, FFrameRate DestinationRate) override;
	virtual float GetTotalWeightValue(FFrameTime InTime) const override;

	/** ~UObject interface */
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;

	/** ~IMovieSceneEntityProvider interface */
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& InParams, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

private:

	//~ UObject interface

#if WITH_EDITOR
	virtual void PreEditChange(FProperty* PropertyAboutToChange) override;
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;

public:
	virtual bool Modify(bool bAlwaysMarkDirty = true) override;
	float PreviousPlayRate;
private:
	virtual void PostEditImport() override;
	virtual void PostEditUndo() override;

#endif

private:

	UPROPERTY()
	TObjectPtr<class UAnimSequence> AnimSequence_DEPRECATED;

	UPROPERTY()
	TObjectPtr<UAnimSequenceBase> Animation_DEPRECATED;

	UPROPERTY()
	float StartOffset_DEPRECATED;
	
	UPROPERTY()
	float EndOffset_DEPRECATED;
	
	UPROPERTY()
	float PlayRate_DEPRECATED;

	UPROPERTY()
	uint32 bReverse_DEPRECATED:1;

	UPROPERTY()
	FName SlotName_DEPRECATED;

public:
	/* Location offset applied before the matched offset*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Root Motions")
	FVector StartLocationOffset;

	/* Location offset applied after the matched offset*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Root Motions")

	FRotator StartRotationOffset;

	UPROPERTY()
	FName MatchedBoneName;

	/* Location offset determined by matching*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Root Motions")
	FVector MatchedLocationOffset;

	/* Rotation offset determined by matching*/
	UPROPERTY(VisibleAnywhere, BlueprintReadOnly, Category = "Root Motions")
	FRotator MatchedRotationOffset;

	UPROPERTY()
	uint8 bMatchWithPrevious : 1;

	UPROPERTY()
	uint8 bMatchTranslation : 1;

	UPROPERTY()
	uint8 bMatchIncludeZHeight : 1;

	UPROPERTY()
	uint8 bMatchRotationYaw : 1;

	UPROPERTY()
	uint8 bMatchRotationPitch : 1;

	UPROPERTY()
	uint8 bMatchRotationRoll : 1;

	UPROPERTY()
	uint8 bDebugForceTickPose : 1;

#if WITH_EDITORONLY_DATA
	/** Whether to show the underlying skeleton for this section. */
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Category = "Root Motions")
	uint8 bShowSkeleton : 1;
#endif

	//Previous transform used to specify the global OffsetTransform while calculting the root motions.
	FTransform PreviousTransform;

	//Temporary index used by GetRootMotionTransform and set by SetBoneIndexforRootMotionCalculations
	TOptional<int32> TempRootBoneIndex;
public:

	struct FRootMotionTransformParam
	{
		FFrameTime CurrentTime; //current time
		FFrameRate FrameRate; //scene frame rate
		bool bOutIsAdditive; //if this is additive or not
		FTransform OutTransform; //total transform including current pose plus offset
		FTransform OutParentTransform; //offset transform not including original bone transform
		FTransform OutPoseTransform; //original bone transform
		FTransform OutRootStartTransform;//start of the root
		float OutWeight; //weight at specified time
	};

	// Functions/params related to root motion calcuations.
	MOVIESCENETRACKS_API FMovieSceneSkeletalAnimRootMotionTrackParams* GetRootMotionParams() const;

	MOVIESCENETRACKS_API bool GetRootMotionVelocity(FFrameTime PreviousTime, FFrameTime CurrentTime, FFrameRate InTickResolution, FTransform& OutVelocity, float& OutWeight) const;
	MOVIESCENETRACKS_API int32 SetBoneIndexForRootMotionCalculations(bool bBlendFirstChildOfRoot);
	MOVIESCENETRACKS_API bool GetRootMotionTransform(FAnimationPoseData& PoseData, FRootMotionTransformParam& InOutParams) const;
	MOVIESCENETRACKS_API void MatchSectionByBoneTransform(USkeletalMeshComponent* SkelMeshComp, FFrameTime CurrentFrame, FFrameRate InTickResolution,
		const FName& BoneName); //add options for z and rotation

	MOVIESCENETRACKS_API void ClearMatchedOffsetTransforms();
	
	MOVIESCENETRACKS_API void GetRootMotion(FFrameTime CurrentTime, FRootMotionParams& OutRootMotionParams) const;

	MOVIESCENETRACKS_API void ToggleMatchTranslation();

	MOVIESCENETRACKS_API void ToggleMatchIncludeZHeight();

	MOVIESCENETRACKS_API void ToggleMatchIncludeYawRotation();

	MOVIESCENETRACKS_API void ToggleMatchIncludePitchRotation();

	MOVIESCENETRACKS_API void ToggleMatchIncludeRollRotation();

#if WITH_EDITORONLY_DATA
	MOVIESCENETRACKS_API void ToggleShowSkeleton();
#endif

	MOVIESCENETRACKS_API FTransform GetRootMotionStartOffset() const;

private:
	void MultiplyOutInverseOnNextClips(FVector PreviousMatchedLocationOffset, FRotator PreviousMatchedRotationOffset);

};
