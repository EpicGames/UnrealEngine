// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneFloatChannel.h"
#include "CoreMinimal.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "Sections/MovieSceneFloatSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneSlomoSection.generated.h"

class UObject;


/**
 * A single floating point section.
 */
UCLASS(MinimalAPI)
class UMovieSceneSlomoSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

	/** Default constructor. */
	UMovieSceneSlomoSection();

	/** IMovieSceneEntityProvider interface */
	void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

public:

	/** Float data */
	UPROPERTY()
	FMovieSceneFloatChannel FloatCurve;
};
