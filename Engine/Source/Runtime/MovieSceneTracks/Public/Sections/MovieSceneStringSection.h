// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneStringChannel.h"
#include "CoreMinimal.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneStringSection.generated.h"

class UObject;

/**
 * A single string section
 */
UCLASS(MinimalAPI)
class UMovieSceneStringSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Public access to this section's internal data function
	 */
	const FMovieSceneStringChannel& GetChannel() const { return StringCurve; }

	//~ IMovieSceneEntityProvider interface
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;

private:

	/** Curve data */
	UPROPERTY()
	FMovieSceneStringChannel StringCurve;
};
