// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneTextChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneSection.h"
#include "MovieSceneTextSection.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneTextSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

public:
	UMovieSceneTextSection(const FObjectInitializer& InObjectInitializer);

	/** Public access to this section's internal data function */
	const FMovieSceneTextChannel& GetChannel() const
	{
		return TextChannel;
	}

	//~ Begin IMovieSceneEntityProvider
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	//~ End IMovieSceneEntityProvider

private:
	UPROPERTY()
	FMovieSceneTextChannel TextChannel;
};
