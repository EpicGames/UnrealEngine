// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Curves/KeyHandle.h"
#include "Curves/RichCurve.h"
#include "MovieSceneSection.h"
#include "MovieSceneKeyStruct.h"
#include "Channels/MovieSceneFloatChannel.h"
#include "Channels/MovieSceneDoubleChannel.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneVectorSection.generated.h"

class FStructOnScope;
struct FPropertyChangedEvent;

/**
* Base Proxy structure for float vector section key data.
*/
USTRUCT()
struct FMovieSceneFloatVectorKeyStructBase
	: public FMovieSceneKeyStruct
{
	GENERATED_BODY();

	/** The key's time. */
	UPROPERTY(EditAnywhere, Category=Key)
	FFrameNumber Time;

	FMovieSceneKeyStructHelper KeyStructInterop;

	virtual void PropagateChanges(const FPropertyChangedEvent& ChangeEvent) override;

	/** Gets a ptr value of a channel by index, 0-3 = x-w */
	virtual float* GetPropertyChannelByIndex(int32 Index) PURE_VIRTUAL(FMovieSceneFloatVectorKeyStructBase::GetPropertyChannelByIndex, return nullptr; );
};
template<> struct TStructOpsTypeTraits<FMovieSceneFloatVectorKeyStructBase> : public TStructOpsTypeTraitsBase2<FMovieSceneFloatVectorKeyStructBase> { enum { WithCopy = false, WithPureVirtual = true, }; };


/**
 * Proxy structure for 2f vector section key data.
 */
USTRUCT()
struct FMovieSceneVector2fKeyStruct
	: public FMovieSceneFloatVectorKeyStructBase
{
	GENERATED_BODY()

	/** They key's vector value. */
	UPROPERTY(EditAnywhere, Category=Key)
	FVector2f Vector = FVector2f::ZeroVector;

	//~ FMovieSceneFloatVectorKeyStructBase interface
	virtual float* GetPropertyChannelByIndex(int32 Index) override { return &Vector[Index]; }
};
template<> struct TStructOpsTypeTraits<FMovieSceneVector2fKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieSceneVector2fKeyStruct> { enum { WithCopy = false }; };

/**
* Proxy structure for float vector section key data.
*/
USTRUCT()
struct FMovieSceneVector3fKeyStruct
	: public FMovieSceneFloatVectorKeyStructBase
{
	GENERATED_BODY()

	/** They key's vector value. */
	UPROPERTY(EditAnywhere, Category = Key)
	FVector3f Vector = FVector3f::ZeroVector;

	//~ FMovieSceneFloatVectorKeyStructBase interface
	virtual float* GetPropertyChannelByIndex(int32 Index) override { return &Vector[Index]; }
};
template<> struct TStructOpsTypeTraits<FMovieSceneVector3fKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieSceneVector3fKeyStruct> { enum { WithCopy = false }; };

/**
* Proxy structure for vector4f section key data.
*/
USTRUCT()
struct FMovieSceneVector4fKeyStruct
	: public FMovieSceneFloatVectorKeyStructBase
{
	GENERATED_BODY()

	/** They key's vector value. */
	UPROPERTY(EditAnywhere, Category = Key)
	FVector4f Vector = FVector4f(FVector3f::ZeroVector);

	//~ FMovieSceneFloatVectorKeyStructBase interface
	virtual float* GetPropertyChannelByIndex(int32 Index) override { return &Vector[Index]; }
};
template<> struct TStructOpsTypeTraits<FMovieSceneVector4fKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieSceneVector4fKeyStruct> { enum { WithCopy = false }; };


/**
* Base Proxy structure for double vector section key data.
*/
USTRUCT()
struct FMovieSceneDoubleVectorKeyStructBase
	: public FMovieSceneKeyStruct
{
	GENERATED_BODY();

	/** The key's time. */
	UPROPERTY(EditAnywhere, Category=Key)
	FFrameNumber Time;

	FMovieSceneKeyStructHelper KeyStructInterop;

	virtual void PropagateChanges(const FPropertyChangedEvent& ChangeEvent) override;

	/** Gets a ptr value of a channel by index, 0-3 = x-w */
	virtual double* GetPropertyChannelByIndex(int32 Index) PURE_VIRTUAL(FMovieSceneDoubleVectorKeyStructBase::GetPropertyChannelByIndex, return nullptr; );
};
template<> struct TStructOpsTypeTraits<FMovieSceneDoubleVectorKeyStructBase> : public TStructOpsTypeTraitsBase2<FMovieSceneDoubleVectorKeyStructBase> { enum { WithCopy = false, WithPureVirtual = true, }; };

/**
 * Proxy structure for 2D vector section key data.
 */
USTRUCT()
struct FMovieSceneVector2DKeyStruct
	: public FMovieSceneDoubleVectorKeyStructBase
{
	GENERATED_BODY()

	/** They key's vector value. */
	UPROPERTY(EditAnywhere, Category=Key)
	FVector2D Vector = FVector2D::ZeroVector;

	//~ FMovieSceneDoubleVectorKeyStructBase interface
	virtual double* GetPropertyChannelByIndex(int32 Index) override { return (double*)&Vector[Index]; }
};
template<> struct TStructOpsTypeTraits<FMovieSceneVector2DKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieSceneVector2DKeyStruct> { enum { WithCopy = false }; };

/**
* Proxy structure for double vector section key data.
*/
USTRUCT()
struct FMovieSceneVector3dKeyStruct
	: public FMovieSceneDoubleVectorKeyStructBase
{
	GENERATED_BODY()

	/** They key's vector value. */
	UPROPERTY(EditAnywhere, Category = Key)
	FVector3d Vector = FVector3d::ZeroVector;

	//~ FMovieSceneDoubleVectorKeyStructBase interface
	virtual double* GetPropertyChannelByIndex(int32 Index) override { return &Vector[Index]; }
};
template<> struct TStructOpsTypeTraits<FMovieSceneVector3dKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieSceneVector3dKeyStruct> { enum { WithCopy = false }; };


/**
* Proxy structure for double vector section key data.
*/
USTRUCT()
struct FMovieSceneVector4dKeyStruct
	: public FMovieSceneDoubleVectorKeyStructBase
{
	GENERATED_BODY()

	/** They key's vector value. */
	UPROPERTY(EditAnywhere, Category = Key)
	FVector4d Vector = FVector4d(FVector3d::ZeroVector);

	//~ FMovieSceneDoubleVectorKeyStructBase interface
	virtual double* GetPropertyChannelByIndex(int32 Index) override { return &Vector[Index]; }
};
template<> struct TStructOpsTypeTraits<FMovieSceneVector4dKeyStruct> : public TStructOpsTypeTraitsBase2<FMovieSceneVector4dKeyStruct> { enum { WithCopy = false }; };


/**
 * A float vector section.
 */
UCLASS(MinimalAPI)
class UMovieSceneFloatVectorSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	/** Sets how many channels are to be used */
	void SetChannelsUsed(int32 InChannelsUsed) 
	{
		checkf(InChannelsUsed >= 2 && InChannelsUsed <= 4, TEXT("Only 2-4 channels are supported."));
		ChannelsUsed = InChannelsUsed;
		RecreateChannelProxy();
	}

	/** Gets the number of channels in use */
	int32 GetChannelsUsed() const { return ChannelsUsed; }

	/**
	 * Public access to this section's internal data function
	 */
	const FMovieSceneFloatChannel& GetChannel(int32 Index) const
	{
		check(Index >= 0 && Index < GetChannelsUsed());
		return Curves[Index];
	}

protected:

	//~ UMovieSceneSection interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostEditImport() override;
	virtual TSharedPtr<FStructOnScope> GetKeyStruct(TArrayView<const FKeyHandle> KeyHandles) override;

	MOVIESCENETRACKS_API void RecreateChannelProxy();

private:

	//~ IMovieSceneEntityProvider interface
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

private:

	/** Float functions for the X,Y,Z,W components of the vector */
	UPROPERTY()
	FMovieSceneFloatChannel Curves[4];

	/** How many curves are actually used */
	UPROPERTY()
	int32 ChannelsUsed;
};


/**
 * A double vector section.
 */
UCLASS(MinimalAPI)
class UMovieSceneDoubleVectorSection
	: public UMovieSceneSection
	, public IMovieSceneEntityProvider
{
	GENERATED_UCLASS_BODY()

public:

	/** Sets how many channels are to be used */
	void SetChannelsUsed(int32 InChannelsUsed) 
	{
		checkf(InChannelsUsed >= 2 && InChannelsUsed <= 4, TEXT("Only 2-4 channels are supported."));
		ChannelsUsed = InChannelsUsed;
		RecreateChannelProxy();
	}

	/** Gets the number of channels in use */
	int32 GetChannelsUsed() const { return ChannelsUsed; }

	/**
	 * Public access to this section's internal data function
	 */
	const FMovieSceneDoubleChannel& GetChannel(int32 Index) const
	{
		check(Index >= 0 && Index < GetChannelsUsed());
		return Curves[Index];
	}

protected:

	//~ UMovieSceneSection interface
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostEditImport() override;
	virtual TSharedPtr<FStructOnScope> GetKeyStruct(TArrayView<const FKeyHandle> KeyHandles) override;

	MOVIESCENETRACKS_API void RecreateChannelProxy();

private:

	//~ IMovieSceneEntityProvider interface
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

private:

	/** Double functions for the X,Y,Z,W components of the vector */
	UPROPERTY()
	FMovieSceneDoubleChannel Curves[4];

	/** How many curves are actually used */
	UPROPERTY()
	int32 ChannelsUsed;
};
