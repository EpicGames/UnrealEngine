// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Sections/MovieSceneBoolSection.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneVisibilitySection.generated.h"

/**
 * A spawn section.
 */
UCLASS(MinimalAPI)
class UMovieSceneVisibilitySection 
	: public UMovieSceneBoolSection
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

public:

	UMovieSceneVisibilitySection(const FObjectInitializer& Init);

public:

	// IMovieSceneEntityProvider interface
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
};

