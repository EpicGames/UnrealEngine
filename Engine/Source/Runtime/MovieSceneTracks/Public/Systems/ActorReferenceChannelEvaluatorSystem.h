// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"

#include "ActorReferenceChannelEvaluatorSystem.generated.h"

/**
 * System that is responsible for evaluating actor reference channels.
 */
UCLASS(MinimalAPI)
class UActorReferenceChannelEvaluatorSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UActorReferenceChannelEvaluatorSystem(const FObjectInitializer& ObjInit);

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};

