// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "BoolChannelEvaluatorSystem.generated.h"

class UObject;

/**
 * System that is responsible for evaluating bool channels.
 */
UCLASS(MinimalAPI)
class UBoolChannelEvaluatorSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UBoolChannelEvaluatorSystem(const FObjectInitializer& ObjInit);

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};