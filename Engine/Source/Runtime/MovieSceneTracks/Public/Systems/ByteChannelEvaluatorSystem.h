// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ByteChannelEvaluatorSystem.generated.h"

class UObject;

/**
 * System that is responsible for evaluating byte channels.
 */
UCLASS(MinimalAPI)
class UByteChannelEvaluatorSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UByteChannelEvaluatorSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
