// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Set.h"
#include "Containers/SortedMap.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "DoubleChannelEvaluatorSystem.generated.h"

class UMovieSceneEntitySystemLinker;
class UObject;

namespace UE::MovieScene
{
	struct FSourceDoubleChannel;

	namespace Interpolation
	{
		struct FCachedInterpolation;
	}
} // namespace UE::MovieScene

/**
 * System that is responsible for evaluating double channels.
 */
UCLASS(MinimalAPI)
class UDoubleChannelEvaluatorSystem : public UMovieSceneEntitySystem
{
public:


	GENERATED_BODY()

	MOVIESCENETRACKS_API UDoubleChannelEvaluatorSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	MOVIESCENETRACKS_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;

	static MOVIESCENETRACKS_API void RegisterChannelType(TComponentTypeID<UE::MovieScene::FSourceDoubleChannel> SourceChannelType, TComponentTypeID<UE::MovieScene::Interpolation::FCachedInterpolation> CachedInterpolationType, TComponentTypeID<double> ResultType);
};
