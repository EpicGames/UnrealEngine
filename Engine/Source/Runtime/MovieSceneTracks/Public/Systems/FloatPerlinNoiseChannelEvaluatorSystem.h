// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "FloatPerlinNoiseChannelEvaluatorSystem.generated.h"

/**
 * System that is responsible for evaluating double perlin noise channels.
 */
UCLASS(MinimalAPI)
class UFloatPerlinNoiseChannelEvaluatorSystem : public UMovieSceneEntitySystem
{
public:
	GENERATED_BODY()

	MOVIESCENETRACKS_API UFloatPerlinNoiseChannelEvaluatorSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
