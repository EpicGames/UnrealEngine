// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "IntegerChannelEvaluatorSystem.generated.h"

/**
 * System that is responsible for evaluating integer channels.
 */
UCLASS(MinimalAPI)
class UIntegerChannelEvaluatorSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UIntegerChannelEvaluatorSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
