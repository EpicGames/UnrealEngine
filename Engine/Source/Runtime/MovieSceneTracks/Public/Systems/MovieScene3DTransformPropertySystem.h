// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Systems/MovieScenePropertySystem.h"
#include "MovieScene3DTransformPropertySystem.generated.h"


UCLASS(MinimalAPI)
class UMovieScene3DTransformPropertySystem : public UMovieScenePropertySystem
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScene3DTransformPropertySystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
