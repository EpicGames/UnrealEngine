// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Systems/MovieScenePropertySystem.h"

#include "MovieSceneActorReferencePropertySystem.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneActorReferencePropertySystem : public UMovieScenePropertySystem
{
	GENERATED_BODY()

	UMovieSceneActorReferencePropertySystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};

