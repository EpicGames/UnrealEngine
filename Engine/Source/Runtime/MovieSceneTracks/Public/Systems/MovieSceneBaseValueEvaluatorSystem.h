// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneBaseValueEvaluatorSystem.generated.h"

class UObject;

/**
 * System that is responsible for evaluating base values, for "additive from base" blending.
 */
UCLASS(MinimalAPI)
class UMovieSceneBaseValueEvaluatorSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneBaseValueEvaluatorSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};

