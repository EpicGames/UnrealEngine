// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "EntitySystem/MovieSceneEntitySystemLinker.h"
#include "EntitySystem/MovieSceneEntitySystemTask.h"
#include "Templates/UnrealTypeTraits.h"

namespace UE
{
namespace MovieScene
{

/**
 * Simple blended result where a value has been accumulated by a number of contributors.
 */
template<typename PropertyType>
struct TSimpleBlendResult
{
	PropertyType Value;
	uint32 NumContributors;
};

/**
 * Traits class for knowing how to deal with a simply blended property type.
 *
 * This is the default implementation which returns the average of all contributions.
 */
template<typename PropertyType>
struct TSimpleBlendResultTraits
{
	/** Reset accumulated values to their default */
	static void ZeroAccumulationBuffer(TArrayView<TSimpleBlendResult<PropertyType>> Buffer)
	{
		if (Buffer.Num() > 0)
		{
			FMemory::Memzero(Buffer.GetData(), sizeof(TSimpleBlendResult<PropertyType>) * Buffer.Num());
		}
	}

	/** Accumulate a value on top of already accumulated values */
	static void AccumulateResult(TSimpleBlendResult<PropertyType>& InOutValue, typename TCallTraits<PropertyType>::ParamType Contributor)
	{
		InOutValue.Value += Contributor;
		++InOutValue.NumContributors;
	}

	/** Get the final blended value */
	static PropertyType BlendResult(const TSimpleBlendResult<PropertyType>& InResult)
	{
		return InResult.Value / InResult.NumContributors;
	}
};

template<typename PropertyType, typename ResultTraits = TSimpleBlendResultTraits<PropertyType>>
struct TSimpleBlenderGatherResults
{
	FMovieSceneBlenderSystemID SystemID;
	TArray<TSimpleBlendResult<PropertyType>>& BlendChannelResults;

	TSimpleBlenderGatherResults(FMovieSceneBlenderSystemID InSystemID, TArray<TSimpleBlendResult<PropertyType>>& InBlendChannelResults)
		: SystemID(InSystemID)
		, BlendChannelResults(InBlendChannelResults)
	{}

	void ForEachEntity(FMovieSceneBlendChannelID BlendChannelInput, PropertyType Value) const
	{
		ensureMsgf(BlendChannelInput.SystemID == SystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));
		if (ensure(BlendChannelResults.IsValidIndex(BlendChannelInput.ChannelID)))
		{
			ResultTraits::AccumulateResult(BlendChannelResults[BlendChannelInput.ChannelID], Value);
		}
	}
};

template<typename PropertyType, typename ResultTraits = TSimpleBlendResultTraits<PropertyType>>
struct TSimpleBlenderCombineResults
{
	FMovieSceneBlenderSystemID SystemID;
	TArray<TSimpleBlendResult<PropertyType>>& BlendChannelResults;

	TSimpleBlenderCombineResults(FMovieSceneBlenderSystemID InSystemID, TArray<TSimpleBlendResult<PropertyType>>& InBlendChannelResults)
		: SystemID(InSystemID)
		, BlendChannelResults(InBlendChannelResults)
	{}

	void ForEachEntity(FMovieSceneBlendChannelID BlendChannelOutput, PropertyType& OutValue) const
	{
		ensureMsgf(BlendChannelOutput.SystemID == SystemID, TEXT("Overriding the standard blender system of standard types isn't supported."));
		if (ensure(BlendChannelResults.IsValidIndex(BlendChannelOutput.ChannelID)))
		{
			OutValue = ResultTraits::BlendResult(BlendChannelResults[BlendChannelOutput.ChannelID]);
		}
	}
};

/**
 * A helper class for simple blender systems.
 *
 * This class implements a simple blender system that accumulates all contributors to a blend channel, and returns 
 * the blended value, as defined by the value type's traits class (above).
 */
template<typename PropertyType, typename ResultTraits = TSimpleBlendResultTraits<PropertyType>>
class TSimpleBlenderSystemImpl
{
public:

	using FGatherResults = TSimpleBlenderGatherResults<PropertyType, ResultTraits>;
	using FCombineResults = TSimpleBlenderCombineResults<PropertyType, ResultTraits>;

	TSimpleBlenderSystemImpl()
		: BlenderSystem(nullptr)
	{}

	/**
	 * Sets up this helper class. To be called in the owning blender system's constructor.
	 *
	 * @param InBlenderSystem       The owning blender system
	 * @param InResultComponentID   The component type for the values to blend
	 * @param ChannelEvaluationSystem  The evaluation system for the value to blend. If set, the owner blender system will be automatically made a downstream dependency of it (optional)
	 */
	void Setup(UMovieSceneBlenderSystem* InBlenderSystem, TComponentTypeID<PropertyType> InResultComponentID, UClass* ChannelEvaluatorSystem = nullptr)
	{
		BlenderSystem = InBlenderSystem;
		ResultComponentID = InResultComponentID;

		if (ChannelEvaluatorSystem && BlenderSystem->HasAnyFlags(RF_ClassDefaultObject))
		{
			BlenderSystem->DefineImplicitPrerequisite(ChannelEvaluatorSystem, BlenderSystem->GetClass());
		}
	}

	void Schedule(UMovieSceneEntitySystemLinker* Linker, TBitArray<>& AllocatedBlendChannels, IEntitySystemScheduler* TaskScheduler)
	{
		using namespace UE::MovieScene;

		// If we have no blend channels, we're done.
		const int32 MaximumNumBlends = AllocatedBlendChannels.Num();
		if (MaximumNumBlends == 0)
		{
			return;
		}

		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		BlendChannelResults.Reset();

		const bool bHasChannels = Linker->EntityManager.Contains(FEntityComponentFilter().All({
					ResultComponentID, 
					BlenderSystem->GetBlenderTypeTag(),
					BuiltInComponents->BlendChannelInput }));
		if (bHasChannels)
		{
			BlendChannelResults.SetNum(MaximumNumBlends);
		}

		// Reset the result buffer to the default values, as defined by the value type's traits.
		FTaskID ResetWeightsTask = TaskScheduler->AddMemberFunctionTask(
			FTaskParams(TEXT("Reset Simple Blender Weights")),
			this, &TSimpleBlenderSystemImpl::ZeroAccumulationBuffers);

		// Accumulate all contributors into the result buffer.
		FTaskID AccumulateTask = FEntityTaskBuilder()
			.Read(BuiltInComponents->BlendChannelInput)
			.Read(ResultComponentID)
			.FilterAll({ BlenderSystem->GetBlenderTypeTag() })
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.template Schedule_PerEntity<FGatherResults>(
					&Linker->EntityManager, TaskScheduler, BlenderSystem->GetBlenderSystemID(), BlendChannelResults);

		TaskScheduler->AddPrerequisite(ResetWeightsTask, AccumulateTask);

		// Compute the final blended values and assign them to the blend channel output entities.
		FTaskID CombineTask = FEntityTaskBuilder()
			.Read(BuiltInComponents->BlendChannelOutput)
			.Write(ResultComponentID)
			.FilterAll({ BlenderSystem->GetBlenderTypeTag() })
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.template Fork_PerEntity<FCombineResults>(
					&Linker->EntityManager, TaskScheduler, BlenderSystem->GetBlenderSystemID(), BlendChannelResults);

		TaskScheduler->AddPrerequisite(AccumulateTask, CombineTask);
	}

	/**
	 * Runs the blender system
	 */
	void Run(UMovieSceneEntitySystemLinker* Linker, TBitArray<>& AllocatedBlendChannels, FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
	{
		using namespace UE::MovieScene;

		// If we have no blend channels, we're done.
		const int32 MaximumNumBlends = AllocatedBlendChannels.Num();
		if (MaximumNumBlends == 0)
		{
			return;
		}

		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();

		// If the entity manager structure has been modified, let's reallocate our result buffer.
		if (ChannelRelevancyCache.Update(Linker->EntityManager) == ECachedEntityManagerState::Stale)
		{
			BlendChannelResults.Reset();

			const bool bHasChannels = Linker->EntityManager.Contains(FEntityComponentFilter().All({
						ResultComponentID, 
						BlenderSystem->GetBlenderTypeTag(),
						BuiltInComponents->BlendChannelInput }));
			if (bHasChannels)
			{
				BlendChannelResults.SetNum(MaximumNumBlends);
			}
		}

		using FResultTraits = TSimpleBlendResultTraits<PropertyType>;

		// Reset the result buffer to the default values, as defined by the value type's traits.
		FResultTraits::ZeroAccumulationBuffer(MakeArrayView(BlendChannelResults));

		// Accumulate all contributors into the result buffer.
		FSystemTaskPrerequisites Prereqs;

		FGraphEventRef Task = FEntityTaskBuilder()
			.Read(BuiltInComponents->BlendChannelInput)
			.Read(ResultComponentID)
			.FilterAll({ BlenderSystem->GetBlenderTypeTag() })
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.template Dispatch_PerEntity<FGatherResults>(
					&Linker->EntityManager, InPrerequisites, nullptr, BlenderSystem->GetBlenderSystemID(), BlendChannelResults);
		if (Task)
		{
			Prereqs.AddRootTask(Task);
		}

		// Compute the final blended values and assign them to the blend channel output entities.
		FEntityTaskBuilder()
			.Read(BuiltInComponents->BlendChannelOutput)
			.Write(ResultComponentID)
			.FilterAll({ BlenderSystem->GetBlenderTypeTag() })
			.FilterNone({ BuiltInComponents->Tags.Ignored })
			.template Dispatch_PerEntity<FCombineResults>(
					&Linker->EntityManager, Prereqs, &Subsequents, BlenderSystem->GetBlenderSystemID(), BlendChannelResults);
	}

	void ZeroAccumulationBuffers()
	{
		using FResultTraits = TSimpleBlendResultTraits<PropertyType>;
		FResultTraits::ZeroAccumulationBuffer(MakeArrayView(BlendChannelResults));
	}

private:
	UMovieSceneBlenderSystem* BlenderSystem;
	TComponentTypeID<PropertyType> ResultComponentID;

	FCachedEntityManagerState ChannelRelevancyCache;

	TArray<TSimpleBlendResult<PropertyType>> BlendChannelResults;
};

}
}
