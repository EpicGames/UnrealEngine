// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntitySystemLinkerExtension.h"
#include "EntitySystem/MovieSceneEntitySystemLinkerSharedExtension.h"
#include "Evaluation/MovieSceneCameraShakePreviewer.h"
#include "Sections/MovieSceneCameraShakeSourceTriggerSection.h"

#include "MovieSceneCameraShakeSystem.generated.h"

namespace UE::MovieScene
{
	struct FPreAnimatedCameraShakeStateStorage;
	struct FPreAnimatedCameraComponentShakeStateStorage;
	struct FPreAnimatedCameraSourceShakeStateStorage;

#if WITH_EDITOR
	/**
	 * Linker extension for storing camera shake previewers shared between the instantiator and the
	 * evaluation systems below. This extension is held by a shared pointer, so it should unregister
	 * and delete itself when both systems are unlinked.
	 */
	struct FCameraShakePreviewerLinkerExtension : public TSharedEntitySystemLinkerExtension<FCameraShakePreviewerLinkerExtension>
	{
		/** The ID of this extension */
		static TEntitySystemLinkerExtensionID<FCameraShakePreviewerLinkerExtension> GetExtensionID();

		/** Get the extension from the given linker, or create a new one */
		static TSharedPtr<FCameraShakePreviewerLinkerExtension> GetOrCreateExtension(UMovieSceneEntitySystemLinker* Linker);

		/** Constructor */
		FCameraShakePreviewerLinkerExtension(UMovieSceneEntitySystemLinker* Linker);
		/** Destructor */
		virtual ~FCameraShakePreviewerLinkerExtension();

		/** Find the previewer for the given sequence instance, if it has already been created */
		TSharedPtr<FCameraShakePreviewer> FindPreviewer(FInstanceHandle InstanceHandle);
		/** Get or create the previewer for the given sequence instance */
		TSharedRef<FCameraShakePreviewer> GetPreviewer(FInstanceHandle InstanceHandle);
		/** Update all previewers by using the update context of each matching sequence instance */
		void UpdateAllPreviewers();
		/** Whether there is any previewer with any shake */
		bool HasAnyShake() const;

	private:
		void OnLevelViewportClientListChanged();

	private:
		TMap<FInstanceHandle, TSharedRef<FCameraShakePreviewer>> Previewers;
	};
#endif
}

/**
 * The instantiator system for camera shakes.
 *
 * This system will create and initialize camera shakes as they come in, and save pre-animated state.
 */
UCLASS()
class UMovieSceneCameraShakeInstantiatorSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:
	UMovieSceneCameraShakeInstantiatorSystem(const FObjectInitializer& ObjInit);

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnLink() override;
	virtual void OnUnlink() override;

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	/** Add a fire-and-forget shake trigger */
	void AddShakeTrigger(UE::MovieScene::FInstanceHandle InInstance, const FGuid& ObjectBindingID, const FFrameTime& InTime, const FMovieSceneCameraShakeSourceTrigger& InTrigger);

private:
	void TriggerOneShotShakes();

#if WITH_EDITOR
	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);
#endif

private:
	struct FTimedTrigger
	{
		FGuid ObjectBindingID;
		FFrameTime Time;
		FMovieSceneCameraShakeSourceTrigger Trigger;
	};

	TSharedPtr<UE::MovieScene::FPreAnimatedCameraShakeStateStorage> PreAnimatedCameraShakeStorage;
	TSharedPtr<UE::MovieScene::FPreAnimatedCameraComponentShakeStateStorage> PreAnimatedCameraComponentShakeStorage;
	TSharedPtr<UE::MovieScene::FPreAnimatedCameraSourceShakeStateStorage> PreAnimatedCameraSourceShakeStorage;

	TMap<UE::MovieScene::FInstanceHandle, TArray<FTimedTrigger>> TriggersByInstance;

#if WITH_EDITOR
	TSharedPtr<UE::MovieScene::FCameraShakePreviewerLinkerExtension> PreviewerExtension;
#endif
};

/**
 * The evaluator system for camera shakes.
 *
 * This system takes care of updating ongoing camera shakes according to the evaluation range.
 */
UCLASS()
class UMovieSceneCameraShakeEvaluatorSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:
	UMovieSceneCameraShakeEvaluatorSystem(const FObjectInitializer& ObjInit);

	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:
#if WITH_EDITOR
	TSharedPtr<UE::MovieScene::FCameraShakePreviewerLinkerExtension> PreviewerExtension;
#endif
};
