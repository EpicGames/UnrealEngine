// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/IMovieScenePropertyComponentHandler.h"
#include "Systems/MovieScenePropertySystem.h"
#include "MovieSceneColorPropertySystem.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneColorPropertySystem : public UMovieScenePropertySystem
{
	GENERATED_BODY()

private:
	UMovieSceneColorPropertySystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
