// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"

#include "MovieSceneTracksComponentTypes.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "EntitySystem/MovieSceneOverlappingEntityTracker.h"

#include "MovieSceneComponentAttachmentSystem.generated.h"

struct FMovieSceneAnimTypeID;

class USceneComponent;
class UMovieScenePreAnimatedComponentTransformSystem;

namespace UE
{
namespace MovieScene
{

struct FPreAnimAttachment
{
	TWeakObjectPtr<USceneComponent> OldAttachParent;
	FName OldAttachSocket;
	UE::MovieScene::FComponentDetachParams DetachParams;
};

} // namespace MovieScene
} // namespace UE


UCLASS(MinimalAPI)
class UMovieSceneComponentAttachmentInvalidatorSystem
	: public UMovieSceneEntityInstantiatorSystem
{
public:

	GENERATED_BODY()

	UMovieSceneComponentAttachmentInvalidatorSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;
};

UCLASS(MinimalAPI)
class UMovieSceneComponentAttachmentSystem
	: public UMovieSceneEntityInstantiatorSystem
	, public IMovieScenePreAnimatedStateSystemInterface
{
public:

	GENERATED_BODY()

	UMovieSceneComponentAttachmentSystem(const FObjectInitializer& ObjInit);

	void AddPendingDetach(USceneComponent* SceneComponent, const UE::MovieScene::FPreAnimAttachment& Attachment);

private:

	virtual void OnLink() override final;
	virtual void OnUnlink() override final;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

	//~ IMovieScenePreAnimatedStateSystemInterface interface
	virtual void SavePreAnimatedState(const FPreAnimationParameters& InParameters) override;
	virtual void RestorePreAnimatedState(const FPreAnimationParameters& InParameters) override;

	UE::MovieScene::TOverlappingEntityTracker<UE::MovieScene::FPreAnimAttachment, FObjectKey> AttachmentTracker;

	TArray<TTuple<USceneComponent*, UE::MovieScene::FPreAnimAttachment>> PendingAttachmentsToRestore;

	UE::MovieScene::FEntityGroupingPolicyKey GroupingKey;
};



