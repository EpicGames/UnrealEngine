// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/TypeHash.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"

#include "Systems/MovieSceneMaterialSystem.h"
#include "Tracks/MovieSceneMaterialTrack.h"

#include "MovieSceneComponentMaterialSystem.generated.h"

class UMovieScenePiecewiseDoubleBlenderSystem;

namespace UE::MovieScene
{

struct FComponentMaterialKey
{
	FObjectKey Object;
	FComponentMaterialInfo MaterialInfo;

	friend uint32 GetTypeHash(const FComponentMaterialKey& In)
	{
		return HashCombine(GetTypeHash(In.Object), GetTypeHash(In.MaterialInfo));
	}
	friend bool operator==(const FComponentMaterialKey& A, const FComponentMaterialKey& B)
	{
		return A.Object == B.Object && A.MaterialInfo == B.MaterialInfo;
	}
};

struct FComponentMaterialAccessor
{
	using KeyType = FComponentMaterialKey;

	UObject* Object;
	FComponentMaterialInfo MaterialInfo;

	FComponentMaterialAccessor(const FComponentMaterialKey& InKey);
	FComponentMaterialAccessor(UObject* InObject, const FComponentMaterialInfo& InMaterialInfo);

	explicit operator bool() const;

	UMaterialInterface* GetMaterial() const;
	void SetMaterial(UMaterialInterface* InMaterial) const;
	UMaterialInstanceDynamic* CreateDynamicMaterial(UMaterialInterface* InMaterial);
	FString ToString() const;
};

using FPreAnimatedComponentMaterialTraits          = TPreAnimatedMaterialTraits<FComponentMaterialAccessor, UObject*, FComponentMaterialInfo>;
using FPreAnimatedComponentMaterialParameterTraits = TPreAnimatedMaterialParameterTraits<FComponentMaterialAccessor, UObject*, FComponentMaterialInfo>;

struct FPreAnimatedComponentMaterialSwitcherStorage
	: public TPreAnimatedStateStorage<TPreAnimatedMaterialTraits<FComponentMaterialAccessor, UObject*, FComponentMaterialInfo>>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedComponentMaterialSwitcherStorage> StorageID;
};

struct FPreAnimatedComponentMaterialParameterStorage
	: public TPreAnimatedStateStorage<TPreAnimatedMaterialParameterTraits<FComponentMaterialAccessor, UObject*, FComponentMaterialInfo>>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedComponentMaterialParameterStorage> StorageID;
};

} // namespace UE::MovieScene


UCLASS(MinimalAPI)
class UMovieSceneComponentMaterialSystem
	: public UMovieSceneEntitySystem
	, public IMovieScenePreAnimatedStateSystemInterface
{
public:

	GENERATED_BODY()

	UMovieSceneComponentMaterialSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	virtual void SavePreAnimatedState(const FPreAnimationParameters& InParameters) override;

private:

	UE::MovieScene::TMovieSceneMaterialSystem<UE::MovieScene::FComponentMaterialAccessor, UObject*, FComponentMaterialInfo> SystemImpl;
};
