// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "EntitySystem/MovieSceneOverlappingEntityTracker.h"
#include "Engine/EngineTypes.h"
#include "Evaluation/PreAnimatedState/IMovieScenePreAnimatedStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStorageID.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"

#include "MovieSceneComponentMobilitySystem.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneComponentMobilitySystem
	: public UMovieSceneEntityInstantiatorSystem
	, public IMovieScenePreAnimatedStateSystemInterface
{
public:

	GENERATED_BODY()

	UMovieSceneComponentMobilitySystem(const FObjectInitializer& ObjInit);

private:

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnLink() override final;
	virtual void OnUnlink() override final;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

	//~ IMovieScenePreAnimatedStateSystemInterface interface
	virtual void SavePreAnimatedState(const FPreAnimationParameters& InParameters) override;
	virtual void RestorePreAnimatedState(const FPreAnimationParameters& InParameters) override;

private:

	UE::MovieScene::TOverlappingEntityTracker<EComponentMobility::Type, FObjectKey> MobilityTracker;

	UE::MovieScene::FEntityComponentFilter Filter;

	TArray<TTuple<USceneComponent*, EComponentMobility::Type>> PendingMobilitiesToRestore;
};


namespace UE
{
namespace MovieScene
{

struct FPreAnimatedMobilityTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType     = FObjectKey;
	using StorageType = EComponentMobility::Type;

	void RestorePreAnimatedValue(const FObjectKey& InKey, EComponentMobility::Type Mobility, const FRestoreStateParams& Params);
	EComponentMobility::Type CachePreAnimatedValue(UObject* InObject);
};

struct FPreAnimatedComponentMobilityStorage : TPreAnimatedStateStorage_ObjectTraits<FPreAnimatedMobilityTraits>
{
	static TAutoRegisterPreAnimatedStorageID<FPreAnimatedComponentMobilityStorage> StorageID;

	FPreAnimatedStateEntry FindEntry(USceneComponent* InSceneComponent);
	FPreAnimatedStateEntry MakeEntry(USceneComponent* InSceneComponent);
};

} // namespace MovieScene
} // namespace UE

