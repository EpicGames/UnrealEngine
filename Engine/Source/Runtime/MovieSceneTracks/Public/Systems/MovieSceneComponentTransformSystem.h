// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Systems/MovieScenePropertySystem.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "EntitySystem/MovieSceneOverlappingEntityTracker.h"
#include "MovieSceneTracksComponentTypes.h"

#include "MovieSceneComponentTransformSystem.generated.h"


UCLASS(MinimalAPI)
class UMovieSceneComponentTransformSystem : public UMovieScenePropertySystem
{
public:

	GENERATED_BODY()

	UMovieSceneComponentTransformSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
