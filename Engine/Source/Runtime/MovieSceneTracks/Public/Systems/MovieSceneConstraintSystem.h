// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "MovieSceneConstraintSystem.generated.h"

class UTransformableComponentHandle;
class UTickableTransformConstraint;

namespace UE::MovieScene
{
	struct FEvaluateConstraintChannels;
}

/**
 * System that is responsible for propagating constraints to a bound object's FConstraintsManagerController.
 */
UCLASS(MinimalAPI)
class UMovieSceneConstraintSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()
	//if constraint has a dynamic offset we need to make sure the constraint get's notified the handle is updated
	struct FUpdateHandleForConstraint
	{
		TWeakObjectPtr<UTickableTransformConstraint> Constraint;
		TWeakObjectPtr<UTransformableComponentHandle> TransformHandle;
	};

	MOVIESCENETRACKS_API UMovieSceneConstraintSystem(const FObjectInitializer& ObjInit);
	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

protected:
	friend UE::MovieScene::FEvaluateConstraintChannels;

	TArray<FUpdateHandleForConstraint>  DynamicOffsets;
};
