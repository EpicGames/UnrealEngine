// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneOverlappingEntityTracker.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "EntitySystem/BuiltInComponentTypes.h"

#include "MovieSceneCustomPrimitiveDataSystem.generated.h"

class UMovieSceneBlenderSystem;
class UMovieScenePiecewiseDoubleBlenderSystem;

namespace UE::MovieScene
{

struct FPreAnimatedCustomPrimitiveDataEntryStorage;

struct FAnimatedCustomPrimitiveDataInfo
{
	/** Weak linker ptr - only assigned if the output entity is ever allocated */
	TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;
	/** Weak blender system ptr - only assigned if the blend channel is ever allocated */
	TWeakObjectPtr<UMovieSceneBlenderSystem> WeakBlenderSystem;
	int32 NumContributors = 0;
	FMovieSceneEntityID OutputEntityID;
	FMovieSceneBlendChannelID BlendChannelID;

	~FAnimatedCustomPrimitiveDataInfo();
};

MOVIESCENETRACKS_API void CollectGarbageForOutput(FAnimatedCustomPrimitiveDataInfo* Output);

} // namespace UE::MovieScene


/**
 * System responsible for tracking and animating custom primitive data entities.
 * Operates on the following component types from FMovieSceneTracksComponentTypes:
 *
 * Instantiation: Tracks any ScalarParameterName with a CustomPrimitiveData tag on a BoundObject where that BoundObject is a Primitive Component.
 *                Manages adding BlendChannelInputs and Outputs where multiple entities animate the same custom primitive data index.
 *
 * Evaluation:    Visits any BoundObject with the supported parameter name and either a BlendChannelOutput component
 *                or no BlendChannelInput, and applies the resulting parameter to the Custom Primitive Data on the Primitive Component.
 */
UCLASS(MinimalAPI)
class UMovieSceneCustomPrimitiveDataSystem
	: public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneCustomPrimitiveDataSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	void OnInstantiation();
	void OnEvaluation(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents);

private:

	/** Overlapping trackers that track multiple entities animating the same bound object and name */
	UE::MovieScene::TOverlappingEntityTracker<UE::MovieScene::FAnimatedCustomPrimitiveDataInfo, FObjectKey, FName> ScalarParameterTracker;

	/** Holds pre-animated values for custom primitive data entries */
	TSharedPtr<UE::MovieScene::FPreAnimatedCustomPrimitiveDataEntryStorage> ScalarParameterStorage;

	UE::MovieScene::FEntityGroupingPolicyKey GroupingKey;

public:

	UPROPERTY()
	TObjectPtr<UMovieScenePiecewiseDoubleBlenderSystem> DoubleBlenderSystem;
};
