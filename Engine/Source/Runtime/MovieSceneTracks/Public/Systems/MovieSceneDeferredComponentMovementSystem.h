// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Containers/ChunkedArray.h"
#include "Misc/Optional.h"
#include "Engine/ScopedMovementUpdate.h"
#include "MovieSceneDeferredComponentMovementSystem.generated.h"

class USceneComponent;

/**
 * System that maintains a FScopedMovementUpdate for any USceneComponent that has an
 * animated transform or attachment for the duration of the evaluation to avoid repeatedly
 * updating child transforms and/or overlaps.
 * 
 * This system can be enabled/disabled using the cvar Sequencer.DeferMovementUpdates (is disabled by default)
 * Output logging options are controlled by Sequencer.OutputDeferredMovementMode which is useful for tracking
 * how many components are being moved in any given frame.
 *
 * This system runs in the Initialization and Evaluaion phases:
 *     Initialization: Gathers and initializes deferred movement for any AttachParent components that need (un)link
 *     Evaluation: Gathers and initializes deferred movement for any Component Transforms; queues up an event trigger to be executed at the end of the phase
 *     Finalization: Applys all pending movement updates in reverse order
 */
UCLASS(MinimalAPI)
class UMovieSceneDeferredComponentMovementSystem
	: public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneDeferredComponentMovementSystem(const FObjectInitializer& ObjInit);

	/** Request that movement updates for the specified component be deferred until the end of the evaluation */
	MOVIESCENETRACKS_API void DeferMovementUpdates(USceneComponent* InComponent);

private:

	virtual void BeginDestroy() override;

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	virtual void OnUnlink() override final;

	/** Apply all currently pending movement updates */
	void ApplyMovementUpdates();

	/** Output the current movement updates to the log */
	void OutputDeferredMovements();

	/** Check all movements have been flush */
	void EnsureMovementsFlushed();

	/**
	 * WARNING: FScopedMovementUpdate is very specifically designed with strict ordering constraints on
	 * its constructor and destructor. Care is taken here to ensure 2 things:
	 *
	 *    1. FScopedMovementUpdate can never be moved or copied (its address is cached inside USceneComponent::ScopedMovementStack)
	 *    2. Once constructed, they must be destructed in strictly reverse order
	 */
	struct FScopedSequencerMovementUpdate : FScopedMovementUpdate
	{
		FScopedSequencerMovementUpdate(USceneComponent* Component) : FScopedMovementUpdate(Component) {}

		void operator=(const FScopedSequencerMovementUpdate&) = delete;
		FScopedSequencerMovementUpdate(const FScopedSequencerMovementUpdate&) = delete;

		void operator=(FScopedSequencerMovementUpdate&&) = delete;
		FScopedSequencerMovementUpdate(FScopedSequencerMovementUpdate&&) = delete;

		USceneComponent* GetComponent() const;
	};

	/** Struct that allows us to use FScopedMovementUpdate on the heap */
	struct FScopedMovementUpdateContainer
	{
		FScopedMovementUpdateContainer(USceneComponent* Component) : Value(Component) {}

		FScopedSequencerMovementUpdate Value;
	};

	/** Chunked array of movement updates to ensure that once allocated, a movement update's address can never change */
	TChunkedArray<TOptional<FScopedMovementUpdateContainer>, 8192> ScopedUpdates;
};
