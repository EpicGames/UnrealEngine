// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Systems/MovieScenePropertySystem.h"
#include "MovieSceneDoublePropertySystem.generated.h"



UCLASS(MinimalAPI)
class UMovieSceneDoublePropertySystem : public UMovieScenePropertySystem
{
	GENERATED_BODY()

	UMovieSceneDoublePropertySystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
