// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Systems/MovieScenePropertySystem.h"
#include "MovieSceneEnumPropertySystem.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneEnumPropertySystem : public UMovieScenePropertySystem
{
	GENERATED_BODY()

	UMovieSceneEnumPropertySystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
