// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Channels/MovieSceneEvent.h"
#include "Evaluation/IMovieScenePlaybackCapability.h"
#include "MovieSceneEventSystems.generated.h"

namespace UE::MovieScene
{

struct FSharedPlaybackState;

}  // namespace UE::MovieScene

USTRUCT()
struct FMovieSceneEventTriggerData
{
	GENERATED_BODY()

	UPROPERTY()
	FMovieSceneEventPtrs Ptrs;

	UPROPERTY()
	FGuid ObjectBindingID;

	FMovieSceneSequenceID SequenceID;

	FFrameTime RootTime;
};

/**
 * Systems that triggers events based on one-shot FMovieSceneEventComponent components
 * Works by iterating all pending instances of TMovieSceneComponentID<FMovieSceneEventComponent> and triggering inline.
 * Does not dispatch any async tasks
 */

UCLASS(MinimalAPI)
class UMovieSceneEventSystem : public UMovieSceneEntitySystem
{
public:
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneEventSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API void AddEvent(UE::MovieScene::FInstanceHandle RootInstance, const FMovieSceneEventTriggerData& TriggerData);

protected:

	MOVIESCENETRACKS_API bool HasEvents() const;
	MOVIESCENETRACKS_API void TriggerAllEvents();

private:

	using FSharedPlaybackState = UE::MovieScene::FSharedPlaybackState;

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	MOVIESCENETRACKS_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	MOVIESCENETRACKS_API virtual void OnUnlink() override final;

	static void TriggerEvents(TArrayView<const FMovieSceneEventTriggerData> Events, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);
	static void TriggerEventWithParameters(UObject* DirectorInstance, const FMovieSceneEventTriggerData& Event, TArrayView<UObject* const> GlobalContexts, TSharedRef<const FSharedPlaybackState> SharedPlaybackState);
	static bool PatchBoundObject(uint8* Parameters, UObject* BoundObject, FProperty* BoundObjectProperty, TSharedRef<const FSharedPlaybackState> SharedPlaybackState, FMovieSceneSequenceID SequenceID);

	/** Events grouped by root sequence instance handle */
	TMap< UE::MovieScene::FInstanceHandle, TArray<FMovieSceneEventTriggerData> > EventsByRoot;
};

/** System that triggers events before any spawnables */
UCLASS(MinimalAPI)
class UMovieScenePreSpawnEventSystem : public UMovieSceneEventSystem
{
public:
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScenePreSpawnEventSystem(const FObjectInitializer& ObjInit);
};


/** System that triggers events after any spawnables */
UCLASS(MinimalAPI)
class UMovieScenePostSpawnEventSystem : public UMovieSceneEventSystem
{
public:
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScenePostSpawnEventSystem(const FObjectInitializer& ObjInit);
};

/** System that triggers events right at the end of evaluation */
UCLASS(MinimalAPI)
class UMovieScenePostEvalEventSystem : public UMovieSceneEventSystem
{
public:
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScenePostEvalEventSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)override;
};
