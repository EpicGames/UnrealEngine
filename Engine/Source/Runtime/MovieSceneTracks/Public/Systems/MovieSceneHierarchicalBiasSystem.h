// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneHierarchicalBiasSystem.generated.h"

class UObject;
class UMovieSceneEntitySystemLinker;
class UMovieSceneEntityGroupingSystem;

UCLASS()
class UMovieSceneHierarchicalBiasSystem : public UMovieSceneEntityInstantiatorSystem
{
public:

	GENERATED_BODY()

	UMovieSceneHierarchicalBiasSystem(const FObjectInitializer& ObjInit);

private:

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnLink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	UPROPERTY()
	TObjectPtr<UMovieSceneEntityGroupingSystem> GroupingSystem;
};
