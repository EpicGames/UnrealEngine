// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"

#include "Systems/MovieSceneMaterialSystem.h"

#include "MovieSceneMaterialParameterCollectionSystem.generated.h"

namespace UE::MovieScene
{

struct FPreAnimatedScalarMaterialParameterStorage;
struct FPreAnimatedVectorMaterialParameterStorage;

} // namespace UE::MovieScene


/**
 * System that resolves MPC components into BoundMaterial components that can be used by downstream
 * parameter systems
 */
UCLASS(MinimalAPI)
class UMovieSceneMaterialParameterCollectionSystem
	: public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneMaterialParameterCollectionSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:

	/** Holds pre-animated values for scalar MPC values */
	TSharedPtr<UE::MovieScene::FPreAnimatedScalarMaterialParameterStorage> ScalarParameterStorage;
	/** Holds pre-animated values for vector or color MPC values */
	TSharedPtr<UE::MovieScene::FPreAnimatedVectorMaterialParameterStorage> VectorParameterStorage;
	/** Registered key for the grouping policy that groups MPC tracks together */
	UE::MovieScene::FEntityGroupingPolicyKey GroupingKey;
};
