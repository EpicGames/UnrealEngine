// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneOverlappingEntityTracker.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Materials/MaterialParameters.h"

#include "MovieSceneMaterialParameterSystem.generated.h"

class UMovieSceneBlenderSystem;
class UMovieScenePiecewiseDoubleBlenderSystem;

namespace UE::MovieScene
{

struct FPreAnimatedScalarMaterialParameterStorage;
struct FPreAnimatedVectorMaterialParameterStorage;

struct FAnimatedMaterialParameterInfo
{
	/** Weak linker ptr - only assigned if the output entity is ever allocated */
	TWeakObjectPtr<UMovieSceneEntitySystemLinker> WeakLinker;
	/** Weak blender system ptr - only assigned if the blend channel is ever allocated */
	TWeakObjectPtr<UMovieSceneBlenderSystem> WeakBlenderSystem;
	int32 NumContributors = 0;
	FMovieSceneEntityID OutputEntityID;
	FMovieSceneBlendChannelID BlendChannelID;

	~FAnimatedMaterialParameterInfo();
};

MOVIESCENETRACKS_API void CollectGarbageForOutput(FAnimatedMaterialParameterInfo* Output);

} // namespace UE::MovieScene


/**
 * System responsible for tracking material parameter entities.
 *
 * Tracks any BoundMaterial with a MaterialParameterInfo, as well as deprecated ScalarParameterName,
 * ColorParameterName or VectorParameterName.
 * Manages adding BlendChannelInputs and Outputs where multiple entities animate the same parameter
 * on the same bound material.
 * BoundMaterials may be a UMaterialInstanceDynamic, or a UMaterialParameterCollectionInstance.
 */
UCLASS(MinimalAPI)
class UMovieSceneMaterialParameterInstantiatorSystem : public UMovieSceneEntityInstantiatorSystem
{
public:

	GENERATED_BODY()

	UMovieSceneMaterialParameterInstantiatorSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:

	/** Overlapping trackers that track multiple entities animating the same bound object and name */
	UE::MovieScene::TOverlappingEntityTracker<UE::MovieScene::FAnimatedMaterialParameterInfo, UE::MovieScene::FObjectComponent, FMaterialParameterInfo> ScalarParameterTracker;
	UE::MovieScene::TOverlappingEntityTracker<UE::MovieScene::FAnimatedMaterialParameterInfo, UE::MovieScene::FObjectComponent, FMaterialParameterInfo> VectorParameterTracker;

	/** Holds pre-animated values for scalar values */
	TSharedPtr<UE::MovieScene::FPreAnimatedScalarMaterialParameterStorage> ScalarParameterStorage;
	/** Holds pre-animated values for vector or color values */
	TSharedPtr<UE::MovieScene::FPreAnimatedVectorMaterialParameterStorage> VectorParameterStorage;

public:

	UPROPERTY()
	TObjectPtr<UMovieScenePiecewiseDoubleBlenderSystem> DoubleBlenderSystem;
};


/**
 * System responsible for animating material parameter entities.
 *
 * Visits any BoundMaterial with the supported parameter names and either a BlendChannelOutput component
 * or no BlendChannelInput, and applies the resulting parameter to the bound material instance.
 */
UCLASS(MinimalAPI)
class UMovieSceneMaterialParameterEvaluationSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneMaterialParameterEvaluationSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
