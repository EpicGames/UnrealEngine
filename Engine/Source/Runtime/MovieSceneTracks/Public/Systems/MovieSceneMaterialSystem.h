// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "EntitySystem/MovieSceneEntityMutations.h"

#include "Materials/MaterialInstance.h"
#include "Materials/MaterialInstanceDynamic.h"

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/MovieSceneEntitySystemRunner.h"
#include "EntitySystem/MovieSceneComponentTypeInfo.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "EntitySystem/MovieSceneEntityGroupingSystem.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "MovieSceneTracksComponentTypes.h"
#include "UObject/SoftObjectPtr.h"

#include "MovieSceneMaterialSystem.generated.h"


DECLARE_CYCLE_STAT_EXTERN(TEXT("Reinitialize Bound Materials"), MovieSceneEval_ReinitializeBoundMaterials, STATGROUP_MovieSceneECS, MOVIESCENETRACKS_API);

USTRUCT()
struct FMovieScenePreAnimatedMaterialParameters
{
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMaterialInterface* GetMaterial() const;

	MOVIESCENETRACKS_API void SetMaterial(UMaterialInterface* InMaterial);

private:

	/** Strong ptr to the previously assigned material interface (used when Sequencer.UseSoftObjectPtrsForPreAnimatedMaterial is false) */
	UPROPERTY()
	TObjectPtr<UMaterialInterface> PreviousMaterial;

	/** Soft ptr to the previously assigned material interface (used when Sequencer.UseSoftObjectPtrsForPreAnimatedMaterial is true) */
	UPROPERTY()
	TSoftObjectPtr<UMaterialInterface> SoftPreviousMaterial;
};

namespace UE::MovieScene
{

template<typename AccessorType, typename... RequiredComponents>
struct TPreAnimatedMaterialTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType     = typename AccessorType::KeyType;
	using StorageType = FMovieScenePreAnimatedMaterialParameters;

	static_assert(THasAddReferencedObjectForComponent<StorageType>::Value, "StorageType is not correctly exposed to the reference graph!");

	static FMovieScenePreAnimatedMaterialParameters CachePreAnimatedValue(typename TCallTraits<RequiredComponents>::ParamType... InRequiredComponents)
	{
		AccessorType Accessor{ InRequiredComponents... };

		FMovieScenePreAnimatedMaterialParameters Parameters;

		if (Accessor)
		{
			Parameters.SetMaterial(Accessor.GetMaterial());
		}

		return Parameters;
	}

	static void RestorePreAnimatedValue(const KeyType& InKey, const FMovieScenePreAnimatedMaterialParameters& PreAnimatedValue, const FRestoreStateParams& Params)
	{
		AccessorType Accessor{ InKey };
		if (Accessor)
		{
			if (UMaterialInterface* PreviousMaterial = PreAnimatedValue.GetMaterial())
			{
				Accessor.SetMaterial(PreAnimatedValue.GetMaterial());
			}
		}
	}
};

template<typename AccessorType, typename... RequiredComponents>
using TPreAnimatedMaterialParameterTraits = TPreAnimatedMaterialTraits<AccessorType, RequiredComponents...>;

template<typename AccessorType, typename... RequiredComponents>
class TMovieSceneMaterialSystem
{
public:

	using MaterialSwitcherStorageType = TPreAnimatedStateStorage<TPreAnimatedMaterialTraits<AccessorType, RequiredComponents...>>;
	using MaterialParameterStorageType = TPreAnimatedStateStorage<TPreAnimatedMaterialParameterTraits<AccessorType, RequiredComponents...>>;

	TSharedPtr<MaterialSwitcherStorageType> MaterialSwitcherStorage;
	TSharedPtr<MaterialParameterStorageType> MaterialParameterStorage;

	void OnLink(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<RequiredComponents>... InRequiredComponents);
	void OnUnlink(UMovieSceneEntitySystemLinker* Linker);
	void OnRun(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<RequiredComponents>... InRequiredComponents, FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents);

	void SavePreAnimatedState(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<RequiredComponents>... InRequiredComponents, const IMovieScenePreAnimatedStateSystemInterface::FPreAnimationParameters& InParameters);

	void OnPostSpawn(UMovieSceneEntitySystemLinker* InLinker, TComponentTypeID<RequiredComponents>... InRequiredComponents);

protected:

	struct FMaterialGroupingPolicy
	{
		using GroupKeyType = TTuple<RequiredComponents..., FMaterialParameterInfo>;

		void InitializeGroupKeys(
			TEntityGroupingHandlerBase<FMaterialGroupingPolicy>& Handler,
			FEntityGroupBuilder* Builder,
			FEntityAllocationIteratorItem Item,
			FReadEntityIDs EntityIDs,
			TWrite<FEntityGroupID> GroupIDs,
			TRead<RequiredComponents>... Components)
		{
			const FComponentMask& AllocationType = Item.GetAllocationType();

			FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

			TComponentTypeID<FName> ParameterName;
			if (AllocationType.Contains(TracksComponents->ScalarParameterName))
			{
				ParameterName = TracksComponents->ScalarParameterName;
			}
			else if (AllocationType.Contains(TracksComponents->VectorParameterName))
			{
				ParameterName = TracksComponents->VectorParameterName;
			}
			else if (AllocationType.Contains(TracksComponents->ColorParameterName))
			{
				ParameterName = TracksComponents->ColorParameterName;
			}

			TComponentTypeID<FMaterialParameterInfo> ParameterInfo;
			if (AllocationType.Contains(TracksComponents->ScalarMaterialParameterInfo))
			{
				ParameterInfo = TracksComponents->ScalarMaterialParameterInfo;
			}
			else if (AllocationType.Contains(TracksComponents->VectorMaterialParameterInfo))
			{
				ParameterInfo = TracksComponents->VectorMaterialParameterInfo;
			}
			else if (AllocationType.Contains(TracksComponents->ColorMaterialParameterInfo))
			{
				ParameterInfo = TracksComponents->ColorMaterialParameterInfo;
			}


			const FEntityAllocation* Allocation = Item.GetAllocation();
			const int32              Num        = Allocation->Num();

			TReadOptional<FName>                  ParameterNames = ParameterName ? Allocation->TryReadComponents(ParameterName) : TReadOptional<FName>();
			TReadOptional<FMaterialParameterInfo> ParameterInfos = ParameterInfo ? Allocation->TryReadComponents(ParameterInfo) : TReadOptional<FMaterialParameterInfo>();

			for (int32 Index = 0; Index < Num; ++Index)
			{
				GroupKeyType Key = ParameterInfos
					? MakeTuple(Components[Index]..., ParameterInfos[Index])
					: ParameterNames 
						? MakeTuple(Components[Index]..., FMaterialParameterInfo(ParameterNames[Index]))
						: MakeTuple(Components[Index]..., FMaterialParameterInfo());

				const int32    NewGroupIndex = Handler.GetOrAllocateGroupIndex(Key, Builder);
				FEntityGroupID NewGroupID    = Builder->MakeGroupID(NewGroupIndex);

				Builder->AddEntityToGroup(EntityIDs[Index], NewGroupID);
				// Write out the group ID component
				GroupIDs[Index] = NewGroupID;
			}
		}

	#if WITH_EDITOR
		bool OnObjectsReplaced(GroupKeyType& InOutKey, const TMap<UObject*, UObject*>& ReplacementMap)
		{
			return false;
		}
	#endif
	};

	UE::MovieScene::FEntityGroupingPolicyKey GroupingKey;

	FEntityComponentFilter MaterialSwitcherFilter;
	FEntityComponentFilter MaterialParameterFilter;

	FCachedEntityFilterResult_Allocations ReinitializeBoundMaterials;
};

template<typename AccessorType, typename... RequiredComponents>
struct TApplyMaterialSwitchers
{
	static void ForEachEntity(typename TCallTraits<RequiredComponents>::ParamType... Inputs, const FObjectComponent& ObjectResult)
	{
		// ObjectResult must be a material
		UMaterialInterface* NewMaterial = Cast<UMaterialInterface>(ObjectResult.GetObject());

		AccessorType Accessor(Inputs...);
		if (!Accessor)
		{
			return;
		}

		UMaterialInterface*        ExistingMaterial = Accessor.GetMaterial();
		UMaterialInstanceDynamic*  ExistingMID      = Cast<UMaterialInstanceDynamic>(ExistingMaterial);

		if (ExistingMID && ExistingMID->Parent && ExistingMID->Parent == NewMaterial)
		{
			// Do not re-assign materials when a dynamic instance is already assigned with the same parent (since that's basically the same material, just with animated parameters)
			// This is required for supporting material switchers alongside parameter tracks
			return;
		}

		Accessor.SetMaterial(NewMaterial);
	}
};

template<typename AccessorType, typename... RequiredComponents>
struct TInitializeBoundMaterials
{
	static bool InitializeBoundMaterial(typename TCallTraits<RequiredComponents>::ParamType... Inputs, FObjectComponent& OutDynamicMaterial)
	{
		AccessorType Accessor(Inputs...);
		if (!Accessor)
		{
			OutDynamicMaterial = FObjectComponent::Null();
			return false;
		}

		UMaterialInterface* ExistingMaterial = Accessor.GetMaterial();

		if (!ExistingMaterial)
		{
			// If the object was not previously explicitly null, make it so
			if (OutDynamicMaterial != FObjectComponent::Null())
			{
				OutDynamicMaterial = FObjectComponent::Null();
				return true;
			}
			return false;
		}

		if (UMaterialInstanceDynamic* MID = Cast<UMaterialInstanceDynamic>(ExistingMaterial))
		{
			if (OutDynamicMaterial != MID)
			{
				OutDynamicMaterial = FObjectComponent::Weak(MID);
				return true;
			}
			return false;
		}

		UMaterialInstanceDynamic* CurrentMID = Cast<UMaterialInstanceDynamic>(OutDynamicMaterial.GetObject());
		if (CurrentMID && CurrentMID->Parent == ExistingMaterial)
		{
			Accessor.SetMaterial(CurrentMID);
			return false;
		}
		
		UMaterialInstanceDynamic* NewMaterial = Accessor.CreateDynamicMaterial(ExistingMaterial);
		OutDynamicMaterial = FObjectComponent::Weak(NewMaterial);
		Accessor.SetMaterial(NewMaterial);
		return true;
	}

	static void ForEachEntity(typename TCallTraits<RequiredComponents>::ParamType... Inputs, FObjectComponent& OutDynamicMaterial)
	{
		InitializeBoundMaterial(Inputs..., OutDynamicMaterial);
	}
};


template<typename AccessorType, typename... RequiredComponents>
struct TReinitializeBoundMaterials
{
	UMovieSceneEntitySystemLinker* Linker;
	TArray<FMovieSceneEntityID> ReboundMaterials;

	TReinitializeBoundMaterials(UMovieSceneEntitySystemLinker* InLinker)
		: Linker(InLinker)
	{}

	void ForEachAllocation(int32 Num, const FMovieSceneEntityID* EntityIDs, const RequiredComponents*... Inputs, FObjectComponent* Objects)
	{
		for (int32 Index = 0; Index < Num; ++Index)
		{
			ForEachEntity(EntityIDs[Index], Inputs[Index]..., Objects[Index]);
		}
	}

	void ForEachEntity(FMovieSceneEntityID EntityID, typename TCallTraits<RequiredComponents>::ParamType... Inputs, FObjectComponent& OutDynamicMaterial)
	{
		if (TInitializeBoundMaterials<AccessorType, RequiredComponents...>::InitializeBoundMaterial(Inputs..., OutDynamicMaterial))
		{
			ReboundMaterials.Add(EntityID);
		}
	}

	void PostTask()
	{
		FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();
		for (FMovieSceneEntityID EntityID : ReboundMaterials)
		{
			Linker->EntityManager.AddComponent(EntityID, TracksComponents->Tags.BoundMaterialChanged);
		}
	}
};

template<typename...>
struct TAddBoundMaterialMutationImpl;

template<typename AccessorType, typename... RequiredComponents, int... Indices>
struct TAddBoundMaterialMutationImpl<AccessorType, TIntegerSequence<int, Indices...>, RequiredComponents...> : IMovieSceneEntityMutation
{
	TAddBoundMaterialMutationImpl(TComponentTypeID<RequiredComponents>... InRequiredComponents)
		: ComponentTypes(InRequiredComponents...)
	{
		BuiltInComponents = FBuiltInComponentTypes::Get();
		TracksComponents  = FMovieSceneTracksComponentTypes::Get();
	}
	virtual void CreateMutation(FEntityManager* EntityManager, FComponentMask* InOutEntityComponentTypes) const override
	{
		InOutEntityComponentTypes->Set(TracksComponents->BoundMaterial);
	}
	virtual void InitializeAllocation(FEntityAllocation* Allocation, const FComponentMask& AllocationType) const
	{
		TComponentWriter<FObjectComponent> BoundMaterials = Allocation->WriteComponents(TracksComponents->BoundMaterial, FEntityAllocationWriteContext::NewAllocation());
		InitializeAllocation(Allocation, BoundMaterials, Allocation->ReadComponents(ComponentTypes.template Get<Indices>())...);
	}

	void InitializeAllocation(FEntityAllocation* Allocation, FObjectComponent* OutBoundMaterials, const RequiredComponents*... InRequiredComponents) const
	{
		const int32 Num = Allocation->Num();
		for (int32 Index = 0; Index < Num; ++Index)
		{
			OutBoundMaterials[Index] = FObjectComponent::Null();
		}
	}
private:

	FBuiltInComponentTypes* BuiltInComponents;
	FMovieSceneTracksComponentTypes* TracksComponents;

	TTuple<TComponentTypeID<RequiredComponents>...> ComponentTypes;
};


template<typename AccessorType, typename... RequiredComponents>
void TMovieSceneMaterialSystem<AccessorType, RequiredComponents...>::OnLink(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<RequiredComponents>... InRequiredComponents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	// Define a grouping for these materials. This will make hierarchical bias work.
	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->LinkSystem<UMovieSceneEntityGroupingSystem>();
	GroupingKey = GroupingSystem->AddGrouping(FMaterialGroupingPolicy(), InRequiredComponents...);

	MaterialSwitcherFilter.Reset();
	MaterialSwitcherFilter.All({ InRequiredComponents..., BuiltInComponents->ObjectResult });

	// Currently the only supported entities that we initialize are ones that contain Scalar, Vector or Color parameters
	// Imported entities are implicitly excluded by way of filtering by BoundObject, which do not exist on imported entities
	MaterialParameterFilter.Reset();
	MaterialParameterFilter.All({ InRequiredComponents... });
	MaterialParameterFilter.Any({ TracksComponents->ScalarParameterName, TracksComponents->ColorParameterName, TracksComponents->VectorParameterName, // Old style parameter types for deprecated UMovieSceneParameterSections
		TracksComponents->ScalarMaterialParameterInfo, TracksComponents->ColorMaterialParameterInfo, TracksComponents->VectorMaterialParameterInfo }); // New style parameter types

	Linker->Events.PostSpawnEvent.AddRaw(this, &TMovieSceneMaterialSystem<AccessorType, RequiredComponents...>::OnPostSpawn, InRequiredComponents...);

	ReinitializeBoundMaterials.Filter.All({ TracksComponents->BoundMaterial, InRequiredComponents... });
	ReinitializeBoundMaterials.Filter.None({ BuiltInComponents->Tags.NeedsUnlink });
}

template<typename AccessorType, typename... RequiredComponents>
void TMovieSceneMaterialSystem<AccessorType, RequiredComponents...>::OnUnlink(UMovieSceneEntitySystemLinker* Linker)
{
	UMovieSceneEntityGroupingSystem* GroupingSystem = Linker->FindSystem<UMovieSceneEntityGroupingSystem>();
	if (ensure(GroupingSystem))
	{
		GroupingSystem->RemoveGrouping(GroupingKey);
	}
	GroupingKey = FEntityGroupingPolicyKey();

	Linker->Events.PostSpawnEvent.RemoveAll(this);
}

template<typename AccessorType, typename... RequiredComponents>
void TMovieSceneMaterialSystem<AccessorType, RequiredComponents...>::OnPostSpawn(UMovieSceneEntitySystemLinker* InLinker, TComponentTypeID<RequiredComponents>... InRequiredComponents)
{
	using namespace UE::MovieScene;

	SCOPE_CYCLE_COUNTER(MovieSceneEval_ReinitializeBoundMaterials)

	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	FEntityAllocationWriteContext WriteContext(InLinker->EntityManager);
	TReinitializeBoundMaterials<AccessorType, RequiredComponents...> ReinitializeBoundMaterialsTask(InLinker);

	// Reinitialize bound dynamic materials, adding NeedsLink during PostTask to any that changed
	// This will cause the instantiation phase to be re-run for these entities (and any other new or expired ones)
	for (FEntityAllocation* Allocation : ReinitializeBoundMaterials.GetMatchingAllocations(InLinker->EntityManager))
	{
		FEntityAllocationMutexGuard LockGuard(Allocation, EComponentHeaderLockMode::LockFree);

		const int32 Num = Allocation->Num();
		ReinitializeBoundMaterialsTask.ForEachAllocation(Num, Allocation->GetRawEntityIDs(), Allocation->ReadComponents(InRequiredComponents)..., Allocation->WriteComponents(TracksComponents->BoundMaterial, WriteContext));
	}

	ReinitializeBoundMaterialsTask.PostTask();
}

template<typename AccessorType, typename... RequiredComponents>
void TMovieSceneMaterialSystem<AccessorType, RequiredComponents...>::OnRun(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<RequiredComponents>... InRequiredComponents, FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents)
{
	using namespace UE::MovieScene;

	FBuiltInComponentTypes*          BuiltInComponents = FBuiltInComponentTypes::Get();
	FMovieSceneTracksComponentTypes* TracksComponents  = FMovieSceneTracksComponentTypes::Get();

	TSharedRef<FMovieSceneEntitySystemRunner> Runner = Linker->GetRunner();

	ESystemPhase CurrentPhase = Runner->GetCurrentPhase();
	if (CurrentPhase == ESystemPhase::Instantiation)
	{
		// --------------------------------------------------------------------------------------
		// Apply material switchers
		TApplyMaterialSwitchers<AccessorType, RequiredComponents...> ApplyMaterialSwitchers;

		FEntityTaskBuilder()
		.ReadAllOf(InRequiredComponents...)
		.Read(BuiltInComponents->ObjectResult)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.FilterNone({ BuiltInComponents->Tags.Ignored })
		.RunInline_PerEntity(&Linker->EntityManager, ApplyMaterialSwitchers);

		// --------------------------------------------------------------------------------------
		// Add bound materials for any NeedsLink entities that have material parameters
		FEntityComponentFilter Filter(MaterialParameterFilter);
		Filter.All({ BuiltInComponents->Tags.NeedsLink });
		Filter.None({ BuiltInComponents->Tags.ImportedEntity });

		using MutationType = TAddBoundMaterialMutationImpl<AccessorType, TMakeIntegerSequence<int, sizeof...(RequiredComponents)>, RequiredComponents...>;
		Linker->EntityManager.MutateAll(Filter, MutationType(InRequiredComponents...));

		// --------------------------------------------------------------------------------------
		// (Re)initialize bound materials for any NeedsLink materials
		TReinitializeBoundMaterials<AccessorType, RequiredComponents...> ReinitializeBoundMaterialsTask(Linker);

		FEntityTaskBuilder()
		.ReadEntityIDs()
		.ReadAllOf(InRequiredComponents...)
		.Write(TracksComponents->BoundMaterial)
		.FilterAll({ BuiltInComponents->Tags.NeedsLink })
		.FilterNone({ BuiltInComponents->Tags.Ignored, BuiltInComponents->Tags.NeedsUnlink })
		.RunInline_PerEntity(&Linker->EntityManager, ReinitializeBoundMaterialsTask);
	}
}

template<typename AccessorType, typename... RequiredComponents>
void TMovieSceneMaterialSystem<AccessorType, RequiredComponents...>::SavePreAnimatedState(UMovieSceneEntitySystemLinker* Linker, TComponentTypeID<RequiredComponents>... InRequiredComponents, const IMovieScenePreAnimatedStateSystemInterface::FPreAnimationParameters& InParameters)
{
	using namespace UE::MovieScene;

	// If we have material results to apply save those as well
	if (Linker->EntityManager.Contains(MaterialSwitcherFilter))
	{
		TPreAnimatedStateTaskParams<RequiredComponents...> Params;
		Params.AdditionalFilter = MaterialSwitcherFilter;

		MaterialSwitcherStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, InRequiredComponents...);
	}

	// If we have bound materials to resolve save the current material
	if (Linker->EntityManager.Contains(MaterialParameterFilter))
	{
		TPreAnimatedStateTaskParams<RequiredComponents...> Params;
		Params.AdditionalFilter = MaterialParameterFilter;

		MaterialParameterStorage->BeginTrackingAndCachePreAnimatedValuesTask(Linker, Params, InRequiredComponents...);
	}
}

} // namespace UE::MovieScene
