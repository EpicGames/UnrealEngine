// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Math/Transform.h"
#include "Containers/Map.h"
#include "UObject/ObjectKey.h"
#include "UObject/NameTypes.h"

#include "MovieSceneMotionVectorSimulationSystem.generated.h"

struct FMovieSceneAnimTypeID;

namespace UE
{
namespace MovieScene
{


FFrameTime GetSimulatedMotionVectorTime(const FMovieSceneContext& Context);


} // namespace MovieScene
} // namespace UE

UCLASS(MinimalAPI)
class UMovieSceneMotionVectorSimulationSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneMotionVectorSimulationSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API void EnableThisFrame();
	MOVIESCENETRACKS_API void SimulateAllTransforms();
	bool IsSimulationEnabled() const
	{
		return bSimulationEnabled;
	}

	MOVIESCENETRACKS_API void PreserveSimulatedMotion(bool bShouldPreserveTransforms);

	MOVIESCENETRACKS_API void AddSimulatedTransform(USceneComponent* Component, const FTransform& SimulatedTransform, FName SocketName);

private:

	MOVIESCENETRACKS_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

	MOVIESCENETRACKS_API void OnPostEvaluation();

	MOVIESCENETRACKS_API void ComputeSimulatedMotion();

private:

	MOVIESCENETRACKS_API void PropagateMotionToComponents();

	MOVIESCENETRACKS_API FTransform GetTransform(USceneComponent* Component);

	MOVIESCENETRACKS_API FTransform GetSocketTransform(USceneComponent* Component, FName SocketName);

	MOVIESCENETRACKS_API bool HavePreviousTransformForParent(USceneComponent* InComponent) const;

	MOVIESCENETRACKS_API void ApplySimulatedTransforms(USceneComponent* InComponent, const FTransform& InPreviousTransform);

private:

	/** Transform data relating to a specific object or socket */
	struct FSimulatedTransform
	{
		FSimulatedTransform(const FTransform& InTransform, FName InSocketName = NAME_None)
			: Transform(InTransform), SocketName(InSocketName)
		{}

		/** The transform for the object */
		FTransform Transform;

		/** The socket name to which this relates */
		FName SocketName;
	};

	/** Map of object -> previous transform data */
	TMultiMap<FObjectKey, FSimulatedTransform> TransformData;

	/** Whether to reset TransformData at the end of the frame or not */
	bool bPreserveTransforms = false;

	bool bSimulationEnabled = false;

	bool bSimulateTransformsRequested = false;
};



