// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Systems/MovieScenePropertySystem.h"
#include "MovieSceneObjectPropertySystem.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneObjectPropertySystem : public UMovieScenePropertySystem
{
	GENERATED_BODY()

	UMovieSceneObjectPropertySystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};
