// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Systems/MovieSceneBlenderSystemHelper.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieScenePiecewiseBoolBlenderSystem.generated.h"

class UObject;


UCLASS(MinimalAPI)
class UMovieScenePiecewiseBoolBlenderSystem : public UMovieSceneBlenderSystem
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScenePiecewiseBoolBlenderSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:

	UE::MovieScene::TSimpleBlenderSystemImpl<bool> Impl;
};

