// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "HAL/Platform.h"
#include "Systems/MovieSceneBlenderSystemHelper.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieScenePiecewiseByteBlenderSystem.generated.h"

class UObject;


UCLASS(MinimalAPI)
class UMovieScenePiecewiseByteBlenderSystem : public UMovieSceneBlenderSystem
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScenePiecewiseByteBlenderSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:

	UE::MovieScene::TSimpleBlenderSystemImpl<uint8> Impl;
};

