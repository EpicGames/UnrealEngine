// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SortedMap.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "EntitySystem/MovieSceneEntitySystem.h"

#include "MovieScenePiecewiseDoubleBlenderSystem.generated.h"

namespace UE
{
namespace MovieScene
{

/** Blend result struct that stores the cumulative sum of pre-weighted values, alongside the total weight */
struct FBlendResult
{
	/** Cumulative sum of blend values pre-multiplied with each value's weight. */
	double Total = 0.f;
	/** Cumulative sum of weights. */
	float Weight = 0.f;
};

/* Data for each additive/override section, we don't accumulate until we recieve all of the data*/
struct FAdditveAndOverrideData
{
	/** If Additive, if not then override*/
	bool bIsAdditive;
	/** Section Value*/
	double Value;
	/** Section Weight*/
	float Weight;
	/** Section Blending Order*/
	int32 BlendingOrder;
	bool operator<(const FAdditveAndOverrideData& RHS) const
	{
		return BlendingOrder < RHS.BlendingOrder;
	}
};
/** Structure for holding the sorted blending order data for the additive and override data*/
struct FAdditiveAndOverrides
{
	mutable TSortedMap<int32, FAdditveAndOverrideData> Data;
};

/** Structure for holding the blend results of each blend type */
struct FAccumulationResult
{
	const FBlendResult* Absolutes = nullptr;
	const FBlendResult* Relatives = nullptr;
	const FBlendResult* Additives = nullptr;
	const FBlendResult* AdditivesFromBase = nullptr;
	const FAdditiveAndOverrides* Overrides = nullptr;
	bool IsValid() const
	{
		return Absolutes || Relatives || Additives || AdditivesFromBase
			|| Overrides;
	}

	FBlendResult GetAbsoluteResult(uint16 BlendID) const
	{
		return Absolutes ? Absolutes[BlendID] : FBlendResult{};
	}
	FBlendResult GetRelativeResult(uint16 BlendID) const
	{
		return Relatives ? Relatives[BlendID] : FBlendResult{};
	}
	FBlendResult GetAdditiveResult(uint16 BlendID) const
	{
		return Additives ? Additives[BlendID] : FBlendResult{};
	}
	FBlendResult GetAdditiveFromBaseResult(uint16 BlendID) const
	{
		return AdditivesFromBase ? AdditivesFromBase[BlendID] : FBlendResult{};
	}
	FAdditiveAndOverrides GetAdditiveAndOverrideResult(uint16 BlendID)  const
	{
		return Overrides? Overrides[BlendID] : FAdditiveAndOverrides{};
	}
};

/** Bufffer used for additive/override values*/
struct FAdditiveAndOverrideBuffer
{
	TArray<FAdditiveAndOverrides> Values;
};

/** Buffer used for accumulating additive-from-base values */
struct FAdditiveFromBaseBuffer
{
	TArray<FBlendResult> Buffer;
	TComponentTypeID<double> BaseComponent;
};

/** Struct that maintains accumulation buffers for each blend type, one buffer per float result component type */
struct FAccumulationBuffers
{
	bool IsEmpty() const;

	void Reset();

	FAccumulationResult FindResults(FComponentTypeID InComponentType) const;

	/** Map from value result component type -> Absolute blend accumulation buffer for that channel type */
	TSortedMap<FComponentTypeID, TArray<FBlendResult>> Absolute;
	/** Map from value result component type -> Relative blend accumulation buffer for that channel type */
	TSortedMap<FComponentTypeID, TArray<FBlendResult>> Relative;
	/** Map from value result component type -> Additive blend accumulation buffer for that channel type */
	TSortedMap<FComponentTypeID, TArray<FBlendResult>> Additive;
	/** Map from value result component type -> Additive From Base blend accumulation buffer for that channel type */
	TSortedMap<FComponentTypeID, FAdditiveFromBaseBuffer> AdditiveFromBase;
	/** Map from value result component type -> Additive and Override blending order blneds for that channel type */
	TSortedMap<FComponentTypeID, FAdditiveAndOverrideBuffer> AdditiveAndOverrides;

};

} // namespace MovieScene
} // namespace UE

UCLASS(DisplayName="Weighted per-channel", meta=(Tooltip="Blends each channel of this object's transform as separate scalar components. Useful for blending to/from over-rotated objects (ie, 0 < rotation > 360."), MinimalAPI)
class UMovieScenePiecewiseDoubleBlenderSystem : public UMovieSceneBlenderSystem, public IMovieSceneValueDecomposer
{
public:
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScenePiecewiseDoubleBlenderSystem(const FObjectInitializer& ObjInit);

	using FMovieSceneEntityID  = UE::MovieScene::FMovieSceneEntityID;
	using FComponentTypeID     = UE::MovieScene::FComponentTypeID;

	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	MOVIESCENETRACKS_API virtual FGraphEventRef DispatchDecomposeTask(const UE::MovieScene::FValueDecompositionParams& Params, UE::MovieScene::FAlignedDecomposedValue* Output) override;

private:

	void ReinitializeAccumulationBuffers();
	void ZeroAccumulationBuffers();

	/** Buffers that contain accumulated blend values, separated by blend type */
	UE::MovieScene::FAccumulationBuffers AccumulationBuffers;

	/** Mask that contains value result components that have BlendChannelInput components */
	FComponentMask BlendedResultMask;

	/** Mask that contains property tags for any property type that has has at least one BlendChannelOutput */
	FComponentMask BlendedPropertyMask;

	/** Cache state that is used to invalidate and reset the accumulation buffers when the entity manager has structurally changed */
	UE::MovieScene::FCachedEntityManagerState ChannelRelevancyCache;

	/** Bit array specifying FCompositePropertyTypeID's for properties contained within BlendedPropertyMask */
	TBitArray<> CachedRelevantProperties;

	/** Whether the current entity manager contains any non-property based blends */
	bool bContainsNonPropertyBlends = false;
};

