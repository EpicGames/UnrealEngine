// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "HAL/Platform.h"
#include "Systems/MovieSceneBlenderSystemHelper.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieScenePiecewiseEnumBlenderSystem.generated.h"

class UObject;

namespace UE
{
namespace MovieScene
{

/** 
 * Custom blend result traits for enums:
 *
 * - We don't blend enums, unlike bytes, because we don't even know if the in-between values are valid
 *   enumerations.
 * - Number of contributors doesn't matter. The last one wins.
 */
struct FSimpleBlenderEnumResultTraits
{
	static void ZeroAccumulationBuffer(TArrayView<TSimpleBlendResult<uint8>> Buffer)
	{
		FMemory::Memzero(Buffer.GetData(), sizeof(TSimpleBlendResult<uint8>) * Buffer.Num());
	}

	static void AccumulateResult(TSimpleBlendResult<uint8>& InOutValue, uint8 Contributor)
	{
		InOutValue.Value = Contributor;
		++InOutValue.NumContributors;
	}

	static uint8 BlendResult(const TSimpleBlendResult<uint8>& InResult)
	{
		return InResult.Value;
	}
};

} // namespace MovieScene
} // namespace UE

UCLASS(MinimalAPI)
class UMovieScenePiecewiseEnumBlenderSystem : public UMovieSceneBlenderSystem
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScenePiecewiseEnumBlenderSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:

	UE::MovieScene::TSimpleBlenderSystemImpl<uint8, UE::MovieScene::FSimpleBlenderEnumResultTraits> Impl;
};

