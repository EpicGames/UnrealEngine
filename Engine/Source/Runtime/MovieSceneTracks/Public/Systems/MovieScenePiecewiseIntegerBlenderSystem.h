// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "MovieScenePiecewiseIntegerBlenderSystem.generated.h"

namespace UE
{
namespace MovieScene
{

	/** Blend result struct that stores the cumulative sum of pre-weighted values, alongside the total weight */
	struct FIntegerBlendResult
	{
		/** Cumulative sum of blend values pre-multiplied with each value's weight. */
		int64 Total = 0;
		/** Cumulative sum of weights. */
		double Weight = 0.f;
	};

	/** Struct that maintains accumulation buffers for each blend type */
	struct FIntegerAccumulationBuffers
	{
		bool IsEmpty() const;

		void Reset();

		TArray<FIntegerBlendResult> Absolute;
		TArray<FIntegerBlendResult> Relative;
		TArray<FIntegerBlendResult> Additive;
		TArray<FIntegerBlendResult> AdditiveFromBase;
	};

} // namespace MovieScene
} // namespace UE


UCLASS(MinimalAPI)
class UMovieScenePiecewiseIntegerBlenderSystem : public UMovieSceneBlenderSystem
{
public:
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScenePiecewiseIntegerBlenderSystem(const FObjectInitializer& ObjInit);

	using FMovieSceneEntityID  = UE::MovieScene::FMovieSceneEntityID;
	using FComponentTypeID     = UE::MovieScene::FComponentTypeID;

	MOVIESCENETRACKS_API virtual void OnLink() override;
	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:

	void ReinitializeAccumulationBuffers();
	void ZeroAccumulationBuffers();

private:

	UE::MovieScene::FIntegerAccumulationBuffers AccumulationBuffers;

	UE::MovieScene::FCachedEntityManagerState ChannelRelevancyCache;
};

