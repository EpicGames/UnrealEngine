// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "UObject/ObjectKey.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedObjectStorage.h"
#include "Evaluation/PreAnimatedState/MovieScenePreAnimatedStateStorage.h"
#include "Materials/MaterialParameters.h"

namespace UE::MovieScene
{

struct FMaterialParameterKey
{
	FObjectKey BoundMaterial;
	FMaterialParameterInfo ParameterInfo;

	MOVIESCENETRACKS_API FMaterialParameterKey(const FObjectComponent& InBoundMaterial, const FName& InParameterName);
	MOVIESCENETRACKS_API FMaterialParameterKey(const FObjectComponent& InBoundMaterial, const FMaterialParameterInfo& InParameterInfo);

	friend MOVIESCENETRACKS_API uint32 GetTypeHash(const FMaterialParameterKey& InKey);

	friend MOVIESCENETRACKS_API bool operator==(const FMaterialParameterKey& A, const FMaterialParameterKey& B);
};

struct FMaterialParameterCollectionScalarTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType = FMaterialParameterKey;
	using StorageType = float;

	static MOVIESCENETRACKS_API void ReplaceObject(FMaterialParameterKey& InOutKey, const FObjectKey& NewObject);

	static MOVIESCENETRACKS_API float CachePreAnimatedValue(const FObjectComponent& InBoundMaterial, const FName& ParameterName);
	static MOVIESCENETRACKS_API float CachePreAnimatedValue(const FObjectComponent& InBoundMaterial, const FMaterialParameterInfo& ParameterInfo);

	static MOVIESCENETRACKS_API void RestorePreAnimatedValue(const FMaterialParameterKey& InKey, float OldValue, const FRestoreStateParams& Params);
};

struct FMaterialParameterCollectionVectorTraits : FBoundObjectPreAnimatedStateTraits
{
	using KeyType = FMaterialParameterKey;
	using StorageType = FLinearColor;

	static MOVIESCENETRACKS_API void ReplaceObject(FMaterialParameterKey& InOutKey, const FObjectKey& NewObject);

	static MOVIESCENETRACKS_API FLinearColor CachePreAnimatedValue(const FObjectComponent& InBoundMaterial, const FName& ParameterName);
	static MOVIESCENETRACKS_API FLinearColor CachePreAnimatedValue(const FObjectComponent& InBoundMaterial, const FMaterialParameterInfo& ParameterInfo);

	static MOVIESCENETRACKS_API void RestorePreAnimatedValue(const FMaterialParameterKey& InKey, const FLinearColor& OldValue, const FRestoreStateParams& Params);
};

struct FPreAnimatedScalarMaterialParameterStorage
	: public TPreAnimatedStateStorage<FMaterialParameterCollectionScalarTraits>
{
	static MOVIESCENETRACKS_API TAutoRegisterPreAnimatedStorageID<FPreAnimatedScalarMaterialParameterStorage> StorageID;
};

struct FPreAnimatedVectorMaterialParameterStorage
	: public TPreAnimatedStateStorage<FMaterialParameterCollectionVectorTraits>
{
	static MOVIESCENETRACKS_API TAutoRegisterPreAnimatedStorageID<FPreAnimatedVectorMaterialParameterStorage> StorageID;
};

} // namespace UE::MovieScene