// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SparseArray.h"
#include "Containers/Array.h"
#include "Delegates/DelegateCombinations.h"
#include "Kismet/BlueprintAsyncActionBase.h"
#include "Misc/FrameTime.h"

#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/Interrogation/MovieSceneInterrogationLinker.h"

#include "MovieScenePredictionSystem.generated.h"

struct FGuid;
struct FMovieSceneEntityComponentField;

class USceneComponent;
class UMovieSceneSequencePlayer;
class UMovieSceneEntitySystemLinker;

namespace UE
{
namespace MovieScene
{

struct FEntityImportSequenceParams;

} // namespace MovieScene
} // namespace UE


DECLARE_DYNAMIC_MULTICAST_DELEGATE_OneParam(FMovieSceneActorPredictionResult, FTransform, PredictedTransform);
DECLARE_DYNAMIC_MULTICAST_DELEGATE(FMovieSceneActorPredictionFailure);

/**
 * Async BP action that represents a pending prediction that is dispatched on a playing sequence.
 */
UCLASS(BlueprintType, meta=(ExposedAsyncProxy = "AsyncTask", HasDedicatedAsyncNode))
class UMovieSceneAsyncAction_SequencePrediction : public UBlueprintAsyncActionBase
{
	GENERATED_BODY()

public:

	/**
	 * Initiate an asynchronous prediction for the specified component's world transform at a specific time in a sequence
	 * Changes in attachment between the sequence's current time, and the predicted time are not accounted for
	 * Calling this function on a stopped sequence player is undefined.
	 *
	 * @param Player          An active, currently playing sequence player to use for predicting the transform
	 * @param TargetComponent The component to predict a world transform for
	 * @param TimeInSeconds   The time within the sequence to predict the transform at
	 * @return An asynchronous prediction object that contains Result and Failure delegates
	 */
	UFUNCTION(BlueprintCallable, Category=Cinematics)
	static UMovieSceneAsyncAction_SequencePrediction* PredictWorldTransformAtTime(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, float TimeInSeconds);


	/**
	 * Initiate an asynchronous prediction for the specified component's world transform at a specific time in a sequence
	 * Changes in attachment between the sequence's current time, and the predicted time are not accounted for
	 * Calling this function on a stopped sequence player is undefined.
	 *
	 * @param Player          An active, currently playing sequence player to use for predicting the transform
	 * @param TargetComponent The component to predict a world transform for
	 * @param FrameTime       The frame time to predict at in the sequence's display rate
	 * @return An asynchronous prediction object that contains Result and Failure delegates
	 */
	UFUNCTION(BlueprintCallable, Category=Cinematics)
	static UMovieSceneAsyncAction_SequencePrediction* PredictWorldTransformAtFrame(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, FFrameTime FrameTime);


	/**
	 * Initiate an asynchronous prediction for the specified component's local transform at a specific time in a sequence
	 * Changes in attachment between the sequence's current time, and the predicted time are not accounted for
	 * Calling this function on a stopped sequence player is undefined.
	 *
	 * @param Player          An active, currently playing sequence player to use for predicting the transform
	 * @param TargetComponent The component to predict a world transform for
	 * @param TimeInSeconds   The time within the sequence to predict the transform at
	 * @return An asynchronous prediction object that contains Result and Failure delegates
	 */
	UFUNCTION(BlueprintCallable, Category=Cinematics)
	static UMovieSceneAsyncAction_SequencePrediction* PredictLocalTransformAtTime(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, float TimeInSeconds);


	/**
	 * Initiate an asynchronous prediction for the specified component's local transform at a specific time in a sequence
	 * Changes in attachment between the sequence's current time, and the predicted time are not accounted for
	 * Calling this function on a stopped sequence player is undefined.
	 *
	 * @param Player          An active, currently playing sequence player to use for predicting the transform
	 * @param TargetComponent The component to predict a world transform for
	 * @param FrameTime       The frame time to predict at in the sequence's display rate
	 * @return An asynchronous prediction object that contains Result and Failure delegates
	 */
	UFUNCTION(BlueprintCallable, Category=Cinematics)
	static UMovieSceneAsyncAction_SequencePrediction* PredictLocalTransformAtFrame(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, FFrameTime FrameTime);

	/**
	 * Called during the instantiation phase to query the sequence for any transform track entities in the field
	 * and import the necessary entities into the Entity Manager.
	 *
	 * @param Channels Interrogation channel map for tracking channels for objects
	 */
	void ImportEntities(UE::MovieScene::FInterrogationChannels* Channels);


	/**
	 * Reset this prediction by marking all its entities for unlink
	 */
	void Reset(UMovieSceneEntitySystemLinker* Linker);


	/**
	 * Report the result for this prediction to any observers
	 *
	 * @param Channels    Interrogation channel map for retrieving channels for objects
	 * @param AllResults  The interrogation results for all predictions, in the space requested by this prediction
	 */
	void ReportResult(UE::MovieScene::FInterrogationChannels* Channels, const TSparseArray<TArray<FTransform>>& AllResults);
	void ReportResult(UE::MovieScene::FInterrogationChannels* Channels, const TSparseArray<TArray<UE::MovieScene::FIntermediate3DTransform>>& AllResults);

private:

	static UMovieSceneAsyncAction_SequencePrediction* MakePredictionImpl(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, float TimeInSeconds, bool bInWorldSpace);
	static UMovieSceneAsyncAction_SequencePrediction* MakePredictionImpl(UMovieSceneSequencePlayer* Player, USceneComponent* TargetComponent, FFrameTime TickResolutionTime, bool bInWorldSpace);

	int32 ImportTransformEntities(UMovieSceneEntitySystemLinker* Linker, const UE::MovieScene::FEntityImportSequenceParams& ImportParams, const FGuid& ObjectGuid, FFrameTime PredictedTime, const FMovieSceneEntityComponentField* ComponentField, const UE::MovieScene::FInterrogationKey& InterrogationKey);
	int32 ImportTransformEntities(UObject* PredicateObject, UObject* ObjectContext, const UE::MovieScene::FInterrogationKey& InterrogationKey);

	void ImportLocalTransforms(UE::MovieScene::FInterrogationChannels* Channels, USceneComponent* InSceneComponent);
	void ImportTransformHierarchy(UE::MovieScene::FInterrogationChannels* Channels, USceneComponent* InSceneComponent);

public:

	/** Called when a message is broadcast on the specified channel. Use GetPayload() to request the message payload. */
	UPROPERTY(BlueprintAssignable)
	FMovieSceneActorPredictionResult Result;

	/** Called when a message is broadcast on the specified channel. Use GetPayload() to request the message payload. */
	UPROPERTY(BlueprintAssignable)
	FMovieSceneActorPredictionFailure Failure;

private:

	friend class UMovieScenePredictionSystem;

	/** Cached array of all the entities created by this prediction */
	TArray<UE::MovieScene::FMovieSceneEntityID> ImportedEntities;

	/** The sequence player we're interrogating */
	UPROPERTY()
	TObjectPtr<UMovieSceneSequencePlayer> SequencePlayer;

	/** The target scene component we're interrogating */
	UPROPERTY()
	TObjectPtr<USceneComponent> SceneComponent;

	/** The time at which the interrogation should run */
	FFrameTime RootPredictedTime;

	/** The interrogation index used for any FInterrogationKeys that this prediction creates. Unique to this prediction */
	int32 InterrogationIndex;

	/** Whether we're capturing world space or local space transforms */
	bool bWorldSpace;
};


/**
 * System responsible for managing and reporting on pending UMovieSceneAsyncAction_SequencePrediction tasks
 */
UCLASS(MinimalAPI)
class UMovieScenePredictionSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieScenePredictionSystem(const FObjectInitializer& ObjInit);

	void AddPendingPrediction(UMovieSceneAsyncAction_SequencePrediction* Prediction);

	int32 MakeNewInterrogation(FFrameTime InTime);

private:

	bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	UE::MovieScene::FInterrogationChannels InterrogationChannels;

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneAsyncAction_SequencePrediction>> PendingPredictions;

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneAsyncAction_SequencePrediction>> ProcessingPredictions;
};
