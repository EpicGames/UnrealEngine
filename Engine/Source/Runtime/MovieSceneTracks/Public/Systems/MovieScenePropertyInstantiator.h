// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieScenePropertyComponentHandler.h"
#include "EntitySystem/MovieScenePropertyRegistry.h"

#include "Misc/TVariant.h"
#include "Misc/Optional.h"
#include "Containers/BitArray.h"
#include "Containers/Map.h"
#include "Containers/SortedMap.h"

#include "MovieScenePropertyInstantiator.generated.h"

namespace UE
{
namespace MovieScene
{

struct FPropertyStats;
struct FPropertyDefinition;
class FEntityManager;

} // namespace MovieScene
} // namespace UE

class UMovieSceneBlenderSystem;

/** Class responsible for resolving all property types registered with FBuiltInComponentTypes::PropertyRegistry */
UCLASS(MinimalAPI)
class UMovieScenePropertyInstantiatorSystem
	: public UMovieSceneEntityInstantiatorSystem
{
public:

	using FMovieSceneEntityID = UE::MovieScene::FMovieSceneEntityID;

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScenePropertyInstantiatorSystem(const FObjectInitializer& ObjInit);

	/**
	 * Retrieve the stats for a specific property
	 */
	MOVIESCENETRACKS_API UE::MovieScene::FPropertyStats GetStatsForProperty(UE::MovieScene::FCompositePropertyTypeID PropertyID) const;

	/**
	 * Recompose a value from the constituent parts specified in InQuery, taking into account the weightings of each channel.
	 * For instance, if a property comprises 3 additive values (a:1, b:2, c:3), and we recompose 'a' with an InCurrentValue of 10, the result for a would be 5.
	 * @note: ValueType must be either copy assignable to/from the storage type of the property, or must have a ConvertOperationalProperty overload
	 *
	 * @param InComponents   The components that define the property to decompose
	 * @param InQuery        The query defining the entities and object to recompose
	 * @param InCurrentValue The value of the property to recompose
	 * @return A result matching the storage type of the components, containing recomposed values for each of the entities specified in InQuery
	 */
	template<typename PropertyTraits, typename ValueType>
	UE::MovieScene::TRecompositionResult<ValueType> RecomposeBlend(const UE::MovieScene::TPropertyComponents<PropertyTraits>& InComponents, const UE::MovieScene::FDecompositionQuery& InQuery, const ValueType& InCurrentValue)
	{
		using namespace UE::MovieScene;

		using StorageType = typename PropertyTraits::StorageType;

		StorageType TmpStorage{};
		ConvertOperationalProperty(InCurrentValue, TmpStorage);

		UE::MovieScene::TRecompositionResult<StorageType> Result = RecomposeBlendOperational(InComponents, InQuery, TmpStorage);

		UE::MovieScene::TRecompositionResult<ValueType> ConversionResult(InCurrentValue, Result.Values.Num());
		for (int32 Index = 0; Index < Result.Values.Num(); ++Index)
		{
			ConvertOperationalProperty(Result.Values[Index], ConversionResult.Values[Index]);
		}
		return ConversionResult;
	}


	/**
	 * Recompose a value from the constituent parts specified in InQuery, taking into account the weightings of each channel.
	 * For instance, if a property comprises 3 additive values (a:1, b:2, c:3), and we recompose 'a' with an InCurrentValue of 10, the result for a would be 5.
	 *
	 * @param InComponents   The components that define the property to decompose
	 * @param InQuery        The query defining the entities and object to recompose
	 * @param InCurrentValue The value of the property to recompose
	 * @return A result matching the storage type of the components, containing recomposed values for each of the entities specified in InQuery
	 */
	template<typename PropertyTraits>
	UE::MovieScene::TRecompositionResult<typename PropertyTraits::StorageType> RecomposeBlendOperational(const UE::MovieScene::TPropertyComponents<PropertyTraits>& InComponents, const UE::MovieScene::FDecompositionQuery& InQuery, const typename PropertyTraits::StorageType& InCurrentValue);

	MOVIESCENETRACKS_API UE::MovieScene::FPropertyRecomposerPropertyInfo FindPropertyFromSource(FMovieSceneEntityID EntityID, UObject* Object) const;

private:

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	MOVIESCENETRACKS_API virtual void OnLink() override;
	MOVIESCENETRACKS_API virtual void OnUnlink() override;
	MOVIESCENETRACKS_API virtual void OnCleanTaggedGarbage() override;

private:

	void OnObjectsReplaced(const TMap<UObject*, UObject*>& ReplacementMap);

private:

	using FChannelMask     = TBitArray<TFixedAllocator< 1 >>;
	using FSlowPropertyPtr = TSharedPtr<FTrackInstancePropertyBindings>;

	struct FContributorKey
	{
		FContributorKey(int32 InPropertyIndex)
			: PropertyIndex(InPropertyIndex)
		{}

		int32 PropertyIndex;

		friend uint32 GetTypeHash(FContributorKey In)
		{
			return GetTypeHash(In.PropertyIndex);
		}

		friend bool operator==(FContributorKey A, FContributorKey B)
		{
			return A.PropertyIndex == B.PropertyIndex;
		}
	};

	struct FObjectPropertyInfo
	{
		FObjectPropertyInfo(UE::MovieScene::FResolvedProperty&& InProperty)
			: Property(MoveTemp(InProperty))
			, BoundObject(nullptr)
			, BlendChannel(FMovieSceneBlendChannelID::INVALID_BLEND_CHANNEL)
			, bWantsRestoreState( false)
			, bSupportsFastPath( true)
			, bNeedsInitialValue( false)
			, bMaxHBiasHasChanged(false)
			, bIsPartiallyAnimated(false)
		{}

		/** Variant of the property itself as either a pointer offset, a custom property index, or slow track instance bindings object */
		UE::MovieScene::FResolvedProperty Property;
		/** Pointer to the blender system to use for this property, if its blended */
		TWeakObjectPtr<UMovieSceneBlenderSystem> Blender;
		/** The object being animated */
		UObject* BoundObject;
		/** The path of the property being animated */
		FMovieScenePropertyBinding PropertyBinding;
		/** Mask of composite channels that are not animated (set bits indicate an unanimated channel) */
		FChannelMask EmptyChannels;
		/** The entity that contains the property component itself. Invalid for fast path properties. */
		UE::MovieScene::FMovieSceneEntityID FinalBlendOutputID;
		UE::MovieScene::FMovieSceneEntityID PreviousFastPathID;
		/** Final blend channel for this object. */
		uint16 BlendChannel;
		/** The index of this property within FPropertyRegistry::GetProperties. */
		int32 PropertyDefinitionIndex;
		/** Index of a float-based property if this property has been set for float-to-double conversion */
		TOptional<int32> ConvertedFromPropertyDefinitionIndex;
		int16 HBias = 0;
		uint8 bWantsRestoreState : 1;
		uint8 bSupportsFastPath : 1;
		uint8 bNeedsInitialValue : 1;
		uint8 bMaxHBiasHasChanged : 1;
		uint8 bIsPartiallyAnimated : 1;
	};

private:

	struct FBlenderSystemInfo
	{
		UClass* BlenderSystemClass = nullptr;
		UE::MovieScene::FComponentTypeID BlenderTypeTag;
	};

	struct FSetupBlenderSystemResult
	{
		FBlenderSystemInfo CurrentInfo;
		FBlenderSystemInfo PreviousInfo;
	};

	/* Parameter structure passed around when instantiating a specific instance of a property */
	struct FPropertyParameters
	{
		/** Pointer to the property instance to be animated */
		FObjectPropertyInfo* PropertyInfo;
		/** Pointer to the property type definition from FPropertyRegistry */
		const UE::MovieScene::FPropertyDefinition* PropertyDefinition;
		/** The index of the PropertyInfo member within UMovieScenePropertyInstantiatorSystem::ResolvedProperties */
		int32 PropertyInfoIndex;

		FContributorKey MakeContributorKey() const;

		void MakeOutputComponentType(
			const UE::MovieScene::FEntityManager& EntityManager,
			TArrayView<const UE::MovieScene::FPropertyCompositeDefinition> Composites,
			UE::MovieScene::FComponentMask& OutComponentType) const;
	};

	void DiscoverInvalidatedProperties(TBitArray<>& OutInvalidatedProperties);
	void DiscoverExpiredProperties(TBitArray<>& OutInvalidatedProperties);
	void DiscoverNewProperties(TBitArray<>& OutInvalidatedProperties);
	void UpgradeFloatToDoubleProperties(const TBitArray<>& InvalidatedProperties);
	void ProcessInvalidatedProperties(const TBitArray<>& InvalidatedProperties);
	void UpdatePropertyInfo(const FPropertyParameters& Params);
	void InitializeFastPath(const FPropertyParameters& Params);
	void InitializeBlendPath(const FPropertyParameters& Params);

	void DestroyStaleProperty(int32 PropertyIndex);
	void PostDestroyStaleProperties();

	FSetupBlenderSystemResult SetupBlenderSystem(const FPropertyParameters& Params);

	bool ResolveProperty(UE::MovieScene::FCustomAccessorView CustomAccessors, UObject* Object, const FMovieScenePropertyBinding& PropertyBinding, const UE::MovieScene::FEntityGroupID& GroupID, int32 PropertyDefinitionIndex);

	void InitializePropertyMetaData(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents);

private:

	static constexpr uint16 INVALID_BLEND_CHANNEL = uint16(-1);

	UE::MovieScene::FEntityGroupingPolicyKey PropertyGroupingKey;

	TSparseArray<FObjectPropertyInfo> ResolvedProperties;
	TMultiMap<FContributorKey, FMovieSceneEntityID> Contributors;
	TMultiMap<FContributorKey, FMovieSceneEntityID> NewContributors;

	TArray<UE::MovieScene::FPropertyStats> PropertyStats;

	UE::MovieScene::FComponentMask CleanFastPathMask;

	TBitArray<> PendingInvalidatedProperties;
	TBitArray<> InitializePropertyMetaDataTasks;
	TBitArray<> SaveGlobalStateTasks;

	UE::MovieScene::FBuiltInComponentTypes* BuiltInComponents;
	 
	UE::MovieScene::FPropertyRecomposerImpl RecomposerImpl;
};


template<typename PropertyTraits>
UE::MovieScene::TRecompositionResult<typename PropertyTraits::StorageType> UMovieScenePropertyInstantiatorSystem::RecomposeBlendOperational(const UE::MovieScene::TPropertyComponents<PropertyTraits>& Components, const UE::MovieScene::FDecompositionQuery& InQuery, const typename PropertyTraits::StorageType& InCurrentValue)
{
	return RecomposerImpl.RecomposeBlendOperational<PropertyTraits>(Components, InQuery, InCurrentValue);
}
