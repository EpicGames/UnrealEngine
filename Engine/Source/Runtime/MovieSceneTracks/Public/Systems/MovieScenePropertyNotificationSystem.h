// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"

#include "MovieScenePropertyNotificationSystem.generated.h"

/**
 * A system that can notify an object that some of its properties are being animated.
 */
UCLASS(MinimalAPI)
class UMovieScenePropertyNotificationSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:

	MOVIESCENETRACKS_API UMovieScenePropertyNotificationSystem(const FObjectInitializer& ObjInit);

protected:

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};

