// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieScenePropertySystemTypes.h"
#include "EntitySystem/MovieScenePreAnimatedStateSystem.h"
#include "MovieScenePropertySystem.generated.h"

class UMovieScenePropertyInstantiatorSystem;


/** Abstract base class for any property system that deals with a property registered with FBuiltInComponentTypes::PropertyRegistry */
UCLASS(Abstract, MinimalAPI)
class UMovieScenePropertySystem
	: public UMovieSceneEntitySystem
	, public IMovieScenePreAnimatedStateSystemInterface
{
public:
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieScenePropertySystem(const FObjectInitializer& ObjInit);

	/**
	 * Must be called on construction of derived classes to initialize the members necessary for this system to animate its property
	 */
	template<typename PropertyTraits>
	void BindToProperty(const UE::MovieScene::TPropertyComponents<PropertyTraits>& InComponents)
	{
		check(!RelevantComponent && !CompositePropertyID);

		RelevantComponent = InComponents.PropertyTag;
		CompositePropertyID = InComponents.CompositeID;
	}

protected:

	MOVIESCENETRACKS_API virtual void OnLink() override;
	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	MOVIESCENETRACKS_API virtual void SavePreAnimatedState(const FPreAnimationParameters& InParameters) override;
	MOVIESCENETRACKS_API virtual void RestorePreAnimatedState(const FPreAnimationParameters& InParameters) override;

	/** Pointer to the property instantiator system for retrieving property stats */
	UPROPERTY()
	TObjectPtr<UMovieScenePropertyInstantiatorSystem> InstantiatorSystem;

	/** Must be set on construction - the composite type of the property this system operates with */
	UE::MovieScene::FCompositePropertyTypeID CompositePropertyID;

	/** Pre-animated storage ID saved when we create pre-animated state */
	UE::MovieScene::FPreAnimatedStorageID PreAnimatedStorageID;
};
