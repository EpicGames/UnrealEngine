// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/SortedMap.h"
#include "EntitySystem/MovieSceneBlenderSystem.h"
#include "EntitySystem/MovieSceneCachedEntityFilterResult.h"
#include "EntitySystem/MovieSceneDecompositionQuery.h"
#include "EntitySystem/MovieSceneEntitySystem.h"

#include "MovieSceneQuaternionBlenderSystem.generated.h"

namespace UE::MovieScene
{
	struct FQuatTransform
	{
		FVector Translation;
		FQuat Rotation;
		FVector Scale;
	};

	struct FQuaternionBlenderAccumulationBuffers
	{
		TSparseArray<double> AbsoluteWeights;
		TArray<FQuatTransform> Absolutes;
		TArray<FQuatTransform> Additives;
	};
}

UCLASS(DisplayName="Shortest Rotation (quaternion slerp)", meta=(Tooltip="Uses a quaternion spherical interpolation (slerp) to blend between transforms using the shortest rotation path. Does not support over-rotation."), MinimalAPI)
class UMovieSceneQuaternionBlenderSystem : public UMovieSceneBlenderSystem, public IMovieSceneValueDecomposer
{
public:
	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneQuaternionBlenderSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

	MOVIESCENETRACKS_API virtual FGraphEventRef DispatchDecomposeTask(const UE::MovieScene::FValueDecompositionParams& Params, UE::MovieScene::FAlignedDecomposedValue* Output) override;

private:

	UE::MovieScene::FQuaternionBlenderAccumulationBuffers Buffers;
};

