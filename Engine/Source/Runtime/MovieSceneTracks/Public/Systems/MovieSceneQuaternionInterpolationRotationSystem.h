// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "MovieSceneQuaternionInterpolationRotationSystem.generated.h"


UCLASS(MinimalAPI)
class UMovieSceneQuaternionInterpolationRotationSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneQuaternionInterpolationRotationSystem(const FObjectInitializer& ObjInit);

private:
	MOVIESCENETRACKS_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};

