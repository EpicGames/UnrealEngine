// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Systems/MovieScenePropertySystem.h"
#include "MovieSceneRotatorPropertySystem.generated.h"

/** System for rotator property registered within PropertyRegistry */
UCLASS(MinimalAPI)
class UMovieSceneRotatorPropertySystem : public UMovieScenePropertySystem
{
	GENERATED_BODY()

public:
	UMovieSceneRotatorPropertySystem(const FObjectInitializer& InObjectInitializer);

	//~ Begin UMovieSceneEntitySystem
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& InSubsequents) override;
	//~ End UMovieSceneEntitySystem
};
