// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityGroupingSystem.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "Evaluation/MovieScenePlayback.h"
#include "MovieSceneTracksComponentTypes.h"
#include "UObject/ObjectKey.h"
#include "MovieSceneSkeletalAnimationSystem.generated.h"

class IMovieScenePlayer;
class UAnimMontage;
class UMovieSceneSkeletalAnimationSection;
enum class ESwapRootBone : uint8;

namespace UE::MovieScene
{

/** Information for a single skeletal animation playing on a bound object */
struct FActiveSkeletalAnimation
{
	const UMovieSceneSkeletalAnimationSection* AnimSection;
	FMovieSceneContext Context;
	FFrameTime EvalFrameTime;
	FMovieSceneEntityID EntityID;
	FRootInstanceHandle RootInstanceHandle;
	double BlendWeight;
	float FromEvalTime;
	float ToEvalTime;
	EMovieScenePlayerStatus::Type PlayerStatus;
	uint8 bFireNotifies : 1;
	uint8 bPlaying : 1;
	uint8 bResetDynamics : 1;
	uint8 bWantsRestoreState : 1;
	uint8 bPreviewPlayback : 1;
};

/** DelegateHandle and Skeletal Mesh for bone transform finalized */
struct FBoneTransformFinalizeData
{
	FBoneTransformFinalizeData();
	TWeakObjectPtr<USkeletalMeshComponent> SkeletalMeshComponent;
	ESwapRootBone SwapRootBone;
	FTransform MeshRelativeRootMotionTransform;
	TOptional<FTransform> InitialActorTransform;
	TOptional<FQuat> InverseMeshToActorRotation;
	/** Delegate Handle for skel mesh bone transform finalized callback*/
	FDelegateHandle OnBoneTransformsFinalizedHandle;

	#if WITH_EDITORONLY_DATA
	FDelegateHandle OnBeginActorMovementHandle;
	FDelegateHandle OnEndActorMovementHandle;
	bool bActorBeingMoved = false;
	#endif

	void Register(USkeletalMeshComponent* InSkeleletalMeshCompononent, ESwapRootBone InSwapRootBone, FTransform& InMeshRelativeRootMotionTransform, TOptional<FTransform> InInitialActorTransform);
	void Unregister();
	void BoneTransformFinalized();
};

/** Information for all skeletal animations playing on a bound object */
struct FBoundObjectActiveSkeletalAnimations
{
	using FAnimationArray = TArray<FActiveSkeletalAnimation, TInlineAllocator<2>>;

	/** All active animations on the corresponding bound object */
	FAnimationArray Animations;
	/** Motion vector simulation animations on the corresponding bound object */
	FAnimationArray SimulatedAnimations;
	/** SkelMesh and the bone finalize Delegate*/
	FBoneTransformFinalizeData  BoneTransformFinalizeData;

};

/** Temporary information about montage setups. */
struct FMontagePlayerPerSectionData 
{
	TWeakObjectPtr<UAnimMontage> Montage;
	int32 MontageInstanceId;
};

struct FSkeletalAnimationSystemData
{
	void ResetSkeletalAnimations();

	/** Map of active skeletal animations for each bound object */
	TMap<TWeakObjectPtr<USkeletalMeshComponent>, FBoundObjectActiveSkeletalAnimations> SkeletalAnimations;

	/** Map of persistent montage data */
	TMap<FObjectKey, TMap<FObjectKey, FMontagePlayerPerSectionData>> MontageData;
};

} // namespace UE::MovieScene

UCLASS(MinimalAPI)
class UMovieSceneSkeletalAnimationSystem
	: public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneSkeletalAnimationSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API static UObject* ResolveSkeletalMeshComponentBinding(UObject* InObject);

	MOVIESCENETRACKS_API FTransform GetRootMotionOffset(UObject* InObject) const;
	MOVIESCENETRACKS_API void UpdateRootMotionOffset(UObject* InObject);

	MOVIESCENETRACKS_API TOptional<FTransform> GetInitialActorTransform(UObject* InObject) const;
	MOVIESCENETRACKS_API TOptional<FQuat> GetInverseMeshToActorRotation(UObject* InObject) const;

protected:
	
	struct FAnimationGroupingPolicy
	{
		using GroupKeyType = FObjectKey;

		bool GetGroupKey(UObject* Object, FMovieSceneSkeletalAnimationComponentData ComponentData, GroupKeyType& OutGroupKey)
		{
			// ComponentData is only used to filter the grouping to objects with animations. If it were used for the key, it would only group objects
			// with identical animations, so instead the object is used for both the key and value of the tuple. 
			OutGroupKey = FObjectKey(Object);
			return true;
		}
		
#if WITH_EDITOR
		bool OnObjectsReplaced(GroupKeyType& InOutKey, const TMap<UObject*, UObject*>& ReplacementMap)
		{
			if (UObject* const * NewObject = ReplacementMap.Find(InOutKey.ResolveObjectPtr()))
			{
				InOutKey = *NewObject;
				return true;
			}
			return false;
		}
#endif
	
	};

private:

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;
	virtual void OnLink() override;
	virtual void OnUnlink() override;

	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;

	void CleanSystemData();

private:

	UE::MovieScene::FSkeletalAnimationSystemData SystemData;
	UE::MovieScene::FEntityGroupingPolicyKey GroupingKey;
};

