// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "Math/Transform.h"
#include "Misc/Build.h"

#include "MovieSceneTransformOriginSystem.generated.h"

struct FMovieSceneAnimTypeID;

// Helper struct to define sorting behavior for parent to child mapping. 
struct FInstanceToParentPair
{
	// In this struct representation the Child represents this sequence.
	UE::MovieScene::FInstanceHandle Child;
	UE::MovieScene::FInstanceHandle Parent;
	int32 Depth = 0;

	FInstanceToParentPair(const UE::MovieScene::FInstanceHandle Child, const UE::MovieScene::FInstanceHandle Parent, const int32 Depth)
	{
		this->Child = Child;
		this->Parent = Parent;
		this->Depth = Depth;
	}

	bool operator<(const FInstanceToParentPair& Other) const
	{
		return Depth < Other.Depth;
	}

	bool operator==(const FInstanceToParentPair& Other) const
	{
		return Child == Other.Child && Parent == Other.Parent; 
	}
};

UCLASS(MinimalAPI)
class UMovieSceneTransformOriginInstantiatorSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneTransformOriginInstantiatorSystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;
};


UCLASS(MinimalAPI)
class UMovieSceneTransformOriginSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UMovieSceneTransformOriginSystem(const FObjectInitializer& ObjInit);

public:


	MOVIESCENETRACKS_API bool GetTransformOrigin(UE::MovieScene::FInstanceHandle InstanceHandle, FTransform& OutTransform) const;

	const TSparseArray<FTransform>& GetTransformOriginsByInstanceID() const { return TransformOriginsByInstanceID; }


	const TMap<FMovieSceneSequenceID, UE::MovieScene::FInstanceHandle>& GetSequenceIDToInstanceHandle() const { return SequenceIDToInstanceHandle; }

private:

	virtual void OnLink() override;
	virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override final;

private:

	TSparseArray<FTransform> TransformOriginsByInstanceID;
	TArray<FInstanceToParentPair> InstanceHandleToParentHandle;
	UE::MovieScene::FEntityComponentFilter LocationAndRotationFilterResults;


	TMap<FMovieSceneSequenceID, UE::MovieScene::FInstanceHandle> SequenceIDToInstanceHandle;

};



