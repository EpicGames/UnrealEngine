// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/IMovieScenePropertyComponentHandler.h"
#include "Systems/MovieScenePropertySystem.h"
#include "MovieSceneVectorPropertySystem.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneFloatVectorPropertySystem : public UMovieScenePropertySystem
{
	GENERATED_BODY()

private:
	UMovieSceneFloatVectorPropertySystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};

UCLASS(MinimalAPI)
class UMovieSceneDoubleVectorPropertySystem : public UMovieScenePropertySystem
{
	GENERATED_BODY()

private:
	UMovieSceneDoubleVectorPropertySystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};

