// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneVisibilitySystem.generated.h"

class UMovieSceneEntitySystemLinker;
class UObject;

namespace UE::MovieScene
{
	struct FPreAnimatedVisibilityStorage;
}  // namespace UE::MovieScene

UCLASS()
class UMovieSceneVisibilitySystem : public UMovieSceneEntityInstantiatorSystem
{
public:

	GENERATED_BODY()

	UMovieSceneVisibilitySystem(const FObjectInitializer& ObjInit);

private:

	virtual void OnLink() override;
	virtual void OnUnlink() override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:

	TSharedPtr<UE::MovieScene::FPreAnimatedVisibilityStorage> PreAnimatedStorage;
};

