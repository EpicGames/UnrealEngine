// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "ObjectPathChannelEvaluatorSystem.generated.h"

class UObject;

/**
 * System that is responsible for evaluating object path channels.
 */
UCLASS(MinimalAPI)
class UObjectPathChannelEvaluatorSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	UObjectPathChannelEvaluatorSystem(const FObjectInitializer& ObjInit);

	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
};