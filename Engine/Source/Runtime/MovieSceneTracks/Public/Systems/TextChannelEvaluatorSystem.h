// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "EntitySystem/MovieSceneEntitySystem.h"
#include "TextChannelEvaluatorSystem.generated.h"

/** System responsible for evaluating Text channels */
UCLASS(MinimalAPI)
class UTextChannelEvaluatorSystem : public UMovieSceneEntitySystem
{
	GENERATED_BODY()

public:
	UTextChannelEvaluatorSystem(const FObjectInitializer& InObjectInitializer);

	//~ Begin UMovieSceneEntitySystem
	virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	//~ End UMovieSceneEntitySystem
};
