// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainerAllocationPolicies.h"
#include "Containers/Map.h"
#include "Containers/Set.h"
#include "Containers/SortedMap.h"
#include "Containers/SparseArray.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/EntityAllocationIterator.h"
#include "EntitySystem/MovieSceneEntityIDs.h"
#include "EntitySystem/MovieSceneEntityInstantiatorSystem.h"
#include "EntitySystem/MovieSceneEntitySystem.h"
#include "EntitySystem/MovieSceneSequenceInstanceHandle.h"
#include "HAL/Platform.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "WeightAndEasingEvaluatorSystem.generated.h"

class UMovieSceneEntitySystemLinker;
class UMovieSceneEvalTimeSystem;
class UWeightAndEasingEvaluatorSystem;
class UObject;
struct FMovieSceneSubSequenceData;

// Bezier blending gives a true linear interpolation between biases, but 
// disabled by default because it misbehaves if you feed it different parameters for 't' to each term
#ifndef UE_ENABLE_MOVIESCENE_BEZIER_BLENDING
	#define UE_ENABLE_MOVIESCENE_BEZIER_BLENDING 0 
#endif

namespace UE::MovieScene
{

struct FEasingMutationBase;
struct FAddEasingChannelToProviderMutation;
struct FAddEasingChannelToConsumerMutation;

enum class EHierarchicalBlendMode : uint8
{
	// Blend hierarchical weights by multiplying children with all parent weights (default)
	AccumulateParentToChild,
	// Combine weights from child -> parent using an increasingly small factor based on previously accumulated weights
	// For example:
	//     values = ( (a,t1), (b, t2), (c, t3), (d, t4))
	//     float final_weight = 1;
	//     float t = 1;
	//     for (i : size(values))
	//        float this = values[i].value*t*values[i].weight
	//        t = max(t - this, 0)
	//        final_weight *= this
	ChildFirstBlendTarget,
#if UE_ENABLE_MOVIESCENE_BEZIER_BLENDING
	// Combine weights from child -> parent using a bezier interpolation series
	BezierSeries,
#endif
};

/** Computation data used for accumulating hierarchical weights for sub sequences */
struct FHierarchicalEasingChannelData
{
	/** The final result of this easing channel, accumulated with all parents */
	double FinalResult = 1.0;
	/** Initial easing result from manual and easing weights, not accumulated with all parents */
	double UnaccumulatedResult = 1.0;

	/** Our parent's computation data within PreAllocatedComputationData (in order to support one -> many multiplications). */
	uint16 ParentChannel = uint16(-1);
	/** An optional result channel to feed our result to (in order to support many -> one multiplications) */
	uint16 ResultChannel = uint16(-1);

	/** This channel's HBias */
	int16 HBias = 0;

#if UE_ENABLE_MOVIESCENE_BEZIER_BLENDING
	// Bezier blending coefficients of the form a(1-t)^b + t^c
	float CoeffA = 1.f;
	float ExpB = 1.f;
	float ExpC = 1.f;
#endif

	EHierarchicalBlendMode BlendMode = EHierarchicalBlendMode::AccumulateParentToChild;
};

struct FHierarchicalEasingChannelBuffer
{
	bool IsEmpty() const;

	void Reset();
	void ResetBlendTargets();

	uint16 AddBaseChannel(const FHierarchicalEasingChannelData& InChannelData);
	uint16 AddBlendTargetChannel(const FHierarchicalEasingChannelData& InChannelData);

	TArray<FHierarchicalEasingChannelData> Channels;
	int32 BlendTargetStartIndex = INDEX_NONE;
};

}  // namespace UE::MovieScene

/**
 * System that creates hierarchical easing channels for any newly introduced HierarchicalEasingProvider components
 */
UCLASS(MinimalAPI)
class UMovieSceneHierarchicalEasingInstantiatorSystem : public UMovieSceneEntityInstantiatorSystem
{
public:

	struct FHierarchicalInstanceData
	{
		int32 HierarchicalDepth = -1;
		uint16 StrongRefCount = 0;
		uint16 WeakRefCount = 0;
	};

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneHierarchicalEasingInstantiatorSystem(const FObjectInitializer& ObjInit);

	MOVIESCENETRACKS_API void FinalizeBlendTargets();

private:

	MOVIESCENETRACKS_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const;
	MOVIESCENETRACKS_API virtual void OnLink() override;
	MOVIESCENETRACKS_API virtual void OnUnlink() override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;

private:

	/** Map between a sub-sequence handle and the easing channel affecting it */
	TSparseArray<uint16> CachedInstanceIDToChannel;

	/** Cached output channels for a specific pair (source, target) of hierarchical blends */
	TMap<TTuple<int16, UE::MovieScene::FHierarchicalBlendTarget>, uint16> CachedHierarchicalBlendTargetChannels;

	UPROPERTY()
	TObjectPtr<UWeightAndEasingEvaluatorSystem> EvaluatorSystem;

	/** True of this system just updated all its channels */
	bool bChannelsHaveBeenInvalidated;
};

/**
 * System that finalizes creation of easing channels by allocating new channels for hierarchical blend targets if necessary
 * This is required in order to solve the problem that the hierarchical instantiator needs to run before the property instantiator,
 * but we can't allocate channels for blend targets until _after_ the property instantiator (which sets up the relevant BlendTarget components)
 * 
 */
UCLASS()
class UE_DEPRECATED(5.5, "This system is no longer functional") UMovieSceneHierarchicalEasingFinalizationSystem : public UMovieSceneEntityInstantiatorSystem
{
public:
	GENERATED_BODY()

	UMovieSceneHierarchicalEasingFinalizationSystem(const FObjectInitializer& ObjInit);
};

/**
 * System that combines manual weights and easings and propagates them to entities with matching EasingChannelID components
 */
UCLASS(MinimalAPI)
class UWeightAndEasingEvaluatorSystem : public UMovieSceneEntitySystem
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UWeightAndEasingEvaluatorSystem(const FObjectInitializer& ObjInit);

	/**
	 * Called from the instantiator system once channel IDs have been calculated
	 */
	MOVIESCENETRACKS_API UE::MovieScene::FHierarchicalEasingChannelBuffer& GetComputationBuffer();

private:

	MOVIESCENETRACKS_API virtual void OnSchedulePersistentTasks(UE::MovieScene::IEntitySystemScheduler* TaskScheduler) override;
	MOVIESCENETRACKS_API virtual void OnRun(FSystemTaskPrerequisites& InPrerequisites, FSystemSubsequentTasks& Subsequents) override;
	MOVIESCENETRACKS_API virtual bool IsRelevantImpl(UMovieSceneEntitySystemLinker* InLinker) const override;
	MOVIESCENETRACKS_API virtual void OnLink() override final;
	MOVIESCENETRACKS_API virtual void OnUnlink() override final;

private:

	/** Unstable array of preallocated storage for computing easing results sorted by hierarchical depth. */
	UE::MovieScene::FHierarchicalEasingChannelBuffer PreAllocatedComputationData;
};

