// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Kismet/BlueprintFunctionLibrary.h"

#include "MovieSceneFadeTrackTests.generated.h"

class APlayerCameraManager;

UCLASS()
class UMovieSceneFadeTrackTestLibrary : public UBlueprintFunctionLibrary
{
	GENERATED_BODY()

public:

	UFUNCTION(BlueprintCallable, Category="Sequencer Tests")
	static float GetManualFadeAmount(APlayerCameraManager* PlayerCameraManager);
};

