// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Channels/MovieSceneChannelHandle.h"
#include "Channels/MovieSceneChannelTraits.h"
#include "Channels/MovieSceneChannelProxy.h"
#include "CoreTypes.h"
#include "Generators/MovieSceneEasingCurves.h"
#include "IMovieScenePlayer.h"
#include "KeyParams.h"
#include "MovieScene.h"
#include "MovieSceneSection.h"
#include "MovieSceneSequence.h"
#include "Templates/EnableIf.h"
#include "Templates/PointerIsConvertibleFromTo.h"
#include "Tracks/MovieScenePropertyTrack.h"

#include "MovieSceneTestDataBuilders.generated.h"

/**
 * Simple type of sequence for use in automated tests.
 *
 * Bound objects are specified manually on the sequence and will be simply returned when
 * bindings are resolved.
 */
UCLASS(MinimalAPI)
class UMovieSceneTestSequence : public UMovieSceneSequence
{
	GENERATED_BODY()

public:
	/** The movie scene */
	UPROPERTY()
	TObjectPtr<UMovieScene> MovieScene;

public:
	/** Initialize this test sequence */
	MOVIESCENETRACKS_API void Initialize();

	/**
	 * Add an object binding to the sequnce
	 *
	 * @param InObject The object that will be returned when the binding is resolved
	 * @return The ID of the new object binding
	 */
	MOVIESCENETRACKS_API FGuid AddObjectBinding(TObjectPtr<UObject> InObject);

public:
	/** UMovieSceneSequence interface */
	virtual void BindPossessableObject(const FGuid& ObjectId, UObject& PossessedObject, UObject* Context) override {}
	virtual bool CanPossessObject(UObject& Object, UObject* InPlaybackContext) const override { return true; }
	MOVIESCENETRACKS_API virtual void LocateBoundObjects(const FGuid& ObjectId, UObject* Context, TArray<UObject*, TInlineAllocator<1>>& OutObjects) const override;
	virtual UMovieScene* GetMovieScene() const override { return MovieScene; }
	virtual UObject* GetParentObject(UObject* Object) const override { return nullptr; }
	virtual void UnbindPossessableObjects(const FGuid& ObjectId) override {}
	virtual void UnbindObjects(const FGuid& ObjectId, const TArray<UObject*>& InObjects, UObject* Context) override {}
	virtual void UnbindInvalidObjects(const FGuid& ObjectId, UObject* Context) override {}

private:
	UPROPERTY()
	TArray<TObjectPtr<UObject>> BoundObjects;

	UPROPERTY()
	TArray<FGuid> BindingGuids;
};

namespace UE::MovieScene::Test
{

template<typename TrackClass> struct FSequenceTrackBuilder;
template<typename TrackClass> struct FSequenceSectionBuilder;

/**
 * Utility class for building a test level sequence using a fluid interface API.
 */
struct FSequenceBuilder
{
	TObjectPtr<UMovieSceneTestSequence> Sequence;
	FGuid CurrentBinding;

	FSequenceBuilder()
	{
		Sequence = NewObject<UMovieSceneTestSequence>();
		Sequence->Initialize();
	}

	FSequenceBuilder& AddObjectBinding(TObjectPtr<UObject> InObject)
	{
		CurrentBinding = Sequence->AddObjectBinding(InObject);
		return *this;
	}

	FSequenceBuilder& AddObjectBinding(TObjectPtr<UObject> InObject, FGuid& OutBindingID)
	{
		AddObjectBinding(InObject);
		OutBindingID = CurrentBinding;
		return *this;
	}

	template<typename TrackClass>
	FSequenceTrackBuilder<TrackClass> AddRootTrack()
	{
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		TrackClass* Track = MovieScene->AddTrack<TrackClass>();
		return FSequenceTrackBuilder<TrackClass>(*this, Track);
	}

	template<typename TrackClass>
	FSequenceTrackBuilder<TrackClass> AddTrack()
	{
		checkf(CurrentBinding.IsValid(), TEXT("Specify an object binding first with AddObjectBinding"));
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		TrackClass* Track = MovieScene->AddTrack<TrackClass>(CurrentBinding);
		return FSequenceTrackBuilder<TrackClass>(*this, Track);
	}

	template<typename TrackClass>
	FSequenceTrackBuilder<TrackClass> AddPropertyTrack(FName InPropertyName)
	{
		checkf(CurrentBinding.IsValid(), TEXT("Specify an object binding first with AddObjectBinding"));
		UMovieScene* MovieScene = Sequence->GetMovieScene();
		TrackClass* Track = MovieScene->AddTrack<TrackClass>(CurrentBinding);
		Track->SetPropertyNameAndPath(InPropertyName, InPropertyName.ToString());
		return FSequenceTrackBuilder<TrackClass>(*this, Track);
	}
};

/**
 * Utility class for building a track on a test level sequence using a fluid interface API.
 */
template<typename TrackClass>
struct FSequenceTrackBuilder
{
	FSequenceBuilder& Parent;
	TObjectPtr<TrackClass> Track;

	FSequenceTrackBuilder(FSequenceBuilder& InParent, TrackClass* InTrack)
		: Parent(InParent), Track(InTrack)
	{}

	FSequenceTrackBuilder& Assign(TrackClass*& OutTrack)
	{
		OutTrack = Track;
		return *this;
	}

	template<typename Func>
	FSequenceBuilder& Do(Func&& Callback)
	{
		Callback(Track);
		return *this;
	}

	FSequenceSectionBuilder<TrackClass> AddSection(FFrameNumber InStart, FFrameNumber InEnd, int32 InRowIndex = INDEX_NONE)
	{
		UMovieSceneSection* NewSection = Track->CreateNewSection();
		NewSection->SetRange(TRange<FFrameNumber>(InStart, InEnd));
		if (InRowIndex != INDEX_NONE)
		{
			NewSection->SetRowIndex(InRowIndex);
		}
		Track->AddSection(*NewSection);
		return FSequenceSectionBuilder<TrackClass>(*this, NewSection);
	}

	FSequenceBuilder& Pop() { return Parent; }
};

/**
 * Utility class for building a track section on a test level sequence using a flui interface API.
 */
template<typename ParentTrackClass>
struct FSequenceSectionBuilder
{
	FSequenceTrackBuilder<ParentTrackClass>& Parent;
	TObjectPtr<UMovieSceneSection> Section;

	FSequenceSectionBuilder(FSequenceTrackBuilder<ParentTrackClass>& InParent, UMovieSceneSection* InSection)
		: Parent(InParent), Section(InSection)
	{}

	FSequenceSectionBuilder& Assign(UMovieSceneSection*& OutSection)
	{
		OutSection = Section;
		return *this;
	}

	template<typename SectionClass>
	FSequenceSectionBuilder& Assign(SectionClass*& OutSection)
	{
		OutSection = Cast<SectionClass>(Section);
		return *this;
	}

	FSequenceSectionBuilder& SetBlendType(EMovieSceneBlendType InBlendType)
	{
		Section->SetBlendType(InBlendType);
		return *this;
	}

	FSequenceSectionBuilder& SetEaseIn(int32 InDurationTicks, EMovieSceneBuiltInEasing InEasingType = EMovieSceneBuiltInEasing::Linear)
	{
		FMovieSceneEasingSettings& Easing = Section->Easing;
		Easing.bManualEaseIn = true;
		Easing.ManualEaseInDuration = InDurationTicks;
		if (UMovieSceneBuiltInEasingFunction* Function = Cast<UMovieSceneBuiltInEasingFunction>(Easing.EaseIn.GetObject()))
		{
			Function->Type = InEasingType;
		}
		return *this;
	}

	FSequenceSectionBuilder& SetEaseOut(int32 InDurationTicks, EMovieSceneBuiltInEasing InEasingType = EMovieSceneBuiltInEasing::Linear)
	{
		FMovieSceneEasingSettings& Easing = Section->Easing;
		Easing.bManualEaseOut = true;
		Easing.ManualEaseOutDuration = InDurationTicks;
		if (UMovieSceneBuiltInEasingFunction* Function = Cast<UMovieSceneBuiltInEasingFunction>(Easing.EaseOut.GetObject()))
		{
			Function->Type = InEasingType;
		}
		return *this;
	}

	template<typename ChannelType, typename ValueType>
	FSequenceSectionBuilder& AddKey(
			int32 InChannelIndex,
			FFrameNumber InTime,
			ValueType InValue,
			EMovieSceneKeyInterpolation Interpolation = EMovieSceneKeyInterpolation::Auto)
	{
		FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
		ChannelType* Channel = ChannelProxy.GetChannel<ChannelType>(InChannelIndex);
		return AddKeys(Channel, TArray<FFrameNumber>({ InTime }), TArray<ValueType>({ InValue }), Interpolation);
	}

	template<typename ChannelType, typename ValueType>
	FSequenceSectionBuilder& AddKeys(
			int32 InChannelIndex,
			const TArray<FFrameNumber>& InTimes,
			const TArray<ValueType>& InValues,
			EMovieSceneKeyInterpolation Interpolation = EMovieSceneKeyInterpolation::Auto)
	{
		FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
		ChannelType* Channel = ChannelProxy.GetChannel<ChannelType>(InChannelIndex);
		return AddKeys(Channel, InTimes, InValues, Interpolation);
	}

#if WITH_EDITOR
	template<typename ChannelType, typename ValueType>
	FSequenceSectionBuilder& AddKeys(
			FName InChannelName, 
			const TArray<FFrameNumber>& InTimes,
			const TArray<ValueType>& InValues,
			EMovieSceneKeyInterpolation Interpolation = EMovieSceneKeyInterpolation::Auto)
	{
		FMovieSceneChannelProxy& ChannelProxy = Section->GetChannelProxy();
		TMovieSceneChannelHandle<ChannelType> ChannelHandle = ChannelProxy.GetChannelByName<ChannelType>(InChannelName);
		return AddKeys(ChannelHandle.Get(), InTimes, InValues, Interpolation);
	}
#endif

	template<typename ChannelType, typename ValueType>
	FSequenceSectionBuilder& AddKeys(
			ChannelType* InChannel,
			const TArray<FFrameNumber>& InTimes, 
			const TArray<ValueType>& InValues, 
			EMovieSceneKeyInterpolation Interpolation = EMovieSceneKeyInterpolation::Auto)
	{
		ensure(InTimes.Num() == InValues.Num());
		int32 Num = FMath::Min(InTimes.Num(), InValues.Num());
		for (int32 Index = 0; Index < Num; ++Index)
		{
			AddKeyToChannel(InChannel, InTimes[Index], InValues[Index], Interpolation);
		}
		return *this;
	}

	template<typename Func>
	FSequenceSectionBuilder& Do(Func&& Callback)
	{
		Callback(Section);
		return *this;
	}

	template<typename SectionClass, typename Func>
	FSequenceSectionBuilder& Do(Func&& Callback)
	{
		Callback(Cast<SectionClass>(Section));
		return *this;
	}

	FSequenceTrackBuilder<ParentTrackClass>& Pop() { return Parent; }
};

} // namespace UE::MovieScene::Test

