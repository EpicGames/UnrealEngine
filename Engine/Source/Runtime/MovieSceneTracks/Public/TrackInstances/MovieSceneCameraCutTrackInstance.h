// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/TrackInstance/MovieSceneTrackInstance.h"
#include "UObject/ObjectMacros.h"

#include "MovieSceneCameraCutTrackInstance.generated.h"

#define UE_API MOVIESCENETRACKS_API

class IMovieScenePlayer;
class UMovieSceneCameraCutSection;

namespace UE::MovieScene
{ 
	class FCameraCutViewportPreviewer;
	struct FCameraCutAnimator; 
	struct FCameraCutPlaybackCapability;
	struct FOnCameraCutUpdatedParams;
	struct FSequenceInstance;

	// Backwards compatibilty wrapper for camera cut playback capability.
	struct FCameraCutPlaybackCapabilityCompatibilityWrapper
	{
		FCameraCutPlaybackCapabilityCompatibilityWrapper(const FSequenceInstance& SequenceInstance);

		bool ShouldUpdateCameraCut();
		void OnCameraCutUpdated(const FOnCameraCutUpdatedParams& Params);
#if WITH_EDITOR
		bool ShouldRestoreEditorViewports();
#endif

		FCameraCutPlaybackCapability* CameraCutCapability;
		IMovieScenePlayer* Player;
	};
}

/**
 * Track instance used to animate camera cuts.
 */
UCLASS(MinimalAPI)
class UMovieSceneCameraCutTrackInstance : public UMovieSceneTrackInstance
{
	GENERATED_BODY()

public:
#if WITH_EDITOR
	/**
	 * Toggle camera cut lock on cinematic editor viewports while also correctly managing
	 * remember/restoring/discarding pre-animated viewport positions based on sequencer
	 * settings.
	 */
	static UE_API void ToggleCameraCutLock(UMovieSceneEntitySystemLinker* Linker, bool bEnableCameraCuts, bool bRestoreViewports);
#endif

private:
	UE_API virtual void OnInitialize() override;
	UE_API virtual void OnAnimate() override;
	UE_API virtual void OnEndUpdateInputs() override;
	UE_API virtual void OnDestroyed() override;

private:
	/**
	 * Stores information about the last set camera in order to differentiate
	 * between new and pre-existing cuts.
	 */
	struct FCameraCutCache
	{
		TWeakObjectPtr<> LastLockedCamera;
		FMovieSceneTrackInstanceInput LastInput;
	};

	/**
	 * Track instance input qualified with the global start time of its corresponding
	 * section, used for sorting inputs and prioritizing more "recent" camera cuts
	 * over "older" ones.
	 */
	struct FCameraCutInputInfo
	{
		FMovieSceneTrackInstanceInput Input;
		float GlobalStartTime = 0.f;
		bool bSavePreAnimatedValue = false;
	};

	FCameraCutCache CameraCutCache;
	TArray<FCameraCutInputInfo> SortedInputInfos;

#if WITH_EDITOR
	TUniquePtr<UE::MovieScene::FCameraCutViewportPreviewer> ViewportPreviewer;
#endif

private:

	friend struct UE::MovieScene::FCameraCutAnimator;
};

#undef UE_API
