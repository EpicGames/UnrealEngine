// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "UObject/NameTypes.h"
#include "UObject/Interface.h"
#include "IMovieSceneSectionsToKey.generated.h"

class UMovieSceneSection;

/**
 * Functionality for having multiple sections per key
 */

UINTERFACE(MinimalAPI)
class UMovieSceneSectionsToKey : public UInterface
{
public:
	GENERATED_BODY()
};

/**
 * Interface to be added to UMovieSceneTrack when they have multiple sections to key
 */
class IMovieSceneSectionsToKey
{
public:

	GENERATED_BODY()
	MOVIESCENETRACKS_API IMovieSceneSectionsToKey();

	/* Get multiple sections to key*/
	virtual TArray<TWeakObjectPtr<UMovieSceneSection>> GetSectionsToKey() const = 0;

};
