// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/ObjectMacros.h"
#include "UObject/Interface.h"

#include "IMovieSceneTransformOrigin.generated.h"


UINTERFACE(Category="Sequencer", Blueprintable, meta=(DisplayName = "Transform Origin"), MinimalAPI)
class UMovieSceneTransformOrigin : public UInterface
{
public:
	GENERATED_BODY()
};

/** Interface that should be implemented to provide transform tracks with an origin transform. Scale is ignored. */
class IMovieSceneTransformOrigin
{
public:
	GENERATED_BODY()

	/** Get the transform origin for this interface */
	FTransform GetTransformOrigin() const
	{
		return NativeGetTransformOrigin();
	}

protected:

	/** Get the transform from which all absolute component transform sections should be relative. Scale is ignored. */
	UFUNCTION(BlueprintCallable, BlueprintImplementableEvent, Category = "Sequencer|Section", DisplayName = "GetTransformOrigin", meta=(CallInEditor="true"))
	MOVIESCENETRACKS_API FTransform BP_GetTransformOrigin() const;
	virtual FTransform NativeGetTransformOrigin() const { return BP_GetTransformOrigin(); }
};
