// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "MovieSceneObjectBindingID.h"
#include "Templates/SubclassOf.h"
#include "Tracks/MovieScene3DConstraintTrack.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieScene3DAttachTrack.generated.h"

class UMovieSceneSection;
class UObject;
struct FFrameNumber;

/**
 * Handles manipulation of path tracks in a movie scene.
 */
UCLASS( MinimalAPI )
class UMovieScene3DAttachTrack
	: public UMovieScene3DConstraintTrack
{
	GENERATED_UCLASS_BODY()

public:

	// UMovieScene3DConstraintTrack interface

	virtual UMovieSceneSection* AddConstraint( FFrameNumber Time, int32 Duration, const FName SocketName, const FName ComponentName, const FMovieSceneObjectBindingID& ConstraintBindingID ) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual class UMovieSceneSection* CreateNewSection() override;

public:

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
#endif
};
