// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieScene3DConstraintTrack.generated.h"

class UObject;

/**
 * Base class for constraint tracks (tracks that are dependent upon other objects).
 */
UCLASS( MinimalAPI )
class UMovieScene3DConstraintTrack
	: public UMovieSceneTrack
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Adds a constraint.
	 *
	 * @param Time The time relative to the owning movie scene where the section should be.
	 * @param Duration The length of the constraint section
	 * @param SocketName The socket name for the constraint.
	 * @param ComponentName The name of the component the socket resides in.
	 * @param FMovieSceneObjectBindingID The object binding id to the constraint.
	 * @return The newly created constraint section
	 */
	virtual UMovieSceneSection* AddConstraint(FFrameNumber Time, int32 Duration, const FName SocketName, const FName ComponentName, const FMovieSceneObjectBindingID& ConstraintBindingID) { return nullptr; }

public:

	// UMovieSceneTrack interface

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual class UMovieSceneSection* CreateNewSection() override;
    virtual void RemoveAllAnimationData() override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool IsEmpty() const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
#endif

protected:

	/** List of all constraint sections. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> ConstraintSections;
};
