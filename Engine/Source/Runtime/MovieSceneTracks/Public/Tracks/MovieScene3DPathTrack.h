// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Internationalization/Text.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSection.h"
#include "Templates/SubclassOf.h"
#include "Tracks/MovieScene3DConstraintTrack.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieScene3DPathTrack.generated.h"

class UObject;
struct FFrameNumber;

/**
 * Handles manipulation of path tracks in a movie scene
 */
UCLASS(MinimalAPI)
class UMovieScene3DPathTrack
	: public UMovieScene3DConstraintTrack
	, public IMovieSceneTrackTemplateProducer
{
	GENERATED_UCLASS_BODY()

public:

	// UMovieScene3DConstraintTrack interface

	virtual UMovieSceneSection* AddConstraint(FFrameNumber Time, int32 Duration, const FName SocketName, const FName ComponentName, const FMovieSceneObjectBindingID& ConstraintBindingID) override;

public:

	// UMovieSceneTrack interface

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual class UMovieSceneSection* CreateNewSection() override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
#endif
};
