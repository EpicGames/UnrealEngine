// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Sections/MovieScene3DTransformSection.h"
#include "EntitySystem/IMovieSceneBlenderSystemSupport.h"
#include "MovieScene3DTransformTrack.generated.h"

enum class EMovieSceneTransformChannel : uint32;
class UMovieScene3DTransformSection;
struct FMovieSceneInterrogationKey;

struct FTrajectoryKey
{
	struct FData
	{
		FData(UMovieScene3DTransformSection* InSection, TOptional<FKeyHandle> InKeyHandle, ERichCurveInterpMode InInterpMode, FName InChannelName)
			: Section(InSection), KeyHandle(InKeyHandle), ChannelName(InChannelName), InterpMode(InInterpMode)
		{}

		TWeakObjectPtr<UMovieScene3DTransformSection> Section;
		TOptional<FKeyHandle> KeyHandle;
		FName ChannelName;
		ERichCurveInterpMode InterpMode;
	};

	FTrajectoryKey(FFrameNumber InTime) : Time(InTime) {}

	bool Is(ERichCurveInterpMode InInterpMode) const
	{
		if (KeyData.Num())
		{
			for (const FData& Value : KeyData)
			{
				if (Value.InterpMode != InInterpMode)
				{
					return false;
				}
			}
			return true;
		}
		return false;
	}

	FFrameNumber Time;

	TArray<FData, TInlineAllocator<1>> KeyData;
};

/**
 * Handles manipulation of component transforms in a movie scene
 */
UCLASS(MinimalAPI)
class UMovieScene3DTransformTrack
	: public UMovieScenePropertyTrack
	, public IMovieSceneBlenderSystemSupport
{
	GENERATED_UCLASS_BODY()

public:

	// UMovieSceneTrack interface

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

#if WITH_EDITORONLY_DATA
	virtual bool CanRename() const override { return true; }
	virtual FText GetDisplayName() const override;
#endif

	// IMovieSceneBlenderSystemSupport
	TSubclassOf<UMovieSceneBlenderSystem> GetBlenderSystem() const override;
	void SetBlenderSystem(TSubclassOf<UMovieSceneBlenderSystem> BlenderSystemClass) override;
	void GetSupportedBlenderSystems(TArray<TSubclassOf<UMovieSceneBlenderSystem>>& OutSystemClasses) const override;

#if WITH_EDITOR

	MOVIESCENETRACKS_API TArray<FTrajectoryKey> GetTrajectoryData(FFrameNumber Time, int32 MaxNumDataPoints, TRange<FFrameNumber>) const;
	MOVIESCENETRACKS_API FSlateColor GetLabelColor(const FMovieSceneLabelParams& LabelParams) const override;
#endif

private:

	/** User-defined blender system to use for this track */
	UPROPERTY()
	TSubclassOf<UMovieSceneBlenderSystem> BlenderSystemClass;
};
