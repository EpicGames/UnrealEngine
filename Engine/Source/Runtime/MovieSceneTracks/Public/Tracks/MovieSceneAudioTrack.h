// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneAudioTrack.generated.h"

class USoundBase;

namespace AudioTrackConstants
{
	const float ScrubDuration = 0.050f;
}


/**
 * Handles manipulation of audio.
 */
UCLASS(MinimalAPI)
class UMovieSceneAudioTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_UCLASS_BODY()

public:

	/** Adds a new sound cue to the audio */
	MOVIESCENETRACKS_API virtual UMovieSceneSection* AddNewSoundOnRow(USoundBase* Sound, FFrameNumber Time, int32 RowIndex);

	/** Adds a new sound cue on the next available/non-overlapping row */
	virtual UMovieSceneSection* AddNewSound(USoundBase* Sound, FFrameNumber Time) { return AddNewSoundOnRow(Sound, Time, INDEX_NONE); }

	/** @return The audio sections on this track */
	const TArray<UMovieSceneSection*>& GetAudioSections() const
	{
		return AudioSections;
	}

public:

	// UMovieSceneTrack interface

	MOVIESCENETRACKS_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENETRACKS_API virtual void RemoveAllAnimationData() override;
	MOVIESCENETRACKS_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	MOVIESCENETRACKS_API virtual void AddSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	MOVIESCENETRACKS_API virtual bool IsEmpty() const override;
	MOVIESCENETRACKS_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	MOVIESCENETRACKS_API virtual bool SupportsMultipleRows() const override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* CreateNewSection() override;

	MOVIESCENETRACKS_API virtual void GetCompatibleUserDecorationsImpl(TSet<UClass*>& OutClasses) const override;

	// ~UObject interface
	MOVIESCENETRACKS_API virtual void PostRename(UObject* OldOuter, const FName OldName) override;

private:

	/** List of all root audio sections */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> AudioSections;

#if WITH_EDITORONLY_DATA

public:

	/**
	 * Get the height of this track's rows
	 */
	int32 GetRowHeight() const
	{
		return RowHeight;
	}

	/**
	 * Set the height of this track's rows
	 */
	void SetRowHeight(int32 NewRowHeight)
	{
		RowHeight = FMath::Max(16, NewRowHeight);
	}

private:

	/** The height for each row of this track */
	UPROPERTY()
	int32 RowHeight;

#endif
};
