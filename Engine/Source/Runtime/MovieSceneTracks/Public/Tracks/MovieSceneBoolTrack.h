// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneBoolTrack.generated.h"

/**
 * Handles manipulation of float properties in a movie scene
 */
UCLASS( MinimalAPI )
class UMovieSceneBoolTrack
	: public UMovieScenePropertyTrack
	, public IMovieSceneTrackTemplateProducer
	, public IMovieSceneEntityProvider
{
	GENERATED_BODY()

public:

	// UMovieSceneTrack interface

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;

	//~ IMovieSceneEntityProvider interface
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;
};
