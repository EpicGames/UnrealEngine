// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCVarTrack.generated.h"

class UObject;

/**
 * Track for setting (and restoring) Console Variables during playback.
 */
UCLASS(MinimalAPI)
class UMovieSceneCVarTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_BODY()
	UMovieSceneCVarTrack( const FObjectInitializer& ObjectInitializer );
	
public:

	// UMovieSceneTrack interface
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual bool SupportsMultipleRows() const override;
	virtual EMovieSceneTrackEasingSupportFlags SupportsEasing(FMovieSceneSupportsEasingParams& Params) const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual void RemoveAllAnimationData() override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

private:

	/** All movie scene sections. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};
