// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/FrameNumber.h"
#include "Misc/Guid.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneObjectBindingID.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCameraCutTrack.generated.h"

class UMovieSceneCameraCutSection;
class UObject;

/**
 * Handles manipulation of CameraCut properties in a movie scene.
 */
UCLASS(MinimalAPI)
class UMovieSceneCameraCutTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_BODY()
	UMovieSceneCameraCutTrack( const FObjectInitializer& ObjectInitializer );
	
public:

	/** 
	 * Adds a new CameraCut at the specified time.
	 *	
	 * @param CameraBindingID Handle to the camera that the CameraCut switches to when active.
	 * @param Time The within this track's movie scene where the CameraCut is initially placed.
	 * @return The newly created camera cut section
	 */
	MOVIESCENETRACKS_API UMovieSceneCameraCutSection* AddNewCameraCut(const FMovieSceneObjectBindingID& CameraBindingID, FFrameNumber Time);

public:

	// UMovieSceneTrack interface
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual bool SupportsMultipleRows() const override;
	virtual EMovieSceneTrackEasingSupportFlags SupportsEasing(FMovieSceneSupportsEasingParams& Params) const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual void RemoveAllAnimationData() override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

	/** @return Whether camera cut sections should automatically resize to fill gaps */
	MOVIESCENETRACKS_API bool IsAutoManagingSections() const;

	/** Sets whether camera cut sections should automatically resize to fill gaps */
	MOVIESCENETRACKS_API void SetIsAutoManagingSections(bool bInAutoArrangeSections);

#if WITH_EDITOR
	virtual EMovieSceneSectionMovedResult OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params) override;
#endif

	MOVIESCENETRACKS_API void RearrangeAllSections();
	MOVIESCENETRACKS_API FFrameNumber FindEndTimeForCameraCut(FFrameNumber StartTime);

protected:

	virtual void PreCompileImpl(FMovieSceneTrackPreCompileResult& OutPreCompileResult) override;

	bool AutoArrangeSectionsIfNeeded(UMovieSceneSection& ChangedSection, bool bWasDeletion, bool bCleanUp = false);

public:
	UPROPERTY()
	bool bCanBlend;

private:

	/** All movie scene sections. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

	/** Whether camera cut sections should automatically resize to fill gaps */
	UPROPERTY()
	bool bAutoArrangeSections;
};

