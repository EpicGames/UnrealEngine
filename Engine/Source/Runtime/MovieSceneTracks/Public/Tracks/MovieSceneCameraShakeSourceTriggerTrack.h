// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "MovieSceneSection.h"
#include "MovieSceneTrack.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCameraShakeSourceTriggerTrack.generated.h"

class UObject;
struct FMovieSceneEvaluationTrack;

UCLASS(MinimalAPI)
class UMovieSceneCameraShakeSourceTriggerTrack
	: public UMovieSceneTrack
{
public:

	GENERATED_BODY()

	MOVIESCENETRACKS_API UMovieSceneCameraShakeSourceTriggerTrack(const FObjectInitializer& Obj);

	// UMovieSceneTrack interface
	MOVIESCENETRACKS_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* CreateNewSection() override;
	MOVIESCENETRACKS_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	MOVIESCENETRACKS_API virtual void AddSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	MOVIESCENETRACKS_API virtual void RemoveAllAnimationData() override;
	MOVIESCENETRACKS_API virtual bool IsEmpty() const override;
	MOVIESCENETRACKS_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;

#if WITH_EDITORONLY_DATA
	MOVIESCENETRACKS_API virtual FText GetDisplayName() const override;
#endif

protected:

	/** All the sections in this track */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};

