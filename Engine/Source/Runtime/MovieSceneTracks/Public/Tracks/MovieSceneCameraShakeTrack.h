// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Camera/CameraShakeBase.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/InlineValue.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCameraShakeTrack.generated.h"

class UCameraShakeBase;
class UObject;
struct FFrameNumber;
struct FMovieSceneEvaluationTrack;
struct FMovieSceneSegmentCompilerRules;

/**
 * 
 */
UCLASS(MinimalAPI)
class UMovieSceneCameraShakeTrack : public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:
	virtual UMovieSceneSection* AddNewCameraShake(FFrameNumber KeyTime, TSubclassOf<UCameraShakeBase> ShakeClass);
	
public:

	// UMovieSceneTrack interface
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool IsEmpty() const override;
	virtual bool SupportsMultipleRows() const override { return true; }
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual void RemoveAllAnimationData() override;
	
#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
#endif
	
private:
	/** List of all sections */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> CameraShakeSections;

};
