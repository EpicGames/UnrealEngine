// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/MovieSceneSegmentCompiler.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "Misc/FrameNumber.h"
#include "Misc/InlineValue.h"
#include "MovieSceneTrack.h"
#include "Templates/SubclassOf.h"
#include "Tracks/MovieSceneSubTrack.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneCinematicShotTrack.generated.h"

class UMovieSceneSection;
class UMovieSceneSequence;
class UMovieSceneSubSection;
class UObject;
struct FMovieSceneTrackEvaluationData;
template <typename> struct TMovieSceneEvaluationTree;

/**
* A track that holds consecutive sub sequences.
*/
UCLASS(MinimalAPI)
class UMovieSceneCinematicShotTrack
	: public UMovieSceneSubTrack
{
	GENERATED_BODY()

public:

	MOVIESCENETRACKS_API UMovieSceneCinematicShotTrack(const FObjectInitializer& ObjectInitializer);

	MOVIESCENETRACKS_API void SortSections();

	// UMovieSceneSubTrack interface
	virtual UMovieSceneSubSection* AddSequence(UMovieSceneSequence* Sequence, FFrameNumber StartTime, int32 Duration) { return AddSequenceOnRow(Sequence, StartTime, Duration, INDEX_NONE); }

	MOVIESCENETRACKS_API virtual UMovieSceneSubSection* AddSequenceOnRow(UMovieSceneSequence* Sequence, FFrameNumber StartTime, int32 Duration, int32 RowIndex);
	
	// UMovieSceneTrack interface
	MOVIESCENETRACKS_API virtual void AddSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* CreateNewSection() override;
	MOVIESCENETRACKS_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	MOVIESCENETRACKS_API virtual bool SupportsMultipleRows() const override;
	MOVIESCENETRACKS_API virtual bool PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const override;
	MOVIESCENETRACKS_API virtual int8 GetEvaluationFieldVersion() const override;
	
#if WITH_EDITOR
	MOVIESCENETRACKS_API virtual EMovieSceneSectionMovedResult OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params) override;
#endif

#if WITH_EDITORONLY_DATA
	MOVIESCENETRACKS_API virtual FText GetDefaultDisplayName() const override;
#endif
};
