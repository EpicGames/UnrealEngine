// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreTypes.h"
#include "Misc/FrameNumber.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneCommonAnimationTrack.generated.h"

#define UE_API MOVIESCENETRACKS_API

class UAnimSequenceBase;
class UMovieSceneSection;
class UMovieSceneSkeletalAnimationSection;

/**Struct to hold the cached root motion positions based upon how we calculated them.
* Also provides way to get the root motion at a particular time.
*/

USTRUCT()
struct FMovieSceneSkeletalAnimRootMotionTrackParams
{
	GENERATED_BODY()

	FFrameTime FrameTick; 
	FFrameTime StartFrame;
	FFrameTime EndFrame;
	bool bRootMotionsDirty;
	bool bHaveRootMotion;
	FTransform RootMotionStartOffset; // root motion may not be in Mesh Space if we are putting values on a bone that is a child of a root with an offset
	/** Get the Root Motion transform at the specified time.*/
	FMovieSceneSkeletalAnimRootMotionTrackParams() : bRootMotionsDirty(true),bHaveRootMotion(false) {}

#if WITH_EDITORONLY_DATA
	bool bCacheRootTransforms = false;
	TArray<FTransform> RootTransforms;
#endif
};

UCLASS(MinimalAPI)
class UMovieSceneCommonAnimationTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:

	UE_API UMovieSceneCommonAnimationTrack(const FObjectInitializer& ObjInit);

	/** Adds a new animation to this track */
	UE_API virtual UMovieSceneSection* AddNewAnimationOnRow(FFrameNumber KeyTime, UAnimSequenceBase* AnimSequence, int32 RowIndex);

	/** Adds a new animation to this track on the next available/non-overlapping row */
	 virtual UMovieSceneSection* AddNewAnimation(FFrameNumber KeyTime, UAnimSequenceBase* AnimSequence) { return AddNewAnimationOnRow(KeyTime, AnimSequence, INDEX_NONE); }

	/** Adds a section to the track with automatically computed row index. For mixer tracks, computes row based on existing sections and child tracks. */
	UE_API virtual void AddSectionWithAutoRow(UMovieSceneSection* Section);

	/** Computes the next available row index. For mixer tracks, considers both sections and child tracks. */
	UE_API virtual int32 ComputeNextAvailableRowIndex() const;

	/** Adds a child track of the specified class at the given row index. Only valid for mixer tracks. */
	virtual UMovieSceneTrack* AddChildTrack(FGuid ObjectBinding, TSubclassOf<UMovieSceneTrack> TrackClass, int32 RowIndex)
	{
		checkf(false, TEXT("AddChildTrack is only valid for UMovieSceneAnimationMixerTrack"));
		return nullptr;
	}

	/** Gets all child tracks. Only valid for mixer tracks. Base implementation returns empty array. */
	virtual void GetAllChildTracks(TArray<TObjectPtr<UMovieSceneTrack>>& OutChildTracks) const {}

	UE_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	UE_API virtual UMovieSceneSection* CreateNewSection() override;
	UE_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	UE_API virtual bool SupportsMultipleRows() const override;
	UE_API virtual void RemoveAllAnimationData() override;
	UE_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	UE_API virtual void AddSection(UMovieSceneSection& Section) override;
	UE_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	UE_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	UE_API virtual bool IsEmpty() const override;
	UE_API virtual void UpdateEasing() override;

#if WITH_EDITOR
	UE_API virtual void PostEditImport() override;
	UE_API virtual void PostEditUndo() override;
	UE_API virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	virtual bool ShouldUseRootMotions() const
	{
		return false;
	}

public:

	UE_API void SetRootMotionsDirty();
	UE_API void SetUpRootMotions(bool bForce);
	UE_API TOptional<FTransform>  GetRootMotion(FFrameTime CurrentTime);
	UE_API void MatchSectionByBoneTransform(bool bMatchWithPrevious, USkeletalMeshComponent* SkelMeshComp, UMovieSceneSkeletalAnimationSection* CurrentSection, FFrameTime CurrentFrame, FFrameRate InTickResolution,
		const FName& BoneName, FTransform& SecondSectionRootDiff, FVector& TranslationDiff, FQuat& RotationDiff); //add options for z and for rotation.
#if WITH_EDITORONLY_DATA
	UE_API void ToggleShowRootMotionTrail();
#endif

#if WITH_EDITOR
	UE_API virtual EMovieSceneSectionMovedResult OnSectionMoved(UMovieSceneSection& Section, const FMovieSceneSectionMovedParams& Params) override;
#endif

private:
	//Not called yet, will be used to automatch a section when it's added to another
	UE_API void AutoMatchSectionRoot(UMovieSceneSkeletalAnimationSection* AnimSection);

	UE_API void SortSections();

public:

	/** List of all animation sections */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> AnimationSections;

	UPROPERTY()
	FMovieSceneSkeletalAnimRootMotionTrackParams RootMotionParams;

	/** Whether to blend and adjust the first child node with animation instead of the root, this should be true for blending when the root is static, false if the animations have proper root motion*/
	UPROPERTY(EditAnywhere, Category = "RootMotions")
	bool bBlendFirstChildOfRoot;

#if WITH_EDITORONLY_DATA

	/** Whether to show the position of the root for this sections */
	UPROPERTY(EditAnywhere, Category = "RootMotions")
	bool bShowRootMotionTrail;

#endif
};

#undef UE_API
