// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sections/MovieSceneParameterSection.h"
#include "MovieSceneNameableTrack.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Materials/MaterialParameters.h"
#include "MovieSceneMaterialTrack.h"
#include "MovieSceneCustomPrimitiveDataTrack.generated.h"


#if WITH_EDITORONLY_DATA
/**
 * Used to cache mapping between custom primitive data indices and found material parameters using those indices.
 * This is used for nicer display names on channels animating those parameters.
*/
struct FCustomPrimitiveDataMaterialParametersData
{
	EMaterialParameterType MaterialParameterType;
	FComponentMaterialInfo MaterialInfo;
	FMaterialParameterInfo ParameterInfo;
	TWeakObjectPtr<UMaterialInterface> MaterialAsset;
};
#endif


/**
 * Handles manipulation of custom primitive data in a movie scene.
 */
UCLASS(MinimalAPI)
class UMovieSceneCustomPrimitiveDataTrack
	: public UMovieSceneNameableTrack
	, public IMovieSceneEntityProvider
	, public IMovieSceneParameterSectionExtender
{
	GENERATED_BODY()

public:

	MOVIESCENETRACKS_API UMovieSceneCustomPrimitiveDataTrack(const FObjectInitializer& ObjectInitializer);

	// UMovieSceneTrack interface

	MOVIESCENETRACKS_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* CreateNewSection() override;
	MOVIESCENETRACKS_API virtual void RemoveAllAnimationData() override;
	MOVIESCENETRACKS_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	MOVIESCENETRACKS_API virtual void AddSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	MOVIESCENETRACKS_API virtual bool IsEmpty() const override;
	MOVIESCENETRACKS_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;	
	MOVIESCENETRACKS_API virtual bool SupportsMultipleRows() const override;
	MOVIESCENETRACKS_API virtual void SetSectionToKey(UMovieSceneSection* Section) override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* GetSectionToKey() const override;

public:

	/**
	* Adds a scalar parameter key to the track. 
	* @param CustomPrimitiveDataStartIndex The start index for the custom primitive data
	* @param Time The time to add the new key.
	* @param The value for the new key.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddScalarParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, float Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	 * Adds a scalar parameter key to the track. 
	 * @param CustomPrimitiveDataStartIndex The start index for the custom primitive data
	 * @param Time The time to add the new key.
	 * @param RowIndex The preferred row index on which to look for sections.
	 * @param The value for the new key.
	 * @param DefaultInterpolation The default interpolation to use for thew new key.
	 */
	MOVIESCENETRACKS_API void AddScalarParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, int32 RowIndex, float Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);
	
	/**
	 * Adds a Vector2D parameter key to the track.
	 * @param CustomPrimitiveDataStartIndex The start index for the custom primitive data
	 * @param Time The time to add the new key.
	 * @param The value for the new key.
	 * @param DefaultInterpolation The default interpolation to use for thew new key.
	 */
	MOVIESCENETRACKS_API void AddVector2DParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, FVector2D Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	 * Adds a Vector2D parameter key to the track.
	 * @param CustomPrimitiveDataStartIndex The start index for the custom primitive data
	 * @param Time The time to add the new key.
	 * @param RowIndex The preferred row index on which to look for sections.
	 * @param The value for the new key.
	 * @param DefaultInterpolation The default interpolation to use for thew new key.
	 */
	MOVIESCENETRACKS_API void AddVector2DParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, int32 RowIndex, FVector2D Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	 * Adds a Vector parameter key to the track.
	 * @param CustomPrimitiveDataStartIndex The start index for the custom primitive data
	 * @param Time The time to add the new key.
	 * @param The value for the new key.
	 * @param DefaultInterpolation The default interpolation to use for thew new key.
	 */
	MOVIESCENETRACKS_API void AddVectorParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, FVector Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	 * Adds a Vector parameter key to the track.
	 * @param CustomPrimitiveDataStartIndex The start index for the custom primitive data
	 * @param Time The time to add the new key.
	 * @param RowIndex The preferred row index on which to look for sections.
	 * @param The value for the new key.
	 * @param DefaultInterpolation The default interpolation to use for thew new key.
	 */
	MOVIESCENETRACKS_API void AddVectorParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, int32 RowIndex, FVector Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a Vector2D parameter key to the track.
	* @param CustomPrimitiveDataStartIndex The start index for the custom primitive data
	* @param Time The time to add the new key.
	* @param The value for the new key.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddColorParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, FLinearColor Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a Vector2D parameter key to the track.
	* @param CustomPrimitiveDataStartIndex The start index for the custom primitive data
	* @param Time The time to add the new key.
	* @param RowIndex The preferred row index on which to look for sections.
	* @param The value for the new key.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddColorParameterKey(uint8 CustomPrimitiveDataStartIndex, FFrameNumber Time, int32 RowIndex, FLinearColor Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/*~ IMovieSceneEntityProvider */
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	/*~ IMovieSceneParameterSectionExtender */
	virtual void ExtendEntityImpl(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity) override;

#if WITH_EDITORONLY_DATA

	/*
	* Returns the mappings from custom primitive data indices to any material parameters on the bound object.
	* @param Player Movie scene player
	* @param BoundObjectID Bound object ID for the component this track is on
	* @param SequenceID The sequence ID this track is on
	*/
	MOVIESCENETRACKS_API void GetCPDMaterialData(IMovieScenePlayer& Player, FGuid BoundObjectId, FMovieSceneSequenceID SequenceID, TSortedMap<uint8, TArray<FCustomPrimitiveDataMaterialParametersData>>& OutCPDMaterialData);


public:

	FText GetDefaultDisplayName() const override;

private:

#endif

private:

	/** The sections owned by this track .*/
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

	/** Section we should Key */
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> SectionToKey;
};