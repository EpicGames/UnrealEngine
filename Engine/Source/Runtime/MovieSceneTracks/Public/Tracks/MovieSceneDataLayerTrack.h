// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "CoreTypes.h"
#include "Internationalization/Text.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneDataLayerTrack.generated.h"

class UObject;

/**
 * A track that controls loading, unloading and visibility of data layers
 */
UCLASS( MinimalAPI )
class UMovieSceneDataLayerTrack : public UMovieSceneNameableTrack
{
public:
	GENERATED_BODY()

	UMovieSceneDataLayerTrack(const FObjectInitializer& ObjectInitializer);

	/** UMovieSceneTrack interface */
	virtual bool IsEmpty() const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection( UMovieSceneSection& Section ) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool SupportsMultipleRows() const override { return true; }

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

private:

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};
