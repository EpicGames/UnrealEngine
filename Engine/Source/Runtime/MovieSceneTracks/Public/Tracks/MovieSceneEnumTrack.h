// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieScene.h"
#include "MovieSceneTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneEnumTrack.generated.h"

class UEnum;

/**
 * Handles manipulation of byte properties in a movie scene
 */
UCLASS(MinimalAPI)
class UMovieSceneEnumTrack : public UMovieScenePropertyTrack
{
	GENERATED_UCLASS_BODY()

public:
	/** UMovieSceneTrack interface */
	MOVIESCENETRACKS_API virtual void PostLoad() override;
	MOVIESCENETRACKS_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* CreateNewSection() override;

	MOVIESCENETRACKS_API void SetEnum(UEnum* Enum);

	MOVIESCENETRACKS_API class UEnum* GetEnum() const;

protected:
	UPROPERTY()
	TObjectPtr<UEnum> Enum;
};
