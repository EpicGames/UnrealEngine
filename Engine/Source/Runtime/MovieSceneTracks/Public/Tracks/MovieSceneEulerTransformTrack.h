// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneEulerTransformTrack.generated.h"

/**
 * Handles manipulation of 3D euler transform properties in a movie scene
 */
UCLASS(MinimalAPI)
class UMovieSceneEulerTransformTrack
	: public UMovieScenePropertyTrack
{
	GENERATED_UCLASS_BODY()

public:

	// UMovieSceneTrack interface

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

};

