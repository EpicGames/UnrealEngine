// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneNameableTrack.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "MovieSceneObjectBindingID.h"
#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "Compilation/IMovieSceneDeterminismSource.h"
#include "MovieSceneEventTrack.generated.h"

struct FMovieSceneEvaluationTrack;

/** Indicates at what point in the sequence evaluation events should fire */
UENUM()
enum class EFireEventsAtPosition : uint8
{
	/** Fire events before anything else is evaluated in the sequence */
	AtStartOfEvaluation,
	/** Fire events after everything else has been evaluated in the sequence */
	AtEndOfEvaluation,
	/** Fire events right after any spawn tracks have been evaluated */
	AfterSpawn,
};

/**
 * Implements a movie scene track that triggers discrete events during playback.
 */
UCLASS(MinimalAPI)
class UMovieSceneEventTrack
	: public UMovieSceneNameableTrack
	, public IMovieSceneTrackTemplateProducer
	, public IMovieSceneDeterminismSource
{
	GENERATED_BODY()

public:

	/** Default constructor. */
	UMovieSceneEventTrack()
		: bFireEventsWhenForwards(true)
		, bFireEventsWhenBackwards(true)
		, EventPosition(EFireEventsAtPosition::AtEndOfEvaluation)
	{
#if WITH_EDITORONLY_DATA
		TrackTint = FColor(41, 98, 41, 150);
#endif
	}

public:

	// UMovieSceneTrack interface

	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveAllAnimationData() override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual EMovieSceneCompileResult CustomCompile(FMovieSceneEvaluationTrack& Track, const FMovieSceneTrackCompilerArgs& Args) const override;
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
	virtual void PostCompile(FMovieSceneEvaluationTrack& Track, const FMovieSceneTrackCompilerArgs& Args) const override;
	virtual bool SupportsMultipleRows() const override { return true; }

	virtual void PopulateDeterminismData(FMovieSceneDeterminismData& OutData, const TRange<FFrameNumber>& Range) const override;

#if WITH_EDITORONLY_DATA
	virtual void Serialize(FArchive& Ar) override;
	virtual void PostRename(UObject* OldOuter, const FName OldName) override;
#endif

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

public:

	/** If events should be fired when passed playing the sequence forwards. */
	UPROPERTY(EditAnywhere, Category=TrackEvent)
	uint32 bFireEventsWhenForwards:1;

	/** If events should be fired when passed playing the sequence backwards. */
	UPROPERTY(EditAnywhere, Category=TrackEvent)
	uint32 bFireEventsWhenBackwards:1;

	/** Defines where in the evaluation to trigger events */
	UPROPERTY(EditAnywhere, Category=TrackEvent)
	EFireEventsAtPosition EventPosition;

private:
	
	/** The track's sections. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};
