// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "MovieSceneFadeTrack.generated.h"

/**
 * Implements a movie scene track that controls a fade.
 */
UCLASS(MinimalAPI)
class UMovieSceneFadeTrack
	: public UMovieSceneFloatTrack
{
	GENERATED_BODY()

public:

	UMovieSceneFadeTrack(const FObjectInitializer& Init);

	// UMovieSceneTrack interface

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
	virtual bool CanRename() const override { return true; }
#endif
};
