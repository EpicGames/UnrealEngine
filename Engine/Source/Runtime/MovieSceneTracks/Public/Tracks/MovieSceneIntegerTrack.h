// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieScene.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneIntegerTrack.generated.h"

/**
 * Handles manipulation of integer properties in a movie scene
 */
UCLASS(MinimalAPI)
class UMovieSceneIntegerTrack : public UMovieScenePropertyTrack
{
	GENERATED_UCLASS_BODY()

public:
	/** UMovieSceneTrack interface */
	MOVIESCENETRACKS_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* CreateNewSection() override;
};
