// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneNameableTrack.h"
#include "Tracks/MovieSceneSpawnTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneLevelVisibilityTrack.generated.h"

struct FMovieSceneEvaluationTrack;

/**
 * A track for controlling the visibility of streamed levels.
 */
UCLASS( MinimalAPI )
class UMovieSceneLevelVisibilityTrack : public UMovieSceneNameableTrack
{
	GENERATED_UCLASS_BODY()

public:

	static uint16 GetEvaluationPriority() { return UMovieSceneSpawnTrack::GetEvaluationPriority() + 100; }

	/** UMovieSceneTrack interface */
	virtual bool IsEmpty() const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection( UMovieSceneSection& Section ) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool SupportsMultipleRows() const override { return true; }

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

private:

	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};
