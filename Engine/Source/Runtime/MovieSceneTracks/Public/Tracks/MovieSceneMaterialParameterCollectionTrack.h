// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieSceneMaterialTrack.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "MovieSceneMaterialParameterCollectionTrack.generated.h"

class UMaterialParameterCollection;

/**
 * Handles manipulation of material parameter collections in a movie scene.
 */
UCLASS(MinimalAPI)
class UMovieSceneMaterialParameterCollectionTrack
	: public UMovieSceneMaterialTrack
	, public IMovieSceneEntityProvider
	, public IMovieSceneParameterSectionExtender
{
public:

	GENERATED_BODY()

	/** The material parameter collection to manipulate */
	UPROPERTY(EditAnywhere, Category=General, DisplayName="Material Parameter Collection")
	TObjectPtr<UMaterialParameterCollection> MPC;

	MOVIESCENETRACKS_API UMovieSceneMaterialParameterCollectionTrack(const FObjectInitializer& ObjectInitializer);

	MOVIESCENETRACKS_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* CreateNewSection() override;

	/*~ IMovieSceneEntityProvider */
	MOVIESCENETRACKS_API virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	MOVIESCENETRACKS_API virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	/*~ IMovieSceneParameterSectionExtender */
	MOVIESCENETRACKS_API virtual void ExtendEntityImpl(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity) override;

#if WITH_EDITORONLY_DATA
	MOVIESCENETRACKS_API virtual FText GetDefaultDisplayName() const override;
#endif

protected:
	//~ UObject interface
#if WITH_EDITOR
	MOVIESCENETRACKS_API virtual void PostLoad() override;
#endif

};
