// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Sections/MovieSceneParameterSection.h"
#include "MovieSceneNameableTrack.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "Materials/MaterialParameters.h"
#if WITH_EDITOR
#include "Styling/SlateColor.h"
#endif
#include "MovieSceneMaterialTrack.generated.h"

UENUM(BlueprintType)
enum class EComponentMaterialType : uint8
{
	/* Empty/Uninitialized*/
	Empty,
	/* A material in one of the indexed slots on a primitive component*/
	IndexedMaterial,
	/* An overlay material on a mesh component*/
	OverlayMaterial,
	/* A decal material*/
	DecalMaterial,
	/* Volumetric Cloud Material*/
	VolumetricCloudMaterial,
	/* A per-slot overlay material on a mesh component */
	SlotOverlayMaterial,
};

/**
 * Contains what is necessary to uniquely identify a material on a component, whether that be an indexed material, one with a slot name, or an overlay material.
 */
USTRUCT(BlueprintType)
struct FComponentMaterialInfo
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category="Sequencer|Material")
	FName MaterialSlotName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sequencer|Material")
	int32 MaterialSlotIndex = 0;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Sequencer|Material")
	EComponentMaterialType MaterialType = EComponentMaterialType::Empty;

	friend uint32 GetTypeHash(const FComponentMaterialInfo& In)
	{
		return GetTypeHash(In.MaterialSlotName) ^ ::GetTypeHash(In.MaterialSlotIndex) ^ ::GetTypeHash(In.MaterialType);
	}
	friend bool operator==(const FComponentMaterialInfo& A, const FComponentMaterialInfo& B)
	{
		return A.MaterialSlotIndex == B.MaterialSlotIndex && A.MaterialSlotName == B.MaterialSlotName && A.MaterialType == B.MaterialType;
	}

	FString ToString() const 
	{
		switch (MaterialType)
		{
		case EComponentMaterialType::IndexedMaterial:
			if (!MaterialSlotName.IsNone())
			{
				return FString::Printf(TEXT("Material Slot: %s"), *MaterialSlotName.ToString());
			}
			else
			{
				return FString::Printf(TEXT("Material Element %i"), MaterialSlotIndex);
			}
			break;
		case EComponentMaterialType::OverlayMaterial:
			return TEXT("Overlay Material");
			break;
		case EComponentMaterialType::DecalMaterial:
			return TEXT("Decal Material");
			break;
		case EComponentMaterialType::SlotOverlayMaterial:
			if (!MaterialSlotName.IsNone())
			{
				return FString::Printf(TEXT("Overlay Material Slot: %s"), *MaterialSlotName.ToString());
			}
			else
			{
				return FString::Printf(TEXT("Overlay Material Element %i"), MaterialSlotIndex);
			}
			break;
		}
		return FString();
	}
};

/**
 * Handles manipulation of material parameters in a movie scene.
 */
UCLASS(MinimalAPI)
class UMovieSceneMaterialTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_BODY()

public:

	MOVIESCENETRACKS_API UMovieSceneMaterialTrack(const FObjectInitializer& ObjectInitializer);

	// UMovieSceneTrack interface

	MOVIESCENETRACKS_API virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* CreateNewSection() override;
	MOVIESCENETRACKS_API virtual void RemoveAllAnimationData() override;
	MOVIESCENETRACKS_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	MOVIESCENETRACKS_API virtual void AddSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	MOVIESCENETRACKS_API virtual bool IsEmpty() const override;
	MOVIESCENETRACKS_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;	
	MOVIESCENETRACKS_API virtual bool SupportsMultipleRows() const override;
	MOVIESCENETRACKS_API virtual void SetSectionToKey(UMovieSceneSection* Section) override;
	MOVIESCENETRACKS_API virtual UMovieSceneSection* GetSectionToKey() const override;

public:

	/**
	* Adds a scalar parameter key to the track. 
	* @param ParameterName The name of the parameter to add a key for.
	* @param Time The time to add the new key.
	* @param The value for the new key.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddScalarParameterKey(FName ParameterName, FFrameNumber Time, float Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	 * Adds a scalar parameter key to the track. 
	 * @param ParameterName The name of the parameter to add a key for.
	 * @param Time The time to add the new key.
	 * @param RowIndex The preferred row index on which to look for sections.
	 * @param The value for the new key.
	 * @param DefaultInterpolation The default interpolation to use for thew new key.
	 */
	MOVIESCENETRACKS_API void AddScalarParameterKey(FName ParameterName, FFrameNumber Time, int32 RowIndex, float Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a scalar parameter key to the track.
	* @param ParameterInfo The material parameter info for the parameter you want to add a key for.
	* @param Time The time to add the new key.
	* @param The value for the new key.
	* @param InLayerName Optional layer name for use in UI.
	* @param InAssetName Optional asset name for use in UI.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddScalarParameterKey(const FMaterialParameterInfo& ParameterInfo, FFrameNumber Time, float Value, const FString& InLayerName, const FString& InAssetName, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a scalar parameter key to the track.
	* @param ParameterInfo The material parameter info for the parameter you want to add a key for.
	* @param Time The time to add the new key.
	* @param RowIndex The preferred row index on which to look for sections.
	* @param The value for the new key.
	* @param InLayerName Optional layer name for use in UI.
	* @param InAssetName Optional asset name for use in UI.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddScalarParameterKey(const FMaterialParameterInfo& ParameterInfo, FFrameNumber Time, int32 RowIndex, float Value, const FString& InLayerName, const FString& InAssetName, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a color parameter key to the track.
	* @param ParameterName The name of the parameter to add a key for.
	* @param Time The time to add the new key.
	* @param The value for the new key.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddColorParameterKey(FName ParameterName, FFrameNumber Time, FLinearColor Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a color parameter key to the track.
	* @param ParameterInfo The material parameter info for the parameter you want to add a key for.
	* @param Time The time to add the new key.
	* @param RowIndex The preferred row index on which to look for sections.
	* @param The value for the new key.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddColorParameterKey(FName ParameterName, FFrameNumber Time, int32 RowIndex, FLinearColor Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a color parameter key to the track.
	* @param ParameterName The name of the parameter to add a key for.
	* @param Time The time to add the new key.
	* @param The value for the new key.
	* @param InLayerName Optional layer name for use in UI.
	* @param InAssetName Optional asset name for use in UI.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddColorParameterKey(const FMaterialParameterInfo& ParameterInfo, FFrameNumber Time, FLinearColor Value, const FString& InLayerName, const FString& InAssetName, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a color parameter key to the track.
	* @param ParameterInfo The material parameter info for the parameter you want to add a key for.
	* @param Time The time to add the new key.
	* @param RowIndex The preferred row index on which to look for sections.
	* @param The value for the new key.
	* @param InLayerName Optional layer name for use in UI.
	* @param InAssetName Optional asset name for use in UI.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddColorParameterKey(const FMaterialParameterInfo& ParameterInfo, FFrameNumber Time, int32 RowIndex, FLinearColor Value, const FString& InLayerName, const FString& InAssetName, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a color parameter key to the track.
	* @param ParameterInfo The material parameter info for the parameter you want to add a key for.
	* @param Time The time to add the new key.
	* @param RowIndex The preferred row index on which to look for sections.
	* @param The value for the new key.
	* @param InLayerName Optional layer name for use in UI.
	* @param InAssetName Optional asset name for use in UI.
	* @param InChannelNames Optional channel names for the 4 vector channels for use in the UI.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	MOVIESCENETRACKS_API void AddColorParameterKey(const FMaterialParameterInfo& ParameterInfo, FFrameNumber Time, int32 RowIndex, FLinearColor Value, const FString& InLayerName, const FString& InAssetName, const FParameterChannelNames& InChannelNames, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

private:

	/** The sections owned by this track .*/
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;

	/** Section we should Key */
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> SectionToKey;
};


/**
 * A material track which is specialized for animation materials which are owned by actor components.
 */
UCLASS(MinimalAPI)
class UMovieSceneComponentMaterialTrack
	: public UMovieSceneMaterialTrack
	, public IMovieSceneEntityProvider
	, public IMovieSceneParameterSectionExtender
{
	GENERATED_UCLASS_BODY()

public:

	/*~ IMovieSceneEntityProvider */
	virtual void ImportEntityImpl(UMovieSceneEntitySystemLinker* EntityLinker, const FEntityImportParams& Params, FImportedEntity* OutImportedEntity) override;
	virtual bool PopulateEvaluationFieldImpl(const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder) override;

	/*~ IMovieSceneParameterSectionExtender */
	virtual void ExtendEntityImpl(UMovieSceneParameterSection* Section, UMovieSceneEntitySystemLinker* EntityLinker, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity) override;


#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif
public:

	/** Gets the index of the material in the component. */
	const FComponentMaterialInfo& GetMaterialInfo() const { return MaterialInfo; }

	/** Sets the index of the material in the component. */
	void SetMaterialInfo(const FComponentMaterialInfo& InMaterialInfo)
	{
		MaterialInfo = InMaterialInfo;
	}

#if WITH_EDITOR
	virtual FText GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const override;

	// We override label color if material binding is broken/partially broken.
	virtual FSlateColor GetLabelColor(const FMovieSceneLabelParams& LabelParams) const override;
#endif


#if WITH_EDITORONLY_DATA

protected:
	void PostLoad() override;

private:
	/** The index of this material this track is animating. Has been deprecated in favor of MaterialInfo*/
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use MaterialInfo instead."))
	int32 MaterialIndex_DEPRECATED;

#endif
private:
	/** The info on the material this track is animating.*/
	UPROPERTY()
	FComponentMaterialInfo MaterialInfo;

};
