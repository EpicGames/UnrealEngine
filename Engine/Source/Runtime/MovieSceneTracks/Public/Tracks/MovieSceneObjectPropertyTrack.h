// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneObjectPropertyTrack.generated.h"

UCLASS(MinimalAPI)
class UMovieSceneObjectPropertyTrack : public UMovieScenePropertyTrack
{
public:

	GENERATED_BODY()

	UPROPERTY()
	TObjectPtr<UClass> PropertyClass;

#if WITH_EDITORONLY_DATA
	UPROPERTY()
	bool bClassProperty = false;
#endif

	UMovieSceneObjectPropertyTrack(const FObjectInitializer& ObjInit);

	/*~ UMovieSceneTrack interface */
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
};
