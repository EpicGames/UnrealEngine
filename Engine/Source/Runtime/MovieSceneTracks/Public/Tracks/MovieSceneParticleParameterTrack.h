// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Internationalization/Text.h"
#include "Math/Color.h"
#include "Math/UnrealMathSSE.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "Sections/MovieSceneParameterSection.h"
#include "Templates/SubclassOf.h"
#include "UObject/NameTypes.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneParticleParameterTrack.generated.h"

class UObject;
struct FFrameNumber;

/**
 * Handles manipulation of material parameters in a movie scene.
 */
UCLASS( MinimalAPI )
class UMovieSceneParticleParameterTrack
	: public UMovieSceneNameableTrack
	, public IMovieSceneTrackTemplateProducer
{
	GENERATED_UCLASS_BODY()

public:

	// UMovieSceneTrack interface

	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual void RemoveAllAnimationData() override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool IsEmpty() const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif


public:

	/**
	 * Adds a scalar parameter key to the track. 
	 * @param ParameterName The name of the parameter to add a key for.
	 * @param Time The time to add the new key.
	 * @param The value for the new key.
	 * @param DefaultInterpolation The default interpolation to use for thew new key.
	 */
	void MOVIESCENETRACKS_API AddScalarParameterKey( FName ParameterName, FFrameNumber Time, float Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a Vector parameter key to the track.
	* @param ParameterName The name of the parameter to add a key for.
	* @param Time The time to add the new key.
	* @param The value for the new key.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	void MOVIESCENETRACKS_API AddVectorParameterKey( FName ParameterName, FFrameNumber Time, FVector Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

	/**
	* Adds a color parameter key to the track.
	* @param ParameterName The name of the parameter to add a key for.
	* @param Time The time to add the new key.
	* @param The value for the new key.
	* @param DefaultInterpolation The default interpolation to use for thew new key.
	*/
	void MOVIESCENETRACKS_API AddColorParameterKey( FName ParameterName, FFrameNumber Time, FLinearColor Value, EMovieSceneKeyInterpolation DefaultInterpolation = EMovieSceneKeyInterpolation::Auto);

private:

	/** The sections owned by this track .*/
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};
