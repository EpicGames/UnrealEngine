// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Compilation/IMovieSceneTrackTemplateProducer.h"
#include "Containers/Array.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Evaluation/MovieSceneEvalTemplate.h"
#include "Internationalization/Text.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneSection.h"
#include "Templates/SubclassOf.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "MovieSceneParticleTrack.generated.h"

class UObject;
struct FFrameNumber;

/**
 * Handles triggering of particle emitters
 */
UCLASS(MinimalAPI)
class UMovieSceneParticleTrack
	: public UMovieSceneNameableTrack
	, public IMovieSceneTrackTemplateProducer
{
	GENERATED_UCLASS_BODY()

public:

	/**
	 * Get the track's particle sections.
	 *
	 * @return Particle sections collection.
	 */
	virtual TArray<UMovieSceneSection*> GetAllParticleSections() const
	{
		return ParticleSections;
	}

public:

	// UMovieSceneTrack interface

	virtual void RemoveAllAnimationData() override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	virtual bool IsEmpty() const override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual void AddNewSection(FFrameNumber SectionTime);
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

	// ~IMovieSceneTrackTemplateProducer interface
	virtual FMovieSceneEvalTemplatePtr CreateTemplateForSection(const UMovieSceneSection& InSection) const override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

private:

	/** List of all particle sections. */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> ParticleSections;
};
