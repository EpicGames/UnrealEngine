// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "MovieSceneTrack.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneMaterialTrack.h"
#include "MovieScenePrimitiveMaterialTrack.generated.h"

UCLASS(MinimalAPI)
class UMovieScenePrimitiveMaterialTrack : public UMovieScenePropertyTrack
{
public:

	GENERATED_BODY()

	UMovieScenePrimitiveMaterialTrack(const FObjectInitializer& ObjInit);

	/* Set the material index that this track is assigned to */
	UE_DEPRECATED(5.4, "SetMaterialIndex is deprecated, use SetMaterialInfo instead")
	MOVIESCENETRACKS_API void SetMaterialIndex(int32 MaterialIndex);

	/* Get the material index that this track is assigned to */
	UE_DEPRECATED(5.4, "GetMaterialIndex is deprecated, use GetMaterialInfo instead")
	MOVIESCENETRACKS_API int32 GetMaterialIndex() const;

	/* Set the material info for the material that this track is assigned to */
	MOVIESCENETRACKS_API void SetMaterialInfo(const FComponentMaterialInfo& InMaterialInfo);

	/* Get the material info for the material that this track is assigned to */
	MOVIESCENETRACKS_API const FComponentMaterialInfo& GetMaterialInfo() const;

	/*~ UMovieSceneTrack interface */
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;

#if WITH_EDITOR
	virtual FText GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const override;

	// We override label color if material binding is broken/partially broken.
	virtual FSlateColor GetLabelColor(const FMovieSceneLabelParams& LabelParams) const override;
#endif

protected:
#if WITH_EDITORONLY_DATA
	void PostLoad() override;

private:
	UPROPERTY()
	int32 MaterialIndex_DEPRECATED;

#endif

private:
	UPROPERTY()
	FComponentMaterialInfo MaterialInfo;
};
