// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Channels/IMovieSceneChannelOverrideProvider.h"
#include "Channels/MovieSceneSectionChannelOverrideRegistry.h"
#include "EntitySystem/BuiltInComponentTypes.h"
#include "EntitySystem/IMovieSceneEntityProvider.h"
#include "EntitySystem/MovieSceneEntityBuilder.h"
#include "EntitySystem/MovieScenePropertyBinding.h"
#include "MovieSceneCommonHelpers.h"
#include "MovieSceneNameableTrack.h"
#include "MovieSceneTracksComponentTypes.h"
#include "Systems/MovieScenePiecewiseBoolBlenderSystem.h"
#include "UObject/ObjectMacros.h"
#include "MovieScenePropertyTrack.generated.h"

namespace UE::MovieScene
{
	struct FPropertyDefinition;
	struct FSourcePropertyValue;
}

/**
 * Base class for tracks that animate an object property
 */
UCLASS(abstract, MinimalAPI)
class UMovieScenePropertyTrack
	: public UMovieSceneNameableTrack
{
	GENERATED_UCLASS_BODY()

public:

	// UMovieSceneTrack interface

	MOVIESCENETRACKS_API virtual void RemoveAllAnimationData() override;
	MOVIESCENETRACKS_API virtual bool HasSection(const UMovieSceneSection& Section) const override;
	MOVIESCENETRACKS_API virtual void AddSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSection(UMovieSceneSection& Section) override;
	MOVIESCENETRACKS_API virtual void RemoveSectionAt(int32 SectionIndex) override;
	MOVIESCENETRACKS_API virtual bool IsEmpty() const override;
	MOVIESCENETRACKS_API virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual void InitializeFromProperty(const FProperty* Property, const UE::MovieScene::FPropertyDefinition* Definition)
	{
	}

#if WITH_EDITOR
	/**
	 * Allow specialized property tracks to process a property value before it is used to generate keys.
	 *
	 * UMovieSceneVisibilityTrack overrides this to invert bool values: the bound property (e.g. AActor::bHidden)
	 * stores hidden semantics while the track's bool channel stores visible semantics, so the value must be
	 * inverted at key time.
	 *
	 * @param Property    The property whose value is being keyed.
	 * @param InOutValue  The source property value, transformed in place.
	 */
	virtual void ProcessSourceValueForKeying(const FProperty* Property, UE::MovieScene::FSourcePropertyValue& InOutValue) const {}
#endif

#if WITH_EDITORONLY_DATA
	MOVIESCENETRACKS_API virtual FText GetDefaultDisplayName() const override;
	MOVIESCENETRACKS_API virtual FText GetDisplayNameToolTipText(const FMovieSceneLabelParams& LabelParams) const override;
	MOVIESCENETRACKS_API virtual FSlateColor GetLabelColor(const FMovieSceneLabelParams& LabelParams) const override;
#endif
	MOVIESCENETRACKS_API virtual FName GetTrackName() const override;

	MOVIESCENETRACKS_API virtual void PostLoad() override;
	MOVIESCENETRACKS_API virtual void Serialize(FArchive& Ar) override;

public:

	/**
	 * Sets the property name for this animatable property
	 *
	 * @param InPropertyName The property being animated
	 */
	MOVIESCENETRACKS_API void SetPropertyNameAndPath(FName InPropertyName, const FString& InPropertyPath);

	/** @return the name of the property being animated by this track */
	FName GetPropertyName() const { return PropertyBinding.PropertyName; }

	/** @return The property path for this track */
	FName GetPropertyPath() const { return PropertyBinding.PropertyPath; }

	/** Access the property binding for this track */
	const FMovieScenePropertyBinding& GetPropertyBinding() const { return PropertyBinding; }

	template <typename ValueType>
	TOptional<ValueType> GetCurrentValue(const UObject* Object) const
	{
		return FTrackInstancePropertyBindings::StaticValue<ValueType>(Object, PropertyBinding.PropertyPath.ToString());
	}

	/**
	* Find all sections at the current time.
	*
	*@param Time  The Time relative to the owning movie scene where the section should be
	*@Return All sections at that time
	*/
	MOVIESCENETRACKS_API TArray<UMovieSceneSection*, TInlineAllocator<4>> FindAllSections(FFrameNumber Time);

	/**
	 * Finds a section at the current time.
	 *
	 * @param Time The time relative to the owning movie scene where the section should be
	 * @return The found section.
	 */
	MOVIESCENETRACKS_API class UMovieSceneSection* FindSection(FFrameNumber Time);

	/**
	 * Finds a section at the current time or extends an existing one
	 *
	 * @param Time The time relative to the owning movie scene where the section should be
	 * @param OutWeight The weight of the section if found
	 * @return The found section.
	 */
	MOVIESCENETRACKS_API class UMovieSceneSection* FindOrExtendSection(FFrameNumber Time, float& OutWeight);

	/**
	 * Finds a section at the current time.
	 *
	 * @param Time The time relative to the owning movie scene where the section should be
	 * @param bSectionAdded Whether a section was added or not
	 * @return The found section, or the new section.
	 */
	MOVIESCENETRACKS_API class UMovieSceneSection* FindOrAddSection(FFrameNumber Time, bool& bSectionAdded);

	/**
	 * Set the section we want to key and recieve globally changed values.
	 *
	 * @param Section The section that changes.
	 */
	MOVIESCENETRACKS_API virtual void SetSectionToKey(UMovieSceneSection* Section) override;

	/**
	 * Finds a section we want to key and recieve globally changed values.
	 * @return The Section that changes.
	 */
	MOVIESCENETRACKS_API virtual UMovieSceneSection* GetSectionToKey() const override;

#if WITH_EDITORONLY_DATA
public:
	/** Unique name for this track to afford multiple tracks on a given object (i.e. for array properties) */
	UE_DEPRECATED(5.6, "UniqueTrackName is deprecated, use GetTrackName instead.")
	UPROPERTY()
	FName UniqueTrackName;

	/** Name of the property being changed */
	UPROPERTY()
	FName PropertyName_DEPRECATED;

	/** Path to the property from the source object being changed */
	UPROPERTY()
	FString PropertyPath_DEPRECATED;

#endif


private:
	/** Section we should Key */
	UPROPERTY()
	TObjectPtr<UMovieSceneSection> SectionToKey;

protected:

	UPROPERTY()
	FMovieScenePropertyBinding PropertyBinding;

	/** All the sections in this list */
	UPROPERTY()
	TArray<TObjectPtr<UMovieSceneSection>> Sections;
};


struct FMovieScenePropertyTrackEntityImportHelper
{
	static MOVIESCENETRACKS_API const int32 SectionPropertyValueImportingID;
	static MOVIESCENETRACKS_API const int32 SectionEditConditionToggleImportingID;

	static MOVIESCENETRACKS_API void PopulateEvaluationField(UMovieSceneSection& Section, const TRange<FFrameNumber>& EffectiveRange, const FMovieSceneEvaluationFieldEntityMetaData& InMetaData, FMovieSceneEntityComponentFieldBuilder* OutFieldBuilder);

	static MOVIESCENETRACKS_API bool IsPropertyValueID(const UE::MovieScene::FEntityImportParams& Params);
	static MOVIESCENETRACKS_API bool IsEditConditionToggleID(const UE::MovieScene::FEntityImportParams& Params);
	static MOVIESCENETRACKS_API void ImportEditConditionToggleEntity(const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity);

	static MOVIESCENETRACKS_API FName SanitizeBoolPropertyName(FName InPropertyName);
};


namespace UE
{
namespace MovieScene
{

/**
 * Utility class for importing a customizable property track entity in a way that automatically supports
 * being inside a bound property track or not, and being hooked up to a property with an edit condition or not.
 */
template<typename... T>
struct TPropertyTrackEntityImportHelperImpl
{
	TPropertyTrackEntityImportHelperImpl(TEntityBuilder<T...>&& InBuilder, FComponentTypeID InPropertyTag = {})
		: Builder(MoveTemp(InBuilder))
		, PropertyTag(InPropertyTag)
	{
	}

	template<typename U, typename PayloadType>
	TPropertyTrackEntityImportHelperImpl<T..., TAdd<U>> Add(TComponentTypeID<U> ComponentType, PayloadType&& InPayload)
	{
		return TPropertyTrackEntityImportHelperImpl<T..., TAdd<U>>(Builder.Add(ComponentType, InPayload), PropertyTag);
	}

	template<typename U, typename PayloadType>
	TPropertyTrackEntityImportHelperImpl<T..., TAddConditional<U>> AddConditional(TComponentTypeID<U> ComponentType, PayloadType&& InPayload, bool bCondition)
	{
		return TPropertyTrackEntityImportHelperImpl<T..., TAddConditional<U>>(Builder.AddConditional(ComponentType, InPayload, bCondition), PropertyTag);
	}

	void Commit(const UMovieSceneSection* InSection, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity)
	{
		const FGuid ObjectBindingID = Params.GetObjectBindingID();
		const FMovieSceneEvaluationFieldEntityMetaData* MetaData = Params.EntityMetaData;
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		if (UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(InSection->GetOuter()))
		{
			if (FMovieScenePropertyTrackEntityImportHelper::IsPropertyValueID(Params))
			{
				const FName NotifyFunctionName = (MetaData ? MetaData->NotifyFunctionName : NAME_None);

				OutImportedEntity->AddBuilder(
					Builder
					.Add(BuiltInComponents->PropertyBinding, PropertyTrack->GetPropertyBinding())
					.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
					.AddConditional(TracksComponents->PropertyNotify, FPropertyNotifyComponentData{ NotifyFunctionName }, !NotifyFunctionName.IsNone()));
			}
			else if (ensure(FMovieScenePropertyTrackEntityImportHelper::IsEditConditionToggleID(Params)))
			{
				// We effectively discard the builder we've been setting up, because we just
				// need to import the edit condition toggle entity.
				FMovieScenePropertyTrackEntityImportHelper::ImportEditConditionToggleEntity(Params, OutImportedEntity);
			}
		}
		else
		{
			OutImportedEntity->AddBuilder(
				Builder
				.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid()));
		}
	}

protected:

	TEntityBuilder<T...> Builder;
	FComponentTypeID PropertyTag;
};


/**
 * The starting point for TPropertyTrackEntityImportHelperImpl<...T>
 */
template<>
struct TPropertyTrackEntityImportHelperImpl<>
{
	TPropertyTrackEntityImportHelperImpl(FComponentTypeID InPropertyTag)
		: Builder(FEntityBuilder().AddTag(InPropertyTag))
		, PropertyTag(InPropertyTag)
	{
	}

	template<typename U, typename PayloadType>
	TPropertyTrackEntityImportHelperImpl<FAdd, TAdd<U>> Add(TComponentTypeID<U> ComponentType, PayloadType&& InPayload)
	{
		return TPropertyTrackEntityImportHelperImpl<FAdd, TAdd<U>>(Builder.Add(ComponentType, InPayload), PropertyTag);
	}

	template<typename U, typename PayloadType>
	TPropertyTrackEntityImportHelperImpl<FAdd, TAddConditional<U>> AddConditional(TComponentTypeID<U> ComponentType, PayloadType&& InPayload, bool bCondition)
	{
		return TPropertyTrackEntityImportHelperImpl<FAdd, TAddConditional<U>>(Builder.AddConditional(ComponentType, InPayload, bCondition), PropertyTag);
	}

protected:

	TEntityBuilder<FAdd> Builder;
	FComponentTypeID PropertyTag;
};

struct FPropertyTrackEntityImportHelper : TPropertyTrackEntityImportHelperImpl<>
{
	template<typename PropertyTraits>
	FPropertyTrackEntityImportHelper(const TPropertyComponents<PropertyTraits>& PropertyComponents)
		: TPropertyTrackEntityImportHelperImpl<>(PropertyComponents.PropertyTag)
	{}
};

struct FPropertyTrackEntityImportHelperParamsImpl
{
	FPropertyTrackEntityImportHelperParamsImpl(TScriptInterface<IMovieSceneChannelOverrideProvider> InRegistryProvider)
		: RegistryProvider(InRegistryProvider)
	{
		if (InRegistryProvider)
		{
			Registry = InRegistryProvider->GetChannelOverrideRegistry(false);
		}
	}
	TScriptInterface<IMovieSceneChannelOverrideProvider> RegistryProvider;
	TObjectPtr<UMovieSceneSectionChannelOverrideRegistry> Registry;
	TArray<FChannelOverrideEntityImportParams> ImportParams;
};

/**
 * Same as TPropertyTrackEntityImportHelperImpl but with support for overridable channels.
 */
template<typename... T>
struct TPropertyTrackWithOverridableChannelsEntityImportHelperImpl
{
	TPropertyTrackWithOverridableChannelsEntityImportHelperImpl(TEntityBuilder<T...>&& InBuilder, FPropertyTrackEntityImportHelperParamsImpl& InOverrideInfo, FComponentTypeID InPropertyTag = {})
		: Builder(MoveTemp(InBuilder))
		, OverrideInfo(InOverrideInfo)
		, PropertyTag(InPropertyTag)
	{
	}

	template<typename U, typename PayloadType>
	TPropertyTrackEntityImportHelperImpl<T..., TAddConditional<U>> Add(TComponentTypeID<U> ComponentType, PayloadType&& InPayload, FChannelOverrideEntityImportParams OverrideParams)
	{
		const bool bOverriden = (OverrideInfo.Registry && OverrideInfo.Registry->ContainsChannel(OverrideParams.ChannelName));
		if (bOverriden)
		{
			OverrideInfo.ImportParams.Add(MoveTemp(OverrideParams));
		}
		return TPropertyTrackEntityImportHelperImpl<T..., TAddConditional<U>>(Builder.Add(ComponentType, InPayload, !bOverriden), OverrideInfo, PropertyTag);
	}

	template<typename U, typename PayloadType>
	TPropertyTrackEntityImportHelperImpl<T..., TAddConditional<U>> AddConditional(TComponentTypeID<U> ComponentType, PayloadType&& InPayload, bool bCondition, FChannelOverrideEntityImportParams OverrideParams)
	{
		const bool bOverriden = (OverrideInfo.Registry && OverrideInfo.Registry->ContainsChannel(OverrideParams.ChannelName));
		if (bOverriden)
		{
			OverrideInfo.ImportParams.Add(MoveTemp(OverrideParams));
		}
		return TPropertyTrackEntityImportHelperImpl<T..., TAddConditional<U>>(Builder.AddConditional(ComponentType, InPayload, bCondition && !bOverriden), OverrideInfo, PropertyTag);
	}

	void Commit(const UMovieSceneSection* InSection, const UE::MovieScene::FEntityImportParams& Params, UE::MovieScene::FImportedEntity* OutImportedEntity)
	{
		const FGuid ObjectBindingID = Params.GetObjectBindingID();
		const FMovieSceneEvaluationFieldEntityMetaData* MetaData = Params.EntityMetaData;
		const FBuiltInComponentTypes* BuiltInComponents = FBuiltInComponentTypes::Get();
		const FMovieSceneTracksComponentTypes* TracksComponents = FMovieSceneTracksComponentTypes::Get();

		if (UMovieScenePropertyTrack* PropertyTrack = Cast<UMovieScenePropertyTrack>(InSection->GetOuter()))
		{
			if (FMovieScenePropertyTrackEntityImportHelper::IsPropertyValueID(Params))
			{
				const FName NotifyFunctionName = (MetaData ? MetaData->NotifyFunctionName : NAME_None);

				OutImportedEntity->AddBuilder(
					Builder
					.Add(BuiltInComponents->PropertyBinding, PropertyTrack->GetPropertyBinding())
					.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
					.AddConditional(TracksComponents->PropertyNotify, FPropertyNotifyComponentData{ NotifyFunctionName }, !NotifyFunctionName.IsNone()));
			}
			else if (FMovieScenePropertyTrackEntityImportHelper::IsEditConditionToggleID(Params))
			{
				// We effectively discard the builder we've been setting up, because we just
				// need to import the edit condition toggle entity.
				FMovieScenePropertyTrackEntityImportHelper::ImportEditConditionToggleEntity(Params, OutImportedEntity);
			}
			else if (ensure(OverrideInfo.Registry))
			{
				// This should be an overriden channel, which goes into a different entity.
				if (ensure(OverrideInfo.RegistryProvider))
				{
					auto BaseBuilder = FEntityBuilder()
						.Add(BuiltInComponents->PropertyBinding, PropertyTrack->GetPropertyBinding())
						.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid())
						.AddTag(PropertyTag);
					OutImportedEntity->AddBuilder(BaseBuilder);
				
					FChannelOverrideProviderTraitsHandle Traits = OverrideInfo.RegistryProvider->GetChannelOverrideProviderTraits();
					check(Traits.IsValid());
					const FName ChannelOverrideName = Traits->GetChannelOverrideName(Params.EntityID);
					auto* ChannelOverrideParams = OverrideInfo.ImportParams.FindByPredicate(
							[=](const FChannelOverrideEntityImportParams& CurParams)
							{
								return CurParams.ChannelName == ChannelOverrideName;
							});
					if (ensure(ChannelOverrideParams))
					{
						OverrideInfo.Registry->ImportEntityImpl(*ChannelOverrideParams, Params, OutImportedEntity);
					}
				}
			}
		}
		else
		{
			OutImportedEntity->AddBuilder(
				Builder
				.AddConditional(BuiltInComponents->GenericObjectBinding, ObjectBindingID, ObjectBindingID.IsValid()));
		}
	}

protected:

	TEntityBuilder<T...> Builder;
	FPropertyTrackEntityImportHelperParamsImpl OverrideInfo;
	FComponentTypeID PropertyTag;
};


/**
 * The starting point for TPropertyTrackWithOverridableChannelsEntityImportHelperImpl<...T>
 */
template<>
struct TPropertyTrackWithOverridableChannelsEntityImportHelperImpl<>
{
	TPropertyTrackWithOverridableChannelsEntityImportHelperImpl(FComponentTypeID InPropertyTag, TScriptInterface<IMovieSceneChannelOverrideProvider> InRegistryProvider)
		: Builder(FEntityBuilder().AddTag(InPropertyTag))
		, OverrideInfo(InRegistryProvider)
		, PropertyTag(InPropertyTag)
	{
	}

	template<typename U, typename PayloadType>
	TPropertyTrackWithOverridableChannelsEntityImportHelperImpl<FAdd, TAddConditional<U>> Add(TComponentTypeID<U> ComponentType, PayloadType&& InPayload, FChannelOverrideEntityImportParams OverrideParams)
	{
		const bool bOverriden = (OverrideInfo.Registry && OverrideInfo.Registry->ContainsChannel(OverrideParams.ChannelName));
		if (bOverriden)
		{
			OverrideInfo.ImportParams.Add(MoveTemp(OverrideParams));
		}
		return TPropertyTrackWithOverridableChannelsEntityImportHelperImpl<FAdd, TAddConditional<U>>(Builder.AddConditional(ComponentType, InPayload, !bOverriden), OverrideInfo, PropertyTag);
	}

	template<typename U, typename PayloadType>
	TPropertyTrackWithOverridableChannelsEntityImportHelperImpl<FAdd, TAddConditional<U>> AddConditional(TComponentTypeID<U> ComponentType, PayloadType&& InPayload, bool bCondition, FChannelOverrideEntityImportParams OverrideParams)
	{
		const bool bOverriden = (OverrideInfo.Registry && OverrideInfo.Registry->ContainsChannel(OverrideParams.ChannelName));
		if (bOverriden)
		{
			OverrideInfo.ImportParams.Add(MoveTemp(OverrideParams));
		}
		return TPropertyTrackWithOverridableChannelsEntityImportHelperImpl<FAdd, TAddConditional<U>>(Builder.AddConditional(ComponentType, InPayload, bCondition && !bOverriden), OverrideInfo, PropertyTag);
	}

protected:

	TEntityBuilder<FAdd> Builder;
	FPropertyTrackEntityImportHelperParamsImpl OverrideInfo;
	FComponentTypeID PropertyTag;
};

struct FPropertyTrackWithOverridableChannelsEntityImportHelper : TPropertyTrackWithOverridableChannelsEntityImportHelperImpl<>
{
	template<typename PropertyTraits>
	FPropertyTrackWithOverridableChannelsEntityImportHelper(const TPropertyComponents<PropertyTraits>& PropertyComponents, TScriptInterface<IMovieSceneChannelOverrideProvider> InRegistryProvider)
		: TPropertyTrackWithOverridableChannelsEntityImportHelperImpl<>(PropertyComponents.PropertyTag, InRegistryProvider)
	{}
};

}
}

