// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneRotatorTrack.generated.h"

/** Movie scene track that animates an FRotator property */
UCLASS(MinimalAPI)
class UMovieSceneRotatorTrack : public UMovieScenePropertyTrack
{
	GENERATED_BODY()

public:
	UMovieSceneRotatorTrack(const FObjectInitializer& InObjectInitializer);

	//~ Begin UMovieSceneTrack
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> InSectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual void AddSection(UMovieSceneSection& InSection) override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& InSection) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveAllAnimationData() override;
	virtual void RemoveSection(UMovieSceneSection& InSection) override;
	virtual void RemoveSectionAt(int32 InSectionIndex) override;
	//~ End UMovieSceneTrack
};
