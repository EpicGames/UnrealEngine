// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "MovieSceneNameableTrack.h"
#include "Sections/MovieSceneSkeletalAnimationSection.h"
#include "Tracks/MovieSceneCommonAnimationTrack.h"
#include "MovieSceneSkeletalAnimationTrack.generated.h"

enum class ESwapRootBone : uint8;

/**
 * Handles animation of skeletal mesh actors
 */
UCLASS(MinimalAPI)
class UMovieSceneSkeletalAnimationTrack
	: public UMovieSceneCommonAnimationTrack
{
	GENERATED_UCLASS_BODY()

public:

	/** Gets the animation sections at a certain time */
	MOVIESCENETRACKS_API TArray<UMovieSceneSection*> GetAnimSectionsAtTime(FFrameNumber Time);

public:

	// UMovieSceneTrack interface

	virtual void PostLoad() override;
	virtual bool PopulateEvaluationTree(TMovieSceneEvaluationTree<FMovieSceneTrackEvaluationData>& OutData) const override;
	virtual bool ShouldUseRootMotions() const override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
#endif

public:

	UPROPERTY()
	bool bUseLegacySectionIndexBlend;

	/** If on the root bone transform will be swapped to the specified root*/
	UPROPERTY(BlueprintReadWrite, EditAnywhere, Setter,Getter, Category = "Root Motions")
	ESwapRootBone SwapRootBone;

	UFUNCTION()
	MOVIESCENETRACKS_API void SetSwapRootBone(ESwapRootBone InValue);
	UFUNCTION()
	MOVIESCENETRACKS_API ESwapRootBone GetSwapRootBone() const;

public:
	static MOVIESCENETRACKS_API FColor LinkedAnimSeqTrackColor;

};








