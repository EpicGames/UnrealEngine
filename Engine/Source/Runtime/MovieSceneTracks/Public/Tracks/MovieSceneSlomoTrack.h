// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieSceneFloatTrack.h"
#include "MovieSceneSlomoTrack.generated.h"

/**
 * Implements a movie scene track that controls a movie scene's world time dilation.
 */
UCLASS(MinimalAPI)
class UMovieSceneSlomoTrack
	: public UMovieSceneFloatTrack
{
	GENERATED_BODY()

public:
	
	UMovieSceneSlomoTrack(const FObjectInitializer& Init);

	// UMovieSceneTrack interface

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

#if WITH_EDITORONLY_DATA
	virtual FText GetDefaultDisplayName() const override;
	virtual bool CanRename() const override { return true; }
#endif
};
