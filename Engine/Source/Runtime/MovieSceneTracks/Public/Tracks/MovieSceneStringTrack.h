// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneStringTrack.generated.h"

/**
 * Implements a movie scene track that holds a series of strings.
 */
UCLASS(MinimalAPI)
class UMovieSceneStringTrack
	: public UMovieScenePropertyTrack
{
	GENERATED_BODY()

public:

	/** Default constructor. */
	UMovieSceneStringTrack()
	{
#if WITH_EDITORONLY_DATA
		TrackTint = FColor(128, 128, 128);
#endif
	}

public:

	//~ UMovieSceneTrack interface

	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveAllAnimationData() override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
};
