// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneTextTrack.generated.h"

/** Movie Scene Track that holds a series of texts. */
UCLASS(MinimalAPI)
class UMovieSceneTextTrack : public UMovieScenePropertyTrack
{
	GENERATED_BODY()

public:
	UMovieSceneTextTrack();

	//~ Begin UMovieSceneTrack
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual void AddSection(UMovieSceneSection& Section) override;
	virtual const TArray<UMovieSceneSection*>& GetAllSections() const override;
	virtual bool HasSection(const UMovieSceneSection& Section) const override;
	virtual bool IsEmpty() const override;
	virtual void RemoveAllAnimationData() override;
	virtual void RemoveSection(UMovieSceneSection& Section) override;
	virtual void RemoveSectionAt(int32 SectionIndex) override;
	//~ End UMovieSceneTrack
};
