// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieScenePropertyTrack.h"
#include "MovieSceneTransformTrack.generated.h"

struct FMovieSceneInterrogationKey;

/**
 * Handles manipulation of 3D transform properties in a movie scene
 */
UCLASS(MinimalAPI)
class UMovieSceneTransformTrack
	: public UMovieScenePropertyTrack
{
	GENERATED_UCLASS_BODY()

public:

	// UMovieSceneTrack interface

	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
};

