// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieScenePropertyTrack.h"

#include "MovieSceneVectorTrack.generated.h"

/**
 * Handles manipulation of float vectors in a movie scene
 */
UCLASS(MinimalAPI)
class UMovieSceneFloatVectorTrack : public UMovieScenePropertyTrack
{
	GENERATED_UCLASS_BODY()

public:
	/** UMovieSceneTrack interface */
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

	virtual void InitializeFromProperty(const FProperty* Property, const UE::MovieScene::FPropertyDefinition* Definition) override;

	/** @return Get the number of channels used by the vector */
	int32 GetNumChannelsUsed() const { return NumChannelsUsed; }

	/** Set the number of channels used by the vector */
	void SetNumChannelsUsed( int32 InNumChannelsUsed ) { NumChannelsUsed = InNumChannelsUsed; }

private:
	/** The number of channels used by the vector (2,3, or 4) */
	UPROPERTY()
	int32 NumChannelsUsed;
};

/**
 * Handles manipulation of double vectors in a movie scene
 */
UCLASS(MinimalAPI)
class UMovieSceneDoubleVectorTrack : public UMovieScenePropertyTrack
{
	GENERATED_UCLASS_BODY()

public:
	/** UMovieSceneTrack interface */
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;

	virtual void InitializeFromProperty(const FProperty* Property, const UE::MovieScene::FPropertyDefinition* Definition) override;

	/** @return Get the number of channels used by the vector */
	int32 GetNumChannelsUsed() const { return NumChannelsUsed; }

	/** Set the number of channels used by the vector */
	void SetNumChannelsUsed( int32 InNumChannelsUsed ) { NumChannelsUsed = InNumChannelsUsed; }

private:
	/** The number of channels used by the vector (2,3, or 4) */
	UPROPERTY()
	int32 NumChannelsUsed;
};
