// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/ObjectMacros.h"
#include "Tracks/MovieScenePropertyTrack.h"

#include "MovieSceneVisibilityTrack.generated.h"

/**
 * Handles manipulation of visibility properties in a movie scene
 */
UCLASS(MinimalAPI)
class UMovieSceneVisibilityTrack
	: public UMovieScenePropertyTrack
{
	GENERATED_UCLASS_BODY()

public:

	// UMovieSceneTrack interface
	virtual void PostLoad() override;
	virtual bool SupportsType(TSubclassOf<UMovieSceneSection> SectionClass) const override;
	virtual UMovieSceneSection* CreateNewSection() override;
	virtual void Serialize(FArchive& Ar) override;

#if WITH_EDITOR
	virtual void ProcessSourceValueForKeying(const FProperty* Property, UE::MovieScene::FSourcePropertyValue& InOutValue) const override;
#endif

#if WITH_EDITORONLY_DATA
	virtual FText GetDisplayName() const override;
#endif
};
